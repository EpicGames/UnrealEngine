{
	"FileVersion": 3,

	"FriendlyName": "Online Services",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Shared code for interacting with online services implementations.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesInterface",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OnlineServicesCommon",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "OnlineServicesCommonEngineUtils",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineBase",
			"Enabled": true
		}
	]
}