// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineServicesCommon : ModuleRules
{
	public OnlineServicesCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		NumIncludedBytesPerUnityCPPOverride = 136608; // best unity size found from using UBT ProfileUnitySizes mode

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
				"OnlineBase",
				"OnlineServicesInterface"
			}
		);

		// OnlineService cannot depend on Engine!
		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject"		// CoreUObject temporary dependency
			}
		);
	}
}
