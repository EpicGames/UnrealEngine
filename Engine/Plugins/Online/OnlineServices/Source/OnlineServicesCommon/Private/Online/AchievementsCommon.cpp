// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AchievementsCommon.h"

#include "Online/OnlineUtils.h"
#include "Online/StatsCommon.h"

namespace UE::Online {

bool FAchievementUnlockRule::ContainsStat(const FString& StatName) const
{
	return Conditions.ContainsByPredicate([&StatName](const FAchievementUnlockCondition& Condition) { return Condition.StatName == StatName; });
}

FAchievementsCommon::FAchievementsCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Achievements"), InServices)
{
}

void FAchievementsCommon::Initialize()
{
	TOnlineComponent<IAchievements>::Initialize();

	if (IStats* StatsInterface = Services.Get<IStats>())
	{
		StatEventHandle = StatsInterface->OnStatsUpdated().Add([this](const FStatsUpdated& StatsUpdated) { UnlockAchievementsByStats(StatsUpdated); });
	}
}

void FAchievementsCommon::Shutdown()
{
	StatEventHandle.Unbind();

	TOnlineComponent<IAchievements>::Shutdown();
}

void FAchievementsCommon::UpdateConfig()
{
	TOnlineComponent<IAchievements>::UpdateConfig();
	TOnlineComponent<IAchievements>::LoadConfig(Config);
}

void FAchievementsCommon::RegisterCommands()
{
	RegisterCommand(&FAchievementsCommon::QueryAchievementDefinitions);
	RegisterCommand(&FAchievementsCommon::GetAchievementIds);
	RegisterCommand(&FAchievementsCommon::GetAchievementDefinition);
	RegisterCommand(&FAchievementsCommon::QueryAchievementStates);
	RegisterCommand(&FAchievementsCommon::GetAchievementState);
	RegisterCommand(&FAchievementsCommon::UnlockAchievements);
	RegisterCommand(&FAchievementsCommon::DisplayAchievementUI);
}

TOnlineAsyncOpHandle<FQueryAchievementDefinitions> FAchievementsCommon::QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementDefinitions> Operation = GetOp<FQueryAchievementDefinitions>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());  
	return Operation->GetHandle();
}

void FAchievementsCommon::OnAchievementStatesQueried(const FAccountId& AccountId)
{
	if (!Config.bIsTitleManaged)
	{
		return;
	}

	if (IStats* Stats = Services.Get<IStats>())
	{
		TOnlineResult<FGetCachedStats> CachedStatsResult = Stats->GetCachedStats({});
		check(CachedStatsResult.IsOk());
		const TArray<FUserStats>& UsersStats = CachedStatsResult.GetOkValue().UsersStats;
		for (const FUserStats& UserStats : UsersStats)
		{
			if (UserStats.AccountId == AccountId)
			{
				FStatsUpdated StatsUpdated;
				StatsUpdated.LocalAccountId = AccountId;
				StatsUpdated.UpdateUsersStats.Add(UserStats);
				UnlockAchievementsByStats(StatsUpdated);
			}
		}
	}
}

TOnlineResult<FGetAchievementIds> FAchievementsCommon::GetAchievementIds(FGetAchievementIds::Params&& Params)
{
	return TOnlineResult<FGetAchievementIds>(Errors::NotImplemented());
}

TOnlineResult<FGetAchievementDefinition> FAchievementsCommon::GetAchievementDefinition(FGetAchievementDefinition::Params&& Params)
{
	return TOnlineResult<FGetAchievementDefinition>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FQueryAchievementStates> FAchievementsCommon::QueryAchievementStates(FQueryAchievementStates::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementStates> Operation = GetOp<FQueryAchievementStates>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetAchievementState> FAchievementsCommon::GetAchievementState(FGetAchievementState::Params&& Params) const
{
	return TOnlineResult<FGetAchievementState>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FUnlockAchievements> FAchievementsCommon::UnlockAchievements(FUnlockAchievements::Params&& Params)
{
	TOnlineAsyncOpRef<FUnlockAchievements> Operation = GetOp<FUnlockAchievements>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FDisplayAchievementUI> FAchievementsCommon::DisplayAchievementUI(FDisplayAchievementUI::Params&& Params)
{
	return TOnlineResult<FDisplayAchievementUI>(Errors::NotImplemented());
}

TOnlineEvent<void(const FAchievementStateUpdated&)> FAchievementsCommon::OnAchievementStateUpdated()
{
	return OnAchievementStateUpdatedEvent;
}

void FAchievementsCommon::UnlockAchievementsByStats(const FStatsUpdated& StatsUpdated)
{
	if (!Config.bIsTitleManaged)
	{
		return;
	}

	IStats* StatsInterface = Services.Get<IStats>();
	if (!StatsInterface)
	{
		return;
	}

	TArray<FString> StatNames;
	TArray<FAccountId> AccountIds;
	for (const FUserStats& UserStats : StatsUpdated.UpdateUsersStats)
	{
		if (AchievementStates.Find(UserStats.AccountId))
		{
			for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
			{
				for (const FAchievementUnlockRule& AchievementUnlockRule : Config.UnlockRules)
				{
					if (AchievementUnlockRule.ContainsStat(StatPair.Key))
					{
						for (const FAchievementUnlockCondition& Condition : AchievementUnlockRule.Conditions)
						{
							StatNames.AddUnique(Condition.StatName);
						}
					}
				}
			}

			AccountIds.AddUnique(UserStats.AccountId);
		}
	}

	if (StatNames.IsEmpty() || AccountIds.IsEmpty())
	{
		return;
	}

	FBatchQueryStats::Params BatchQueryStatsParam;
	BatchQueryStatsParam.LocalAccountId = StatsUpdated.LocalAccountId;
	BatchQueryStatsParam.TargetAccountIds = MoveTemp(AccountIds);
	BatchQueryStatsParam.StatNames = MoveTemp(StatNames);

	StatsInterface->BatchQueryStats(MoveTemp(BatchQueryStatsParam))
	.OnComplete([this](const TOnlineResult<FBatchQueryStats>& Result)
	{
		if (Result.IsOk())
		{
			const FBatchQueryStats::Result& BatchQueryStatsResult = Result.GetOkValue();
			for (const FUserStats& UserStats : BatchQueryStatsResult.UsersStats)
			{
				TArray<FString> AchievementsToUnlock;
				for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
				{
					ExecuteUnlockRulesRelatedToStat(UserStats.AccountId, StatPair.Key, UserStats.Stats, AchievementsToUnlock);
				}

				if (!AchievementsToUnlock.IsEmpty())
				{
					FUnlockAchievements::Params UnlockAchievementsParams;
					UnlockAchievementsParams.LocalAccountId = UserStats.AccountId;
					UnlockAchievementsParams.AchievementIds = MoveTemp(AchievementsToUnlock);
					UnlockAchievements(MoveTemp(UnlockAchievementsParams));
				}
			}
		}
	});
}

void FAchievementsCommon::ExecuteUnlockRulesRelatedToStat(const FAccountId& AccountId, const FString& StatName, const TMap<FString, FStatValue>& Stats, TArray<FString>& OutAchievementsToUnlock)
{
	if (!Config.bIsTitleManaged)
	{
		return;
	}

	for (const FAchievementUnlockRule& AchievementUnlockRule : Config.UnlockRules)
	{
		if (AchievementUnlockRule.ContainsStat(StatName)
			&& !IsUnlocked(AccountId, AchievementUnlockRule.AchievementId) 
			&& MeetUnlockCondition(AchievementUnlockRule, Stats))
		{
			OutAchievementsToUnlock.AddUnique(AchievementUnlockRule.AchievementId);
		}
	}
}

bool FAchievementsCommon::MeetUnlockCondition(const FAchievementUnlockRule& AchievementUnlockRule, const TMap<FString, FStatValue>& Stats)
{
	if (!Config.bIsTitleManaged)
	{
		return false;
	}

	FStatsCommon* StatsInterface = Services.Get<FStatsCommon>();
	if (!StatsInterface)
	{
		return false;
	}

	for (const FAchievementUnlockCondition& Condition : AchievementUnlockRule.Conditions)
	{
		if (const FStatDefinition* StatDefinition = StatsInterface->GetStatDefinition(Condition.StatName))
		{
			const FStatValue* StatValue = Stats.Find(Condition.StatName);
			if (!StatValue)
			{
				UE_LOGF(LogOnlineServices, Warning, "Can't find stat %ls when check if it can unlock achievement.", *Condition.StatName);
				return false;
			}

			switch (StatDefinition->ModifyMethod)
			{
			case EStatModifyMethod::Sum: // Intentional fall through
			case EStatModifyMethod::Largest:
				if (StatValue->GetInt64() < Condition.UnlockThreshold.GetInt64())
				{
					return false;
				}
				break;
			case EStatModifyMethod::Set:
				if (StatValue->GetInt64() != Condition.UnlockThreshold.GetInt64())
				{
					return false;
				}
				break;
			case EStatModifyMethod::Smallest:
				if (StatValue->GetInt64() > Condition.UnlockThreshold.GetInt64())
				{
					return false;
				}
				break;
			}
		}
	}

	return true;
}

bool FAchievementsCommon::IsUnlocked(const FAccountId& AccountId, const FString& AchievementName) const
{
	FGetAchievementState::Params Params;
	Params.LocalAccountId = AccountId;
	Params.AchievementId = AchievementName;
	TOnlineResult<FGetAchievementState> Result = GetAchievementState(MoveTemp(Params));
	if (Result.IsOk())
	{
		const FAchievementState& AchievementState = Result.GetOkValue().AchievementState;
		return FMath::IsNearlyEqual(AchievementState.Progress, 1.0f);
	}

	UE_LOGF(LogOnlineServices, Warning, "Can't find state of achievement %ls when check if it's unlocked.", *AchievementName);

	return false;
}

/* UE::Online */ }
