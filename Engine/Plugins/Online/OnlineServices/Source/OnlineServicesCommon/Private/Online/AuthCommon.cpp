// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AuthCommon.h"

#include "Algo/Transform.h"
#include "Misc/ScopeRWLock.h"
#include "Online/OnlineUtils.h"

namespace UE::Online {

TSharedPtr<FAccountInfo> FAccountInfoRegistry::Find(FPlatformUserId PlatformUserId) const
{
	FReadScopeLock Lock(IndexLock);
	return FindNoLock(PlatformUserId);
}

TSharedPtr<FAccountInfo> FAccountInfoRegistry::Find(FAccountId AccountId) const
{
	FReadScopeLock Lock(IndexLock);
	return FindNoLock(AccountId);
}

TArray<TSharedRef<FAccountInfo>> FAccountInfoRegistry::GetAllAccountInfo(TFunction<bool(const TSharedRef<FAccountInfo>&)> Predicate) const
{
	FReadScopeLock Lock(IndexLock);
	TArray<TSharedRef<FAccountInfo>> Result;
	Result.Reserve(AuthDataByPlatformUserId.Num());
	Algo::TransformIf(
		AuthDataByPlatformUserId,
		Result,
		[Predicate](const TPair<FPlatformUserId, TSharedRef<FAccountInfo>>& Value)
		{
			return Predicate(Value.Get<1>());
		},
		[](const TPair<FPlatformUserId, TSharedRef<FAccountInfo>>& Value)
		{
			return Value.Get<1>();
		});
	return Result;
}

TSharedPtr<FAccountInfo> FAccountInfoRegistry::FindNoLock(FPlatformUserId PlatformUserId) const
{
	const TSharedRef<FAccountInfo>* FoundPtr = AuthDataByPlatformUserId.Find(PlatformUserId);
	return FoundPtr ? TSharedPtr<FAccountInfo>(*FoundPtr) : TSharedPtr<FAccountInfo>();
}

TSharedPtr<FAccountInfo> FAccountInfoRegistry::FindNoLock(FAccountId AccountId) const
{
	const TSharedRef<FAccountInfo>* FoundPtr = AuthDataByOnlineAccountIdHandle.Find(AccountId);
	return FoundPtr ? TSharedPtr<FAccountInfo>(*FoundPtr) : TSharedPtr<FAccountInfo>();
}

void FAccountInfoRegistry::DoRegister(const TSharedRef<FAccountInfo>& AccountInfo)
{
	if (ensure(AccountInfo->PlatformUserId != PLATFORMUSERID_NONE))
	{
		AuthDataByPlatformUserId.Add(AccountInfo->PlatformUserId, AccountInfo);
	}

	if (ensure(AccountInfo->AccountId.IsValid()))
	{
		AuthDataByOnlineAccountIdHandle.Add(AccountInfo->AccountId, AccountInfo);
	}
}
void FAccountInfoRegistry::DoUnregister(const TSharedRef<FAccountInfo>& AccountInfo)
{
	if (ensure(AccountInfo->PlatformUserId != PLATFORMUSERID_NONE))
	{
		AuthDataByPlatformUserId.Remove(AccountInfo->PlatformUserId);
	}

	if (ensure(AccountInfo->AccountId.IsValid()))
	{
		AuthDataByOnlineAccountIdHandle.Remove(AccountInfo->AccountId);
	}
}

FAuthCommon::FAuthCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Auth"), InServices)
{
}

void FAuthCommon::RegisterCommands()
{
	RegisterCommand(&FAuthCommon::Login);
	RegisterCommand(&FAuthCommon::Logout);
	RegisterCommand(&FAuthCommon::CreateAccount);
	RegisterCommand(&FAuthCommon::LinkAccount);
	RegisterCommand(&FAuthCommon::ModifyAccountAttributes);
	RegisterCommand(&FAuthCommon::QueryExternalServerAuthTicket);
	RegisterCommand(&FAuthCommon::QueryExternalAuthToken);
	RegisterCommand(&FAuthCommon::QueryVerifiedAuthTicket);
	RegisterCommand(&FAuthCommon::CancelVerifiedAuthTicket);
	RegisterCommand(&FAuthCommon::BeginVerifiedAuthSession);
	RegisterCommand(&FAuthCommon::EndVerifiedAuthSession);
	RegisterCommand(&FAuthCommon::GetLocalOnlineUserByOnlineAccountId);
	RegisterCommand(&FAuthCommon::GetLocalOnlineUserByPlatformUserId);
	RegisterCommand(&FAuthCommon::GetAllLocalOnlineUsers);
	RegisterCommand(&FAuthCommon::GetLinkAccountContinuationId);
	RegisterCommand(&FAuthCommon::GetRelyingParty);
}

TOnlineAsyncOpHandle<FAuthLogin> FAuthCommon::Login(FAuthLogin::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogin> Operation = GetOp<FAuthLogin>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthLogout> FAuthCommon::Logout(FAuthLogout::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogout> Operation = GetOp<FAuthLogout>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthCreateAccount> FAuthCommon::CreateAccount(FAuthCreateAccount::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthCreateAccount> Operation = GetOp<FAuthCreateAccount>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthLinkAccount> FAuthCommon::LinkAccount(FAuthLinkAccount::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLinkAccount> Operation = GetOp<FAuthLinkAccount>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthModifyAccountAttributes> FAuthCommon::ModifyAccountAttributes(FAuthModifyAccountAttributes::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthModifyAccountAttributes> Operation = GetOp<FAuthModifyAccountAttributes>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> FAuthCommon::QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalServerAuthTicket> Operation = GetOp<FAuthQueryExternalServerAuthTicket>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> FAuthCommon::QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalAuthToken> Operation = GetOp<FAuthQueryExternalAuthToken>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryVerifiedAuthTicket> FAuthCommon::QueryVerifiedAuthTicket(FAuthQueryVerifiedAuthTicket::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryVerifiedAuthTicket> Operation = GetOp<FAuthQueryVerifiedAuthTicket>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthCancelVerifiedAuthTicket> FAuthCommon::CancelVerifiedAuthTicket(FAuthCancelVerifiedAuthTicket::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthCancelVerifiedAuthTicket> Operation = GetOp<FAuthCancelVerifiedAuthTicket>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthBeginVerifiedAuthSession> FAuthCommon::BeginVerifiedAuthSession(FAuthBeginVerifiedAuthSession::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthBeginVerifiedAuthSession> Operation = GetOp<FAuthBeginVerifiedAuthSession>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthEndVerifiedAuthSession> FAuthCommon::EndVerifiedAuthSession(FAuthEndVerifiedAuthSession::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthEndVerifiedAuthSession> Operation = GetOp<FAuthEndVerifiedAuthSession>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> FAuthCommon::GetLocalOnlineUserByPlatformUserId(FAuthGetLocalOnlineUserByPlatformUserId::Params&& Params) const
{
	TSharedPtr<FAccountInfo> AccountInfo = GetAccountInfoRegistry().Find(Params.PlatformUserId);
	return (AccountInfo && IsOnlineStatus(AccountInfo->LoginStatus)) ?
		TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId>(FAuthGetLocalOnlineUserByPlatformUserId::Result{ AccountInfo.ToSharedRef() }) :
		TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId>(Errors::InvalidUser());
}

TOnlineResult<FAuthGetLocalOnlineUserByOnlineAccountId> FAuthCommon::GetLocalOnlineUserByOnlineAccountId(FAuthGetLocalOnlineUserByOnlineAccountId::Params&& Params) const
{
	TSharedPtr<FAccountInfo> AccountInfo = GetAccountInfoRegistry().Find(Params.LocalAccountId);
	return (AccountInfo && IsOnlineStatus(AccountInfo->LoginStatus)) ?
		TOnlineResult<FAuthGetLocalOnlineUserByOnlineAccountId>(FAuthGetLocalOnlineUserByOnlineAccountId::Result{ AccountInfo.ToSharedRef() }) :
		TOnlineResult<FAuthGetLocalOnlineUserByOnlineAccountId>(Errors::InvalidUser());
}

TOnlineResult<FAuthGetAllLocalOnlineUsers> FAuthCommon::GetAllLocalOnlineUsers(FAuthGetAllLocalOnlineUsers::Params&& Params) const
{
	return TOnlineResult<FAuthGetAllLocalOnlineUsers>(FAuthGetAllLocalOnlineUsers::Result{
		GetAccountInfoRegistry().GetAllAccountInfo([](const TSharedRef<FAccountInfo>& AccountInfo) { return IsOnlineStatus(AccountInfo->LoginStatus); }) });
}

TOnlineResult<FAuthGetLinkAccountContinuationId> FAuthCommon::GetLinkAccountContinuationId(FAuthGetLinkAccountContinuationId::Params&& Params) const
{
	return TOnlineResult<FAuthGetLinkAccountContinuationId>(Errors::NotImplemented());
}

TOnlineResult<FAuthGetRelyingParty> FAuthCommon::GetRelyingParty(FAuthGetRelyingParty::Params&& Params) const
{
	return TOnlineResult<FAuthGetRelyingParty>(Errors::NotImplemented());
}

TOnlineEvent<void(const FAuthLoginStatusChanged&)> FAuthCommon::OnLoginStatusChanged()
{
	return OnAuthLoginStatusChangedEvent;
}

TOnlineEvent<void(const FAuthPendingAuthExpiration&)> FAuthCommon::OnPendingAuthExpiration()
{
	return OnAuthPendingAuthExpirationEvent;
}

TOnlineEvent<void(const FAuthAccountAttributesChanged&)> FAuthCommon::OnAccountAttributesChanged()
{
	return OnAuthAccountAttributesChangedEvent;
}

bool FAuthCommon::IsLoggedIn(const FAccountId& AccountId) const
{
	const TSharedPtr<FAccountInfo> AccountInfo = GetAccountInfoRegistry().Find(AccountId);
	return (AccountInfo && IsOnlineStatus(AccountInfo->LoginStatus));
}

bool FAuthCommon::IsLoggedIn(const FPlatformUserId& PlatformUserId) const
{
	const TSharedPtr<FAccountInfo> AccountInfo = GetAccountInfoRegistry().Find(PlatformUserId);
	return (AccountInfo && IsOnlineStatus(AccountInfo->LoginStatus));
}

/* UE::Online */ }
