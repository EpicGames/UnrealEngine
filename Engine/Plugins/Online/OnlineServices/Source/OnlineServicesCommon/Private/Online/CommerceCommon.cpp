// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/CommerceCommon.h"
#include "Online/OnlineUtils.h"

namespace UE::Online {

FCommerceCommon::FCommerceCommon(FOnlineServicesCommon& InServices)
: TOnlineComponent(TEXT("Commerce"), InServices)
{
}

void FCommerceCommon::RegisterCommands()
{
	RegisterCommand(&FCommerceCommon::QueryOffers);
	RegisterCommand(&FCommerceCommon::QueryOffersById);
	RegisterCommand(&FCommerceCommon::GetOffers);
	RegisterCommand(&FCommerceCommon::GetOffersById);
	RegisterCommand(&FCommerceCommon::ShowStoreUI);
	RegisterCommand(&FCommerceCommon::Checkout);
	RegisterCommand(&FCommerceCommon::QueryTransactionEntitlements);
	RegisterCommand(&FCommerceCommon::QueryEntitlements);
	RegisterCommand(&FCommerceCommon::GetEntitlements);
	RegisterCommand(&FCommerceCommon::RedeemEntitlement);
	RegisterCommand(&FCommerceCommon::RetrieveS2SToken);
}

TOnlineAsyncOpHandle<FCommerceQueryOffers> FCommerceCommon::QueryOffers(FCommerceQueryOffers::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffers> Op = GetOp<FCommerceQueryOffers>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryOffersById> FCommerceCommon::QueryOffersById(FCommerceQueryOffersById::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryOffersById> Op = GetOp<FCommerceQueryOffersById>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineResult<FCommerceGetOffers> FCommerceCommon::GetOffers(FCommerceGetOffers::Params&& Params)
{
	return TOnlineResult<FCommerceGetOffers>(Errors::NotImplemented());
}

TOnlineResult<FCommerceGetOffersById> FCommerceCommon::GetOffersById(FCommerceGetOffersById::Params&& Params)
{
	return TOnlineResult<FCommerceGetOffersById>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FCommerceShowStoreUI> FCommerceCommon::ShowStoreUI(FCommerceShowStoreUI::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceShowStoreUI> Op = GetOp<FCommerceShowStoreUI>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceCheckout> FCommerceCommon::Checkout(FCommerceCheckout::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceCheckout> Op = GetOp<FCommerceCheckout>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> FCommerceCommon::QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryTransactionEntitlements> Op = GetOp<FCommerceQueryTransactionEntitlements>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryEntitlements> FCommerceCommon::QueryEntitlements(FCommerceQueryEntitlements::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceQueryEntitlements> Op = GetOp<FCommerceQueryEntitlements>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineResult<FCommerceGetEntitlements> FCommerceCommon::GetEntitlements(FCommerceGetEntitlements::Params&& Params)
{
	return TOnlineResult<FCommerceGetEntitlements>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> FCommerceCommon::RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRedeemEntitlement> Op = GetOp<FCommerceRedeemEntitlement>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> FCommerceCommon::RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params)
{
	TOnlineAsyncOpRef<FCommerceRetrieveS2SToken> Op = GetOp<FCommerceRetrieveS2SToken>(MoveTemp(Params));
	Op->SetError(Errors::NotImplemented());
	return Op->GetHandle();
}

TOnlineEvent<void(const FCommerceOnPurchaseComplete&)> FCommerceCommon::OnPurchaseCompleted()
{
	return OnPurchaseCompletedEvent;
}

FText FCommerceCommon::GetFormattedPrice(uint64 Price, int32 DecimalPoint, FString CurrencyCode)
{
	return FText::AsCurrencyBase(Price, CurrencyCode, NULL, DecimalPoint);
}

} // namespace UE::Online
