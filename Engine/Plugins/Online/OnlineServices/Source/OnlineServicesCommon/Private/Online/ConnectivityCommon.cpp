// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ConnectivityCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FConnectivityCommon::FConnectivityCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Connectivity"), InServices)
{
}

void FConnectivityCommon::RegisterCommands()
{
	RegisterCommand(&FConnectivityCommon::GetConnectionStatus);
}

TOnlineResult<FGetConnectionStatus> FConnectivityCommon::GetConnectionStatus(FGetConnectionStatus::Params&& Params)
{
	FGetConnectionStatus::Result Result;
	Result.Status = ConnectionStatus;

	return TOnlineResult<FGetConnectionStatus>(Result);
}

TOnlineEvent<void(const FConnectionStatusChanged&)> FConnectivityCommon::OnConnectionStatusChanged()
{
	return OnConnectionStatusChangedEvent;
}

/* UE::Online */}
