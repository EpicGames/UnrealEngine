// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/ExternalUICommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FExternalUICommon::FExternalUICommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("ExternalUI"), InServices)
{
}

void FExternalUICommon::RegisterCommands()
{
	RegisterCommand(&FExternalUICommon::ShowFriendsUI);
}

TOnlineAsyncOpHandle<FExternalUIShowLoginUI> FExternalUICommon::ShowLoginUI(FExternalUIShowLoginUI::Params&& Params)
{
	TOnlineAsyncOpRef<FExternalUIShowLoginUI> Operation = GetOp<FExternalUIShowLoginUI>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> FExternalUICommon::ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params)
{
	TOnlineAsyncOpRef<FExternalUIShowFriendsUI> Operation = GetOp<FExternalUIShowFriendsUI>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineEvent<void(const FExternalUIStatusChanged&)> FExternalUICommon::OnExternalUIStatusChanged()
{
	return OnExternalUIStatusChangedEvent;
}

/* UE::Online */ }
