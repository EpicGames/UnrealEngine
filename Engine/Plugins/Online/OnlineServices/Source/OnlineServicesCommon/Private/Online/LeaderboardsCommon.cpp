// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LeaderboardsCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

const TCHAR* LexToString(ELeaderboardUpdateMethod Value)
{
	switch (Value)
	{
	case ELeaderboardUpdateMethod::Force:
		return TEXT("Force");
	default: checkNoEntry(); // Intentional fallthrough
	case ELeaderboardUpdateMethod::KeepBest:
		return TEXT("KeepBest");
	}
}

void LexFromString(ELeaderboardUpdateMethod& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("KeepBest")) == 0)
	{
		OutValue = ELeaderboardUpdateMethod::KeepBest;
	}
	else if (FCString::Stricmp(InStr, TEXT("Force")) == 0)
	{
		OutValue = ELeaderboardUpdateMethod::Force;
	}
	else
	{
		ensureMsgf(false, TEXT("Can't convert %s to ELeaderboardUpdateMethod"), InStr);
		OutValue = ELeaderboardUpdateMethod::KeepBest;
	}
}

const TCHAR* LexToString(ELeaderboardOrderMethod Value)
{
	switch (Value)
	{
	case ELeaderboardOrderMethod::Ascending:
		return TEXT("Ascending");
	default: checkNoEntry(); // Intentional fallthrough
	case ELeaderboardOrderMethod::Descending:
		return TEXT("Descending");
	}
}

void LexFromString(ELeaderboardOrderMethod& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Ascending")) == 0)
	{
		OutValue = ELeaderboardOrderMethod::Ascending;
	}
	else if (FCString::Stricmp(InStr, TEXT("Descending")) == 0)
	{
		OutValue = ELeaderboardOrderMethod::Descending;
	}
	else
	{
		ensureMsgf(false, TEXT("Can't convert %s to ELeaderboardOrderMethod"), InStr);
		OutValue = ELeaderboardOrderMethod::Descending;
	}
}

FLeaderboardsCommon::FLeaderboardsCommon(FOnlineServicesCommon& InServices)
	: Super(TEXT("Leaderboards"), InServices)
{
}

void FLeaderboardsCommon::Initialize()
{
	Super::Initialize();

	if (IStats* StatsInterface = Services.Get<IStats>())
	{
		StatEventHandle = StatsInterface->OnStatsUpdated().Add([this](const FStatsUpdated& StatsUpdated) { WriteLeaderboardsByStats(StatsUpdated); });
	}
}

void FLeaderboardsCommon::UpdateConfig()
{
	Super::UpdateConfig();

	FLeaderboardsCommonConfig Config;
	TOnlineComponent::LoadConfig(Config);

	bIsTitleManaged = Config.bIsTitleManaged;

	for (FLeaderboardDefinition& LeaderboardDefinition : Config.LeaderboardDefinitions)
	{
		FString LeaderboardName = LeaderboardDefinition.Name;
		LeaderboardDefinitions.Emplace(MoveTemp(LeaderboardName), MoveTemp(LeaderboardDefinition));
	}
}

void FLeaderboardsCommon::RegisterCommands()
{
	Super::RegisterCommands();

	RegisterCommand(&FLeaderboardsCommon::ReadEntriesForUsers);
	RegisterCommand(&FLeaderboardsCommon::ReadEntriesAroundRank);
	RegisterCommand(&FLeaderboardsCommon::ReadEntriesAroundUser);
}

TOnlineAsyncOpHandle<FReadEntriesForUsers> FLeaderboardsCommon::ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesForUsers> Operation = GetOp<FReadEntriesForUsers>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundRank> FLeaderboardsCommon::ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundRank> Operation = GetOp<FReadEntriesAroundRank>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundUser> FLeaderboardsCommon::ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundUser> Operation = GetOp<FReadEntriesAroundUser>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FWriteLeaderboardScores> FLeaderboardsCommon::WriteLeaderboardScores(FWriteLeaderboardScores::Params&& Params)
{
	TOnlineAsyncOpRef<FWriteLeaderboardScores> Operation = GetOp<FWriteLeaderboardScores>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

void FLeaderboardsCommon::WriteLeaderboardsByStats(const FStatsUpdated& StatsUpdated)
{
	if (!bIsTitleManaged)
	{
		return;
	}

	for (const FUserStats& UserStats : StatsUpdated.UpdateUsersStats)
	{
		for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
		{
			if (LeaderboardDefinitions.Contains(StatPair.Key))
			{
				FWriteLeaderboardScores::Params WriteLeaderboardScoresParam;
				WriteLeaderboardScoresParam.LocalAccountId = UserStats.AccountId;
				WriteLeaderboardScoresParam.BoardName = StatPair.Key;
				WriteLeaderboardScoresParam.Score = StatPair.Value.GetInt64();
				WriteLeaderboardScores(MoveTemp(WriteLeaderboardScoresParam));
			}
		}
	}
}

/* UE::Online */ }
