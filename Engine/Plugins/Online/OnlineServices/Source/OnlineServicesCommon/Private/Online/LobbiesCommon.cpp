// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LobbiesCommon.h"

#include "Online/LobbiesCommonTests.h"
#include "Online/OnlineUtils.h"

namespace UE::Online {

FLobbiesCommon::FLobbiesCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Lobbies"), InServices)
	, SchemaRegistry(MakeShared<FSchemaRegistry>())
{
}

void FLobbiesCommon::Initialize()
{
	TOnlineComponent<ILobbies>::Initialize();

	FSchemaRegistryDescriptorConfig SchemaConfig;
	LoadConfig(SchemaConfig);

	if (!SchemaRegistry->ParseConfig(SchemaConfig))
	{
		UE_LOGF(LogOnlineServices, Error, "[FLobbiesCommon::Initialize] Failed to initialize schema registry");
	}
}

void FLobbiesCommon::RegisterCommands()
{
	TOnlineComponent<ILobbies>::RegisterCommands();

	RegisterCommand(&FLobbiesCommon::CreateLobby);
	RegisterCommand(&FLobbiesCommon::FindLobbies);
	RegisterCommand(&FLobbiesCommon::RestoreLobbies);
	RegisterCommand(&FLobbiesCommon::JoinLobby);
	RegisterCommand(&FLobbiesCommon::LeaveLobby);
	RegisterCommand(&FLobbiesCommon::InviteLobbyMember);
	RegisterCommand(&FLobbiesCommon::DeclineLobbyInvitation);
	RegisterCommand(&FLobbiesCommon::KickLobbyMember);
	RegisterCommand(&FLobbiesCommon::PromoteLobbyMember);
	RegisterCommand(&FLobbiesCommon::ModifyLobbySchema);
	RegisterCommand(&FLobbiesCommon::ModifyLobbyJoinPolicy);
	RegisterCommand(&FLobbiesCommon::ModifyLobbyAttributes);
	RegisterCommand(&FLobbiesCommon::ModifyLobbyMemberAttributes);
	RegisterCommand(&FLobbiesCommon::GetPresenceLobby);
	RegisterCommand(&FLobbiesCommon::IsPresenceLobby);
	RegisterCommand(&FLobbiesCommon::GetJoinedLobbies);
	RegisterCommand(&FLobbiesCommon::GetReceivedInvitations);

#if LOBBIES_FUNCTIONAL_TEST_ENABLED
	RegisterCommand(&FLobbiesCommon::FunctionalTest);
#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED
}

TOnlineAsyncOpHandle<FCreateLobby> FLobbiesCommon::CreateLobby(FCreateLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FCreateLobby> Operation = GetOp<FCreateLobby>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FFindLobbies> FLobbiesCommon::FindLobbies(FFindLobbies::Params&& Params)
{
	TOnlineAsyncOpRef<FFindLobbies> Operation = GetOp<FFindLobbies>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FRestoreLobbies> FLobbiesCommon::RestoreLobbies(FRestoreLobbies::Params&& Params)
{
	TOnlineAsyncOpRef<FRestoreLobbies> Operation = GetOp<FRestoreLobbies>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FJoinLobby> FLobbiesCommon::JoinLobby(FJoinLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FJoinLobby> Operation = GetOp<FJoinLobby>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FLeaveLobby> FLobbiesCommon::LeaveLobby(FLeaveLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FLeaveLobby> Operation = GetOp<FLeaveLobby>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FInviteLobbyMember> FLobbiesCommon::InviteLobbyMember(FInviteLobbyMember::Params&& Params)
{
	TOnlineAsyncOpRef<FInviteLobbyMember> Operation = GetOp<FInviteLobbyMember>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FDeclineLobbyInvitation> FLobbiesCommon::DeclineLobbyInvitation(FDeclineLobbyInvitation::Params&& Params)
{
	TOnlineAsyncOpRef<FDeclineLobbyInvitation> Operation = GetOp<FDeclineLobbyInvitation>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FKickLobbyMember> FLobbiesCommon::KickLobbyMember(FKickLobbyMember::Params&& Params)
{
	TOnlineAsyncOpRef<FKickLobbyMember> Operation = GetOp<FKickLobbyMember>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FPromoteLobbyMember> FLobbiesCommon::PromoteLobbyMember(FPromoteLobbyMember::Params&& Params)
{
	TOnlineAsyncOpRef<FPromoteLobbyMember> Operation = GetOp<FPromoteLobbyMember>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbySchema> FLobbiesCommon::ModifyLobbySchema(FModifyLobbySchema::Params&& Params)
{
	TOnlineAsyncOpRef<FModifyLobbySchema> Operation = GetOp<FModifyLobbySchema>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyJoinPolicy> FLobbiesCommon::ModifyLobbyJoinPolicy(FModifyLobbyJoinPolicy::Params&& Params)
{
	TOnlineAsyncOpRef<FModifyLobbyJoinPolicy> Operation = GetOp<FModifyLobbyJoinPolicy>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyAttributes> FLobbiesCommon::ModifyLobbyAttributes(FModifyLobbyAttributes::Params&& Params)
{
	TOnlineAsyncOpRef<FModifyLobbyAttributes> Operation = GetOp<FModifyLobbyAttributes>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FModifyLobbyMemberAttributes> FLobbiesCommon::ModifyLobbyMemberAttributes(FModifyLobbyMemberAttributes::Params&& Params)
{
	TOnlineAsyncOpRef<FModifyLobbyMemberAttributes> Operation = GetOp<FModifyLobbyMemberAttributes>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetPresenceLobby> FLobbiesCommon::GetPresenceLobby(FGetPresenceLobby::Params&& Params)
{
	if (!Params.LocalAccountId.IsValid())
	{
		UE_LOGF(LogTemp, Verbose, "[%ls] Value of Params.LocalAccountId invalid.", UTF8_TO_TCHAR(__FUNCTION__));
		return TOnlineResult<FGetPresenceLobby>(Errors::InvalidParams());
	}

	if (const FLobbyId* PresenceLobbyId = PresenceLobbiesUserMap.Find(Params.LocalAccountId))
	{
		TOnlineResult<FGetJoinedLobbies> GetJoinedLobbiesResult = GetJoinedLobbies({ Params.LocalAccountId });
		if (GetJoinedLobbiesResult.IsOk())
		{
			const TArray<TSharedRef<const FLobby>>& JoinedLobbies = GetJoinedLobbiesResult.GetOkValue().Lobbies;
			for (const TSharedRef<const FLobby>& Lobby : JoinedLobbies)
			{
				if (Lobby->LobbyId == *PresenceLobbyId)
				{
					return TOnlineResult<FGetPresenceLobby>({ Lobby });
				}
			}
			
			UE_LOGF(LogTemp, Verbose, "[%ls] Presence Lobby was not found among Joined Lobbies", UTF8_TO_TCHAR(__FUNCTION__));
			return TOnlineResult<FGetPresenceLobby>({ Errors::NotFound() });
		}
		else
		{
			UE_LOGF(LogTemp, Verbose, "[%ls] GetJoinedLobbies was not successful", UTF8_TO_TCHAR(__FUNCTION__));
			return TOnlineResult<FGetPresenceLobby>(GetJoinedLobbiesResult.GetErrorValue());
		}
	}
	else
	{
		UE_LOGF(LogTemp, Verbose, "[%ls] No data found for user [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
		return TOnlineResult<FGetPresenceLobby>({ Errors::InvalidState() });
	}
}

TOnlineResult<FIsPresenceLobby> FLobbiesCommon::IsPresenceLobby(FIsPresenceLobby::Params&& Params)
{
	if (!Params.LocalAccountId.IsValid())
	{
		UE_LOGF(LogTemp, Verbose, "[%ls] Value of Params.LocalAccountId invalid.", UTF8_TO_TCHAR(__FUNCTION__));
		return TOnlineResult<FIsPresenceLobby>(Errors::InvalidParams());
	}
	
	if (!Params.LobbyId.IsValid())
	{
		UE_LOGF(LogTemp, Verbose, "[%ls] Value of Params.LobbyId invalid.", UTF8_TO_TCHAR(__FUNCTION__));
		return TOnlineResult<FIsPresenceLobby>(Errors::InvalidParams());
	}

	if (const FLobbyId* PresenceLobbyId = PresenceLobbiesUserMap.Find(Params.LocalAccountId))
	{
		return TOnlineResult<FIsPresenceLobby>(FIsPresenceLobby::Result{ Params.LobbyId == (*PresenceLobbyId) });
	}
	else
	{
		UE_LOGF(LogTemp, Verbose, "[%ls] No data found for user [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
		return TOnlineResult<FIsPresenceLobby>({ Errors::InvalidState() });
	}
}

TOnlineResult<FGetJoinedLobbies> FLobbiesCommon::GetJoinedLobbies(FGetJoinedLobbies::Params&& Params)
{
	return TOnlineResult<FGetJoinedLobbies>(Errors::NotImplemented());
}

TOnlineResult<FGetReceivedInvitations> FLobbiesCommon::GetReceivedInvitations(FGetReceivedInvitations::Params&& Params)
{
	return TOnlineResult<FGetReceivedInvitations>(Errors::NotImplemented());
}

TOnlineEvent<void(const FLobbyJoined&)> FLobbiesCommon::OnLobbyJoined()
{
	return LobbyEvents.OnLobbyJoined;
}

TOnlineEvent<void(const FLobbyLeft&)> FLobbiesCommon::OnLobbyLeft()
{
	return LobbyEvents.OnLobbyLeft;
}

TOnlineEvent<void(const FLobbyMemberJoined&)> FLobbiesCommon::OnLobbyMemberJoined()
{
	return LobbyEvents.OnLobbyMemberJoined;
}

TOnlineEvent<void(const FLobbyMemberLeft&)> FLobbiesCommon::OnLobbyMemberLeft()
{
	return LobbyEvents.OnLobbyMemberLeft;
}

TOnlineEvent<void(const FLobbyLeaderChanged&)> FLobbiesCommon::OnLobbyLeaderChanged()
{
	return LobbyEvents.OnLobbyLeaderChanged;
}

TOnlineEvent<void(const FLobbySchemaChanged&)> FLobbiesCommon::OnLobbySchemaChanged()
{
	return LobbyEvents.OnLobbySchemaChanged;
}

TOnlineEvent<void(const FLobbyAttributesChanged&)> FLobbiesCommon::OnLobbyAttributesChanged()
{
	return LobbyEvents.OnLobbyAttributesChanged;
}

TOnlineEvent<void(const FLobbyMemberAttributesChanged&)> FLobbiesCommon::OnLobbyMemberAttributesChanged()
{
	return LobbyEvents.OnLobbyMemberAttributesChanged;
}

TOnlineEvent<void(const FLobbyInvitationAdded&)> FLobbiesCommon::OnLobbyInvitationAdded()
{
	return LobbyEvents.OnLobbyInvitationAdded;
}

TOnlineEvent<void(const FLobbyInvitationRemoved&)> FLobbiesCommon::OnLobbyInvitationRemoved()
{
	return LobbyEvents.OnLobbyInvitationRemoved;
}

TOnlineEvent<void(const FUILobbyJoinRequested&)> FLobbiesCommon::OnUILobbyJoinRequested()
{
	return LobbyEvents.OnUILobbyJoinRequested;
}

#if LOBBIES_FUNCTIONAL_TEST_ENABLED
TOnlineAsyncOpHandle<FFunctionalTestLobbies> FLobbiesCommon::FunctionalTest(FFunctionalTestLobbies::Params&& Params)
{
	IAuth* AuthInterface = Services.Get<IAuth>();
	if (!AuthInterface)
	{
		TOnlineAsyncOpRef<FFunctionalTestLobbies> Operation = GetOp<FFunctionalTestLobbies>(MoveTemp(Params));
		Operation->SetError(Errors::MissingInterface());
		return Operation->GetHandle();
	}

	return RunLobbyFunctionalTest(*AuthInterface, *this, LobbyEvents);
}
#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

/* UE::Online */ }
