// Copyright Epic Games, Inc. All Rights Reserved.

#include "LobbiesCommonTests.h"

#include "Misc/AutomationTest.h"
#include "Online/LobbiesCommon.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"

namespace UE::Online {

class FLobbyEventCapture final
{
public:
	UE_NONCOPYABLE(FLobbyEventCapture);

	FLobbyEventCapture(FLobbyEvents& LobbyEvents)
		: LobbyJoinedHandle(LobbyEvents.OnLobbyJoined.Add(this, &FLobbyEventCapture::OnLobbyJoined))
		, LobbyLeftHandle(LobbyEvents.OnLobbyLeft.Add(this, &FLobbyEventCapture::OnLobbyLeft))
		, LobbyMemberJoinedHandle(LobbyEvents.OnLobbyMemberJoined.Add(this, &FLobbyEventCapture::OnLobbyMemberJoined))
		, LobbyMemberLeftHandle(LobbyEvents.OnLobbyMemberLeft.Add(this, &FLobbyEventCapture::OnLobbyMemberLeft))
		, LobbyLeaderChangedHandle(LobbyEvents.OnLobbyLeaderChanged.Add(this, &FLobbyEventCapture::OnLobbyLeaderChanged))
		, LobbySchemaChangedHandle(LobbyEvents.OnLobbySchemaChanged.Add(this, &FLobbyEventCapture::OnLobbySchemaChanged))
		, LobbyAttributesChangedHandle(LobbyEvents.OnLobbyAttributesChanged.Add(this, &FLobbyEventCapture::OnLobbyAttributesChanged))
		, LobbyMemberAttributesChangedHandle(LobbyEvents.OnLobbyMemberAttributesChanged.Add(this, &FLobbyEventCapture::OnLobbyMemberAttributesChanged))
		, LobbyInvitationAddedHandle(LobbyEvents.OnLobbyInvitationAdded.Add(this, &FLobbyEventCapture::OnLobbyInvitationAdded))
		, LobbyInvitationRemovedHandle(LobbyEvents.OnLobbyInvitationRemoved.Add(this, &FLobbyEventCapture::OnLobbyInvitationRemoved))
	{
	}

	void Empty()
	{
		LobbyJoined.Empty();
		LobbyLeft.Empty();
		LobbyMemberJoined.Empty();
		LobbyMemberLeft.Empty();
		LobbyLeaderChanged.Empty();
		LobbySchemaChanged.Empty();
		LobbyAttributesChanged.Empty();
		LobbyMemberAttributesChanged.Empty();
		LobbyInvitationAdded.Empty();
		LobbyInvitationRemoved.Empty();
		NextIndex = 0;
		TotalNotificationsReceived = 0;
	}

	uint32 GetTotalNotificationsReceived() const
	{
		return TotalNotificationsReceived;
	}

	template <typename NotificationType>
	struct TNotificationInfo
	{
		NotificationType Notification;
		int32 GlobalIndex = 0;
	};

	TArray<TNotificationInfo<FLobbyJoined>> LobbyJoined;
	TArray<TNotificationInfo<FLobbyLeft>> LobbyLeft;
	TArray<TNotificationInfo<FLobbyMemberJoined>> LobbyMemberJoined;
	TArray<TNotificationInfo<FLobbyMemberLeft>> LobbyMemberLeft;
	TArray<TNotificationInfo<FLobbyLeaderChanged>> LobbyLeaderChanged;
	TArray<TNotificationInfo<FLobbySchemaChanged>> LobbySchemaChanged;
	TArray<TNotificationInfo<FLobbyAttributesChanged>> LobbyAttributesChanged;
	TArray<TNotificationInfo<FLobbyMemberAttributesChanged>> LobbyMemberAttributesChanged;
	TArray<TNotificationInfo<FLobbyInvitationAdded>> LobbyInvitationAdded;
	TArray<TNotificationInfo<FLobbyInvitationRemoved>> LobbyInvitationRemoved;

private:
	template <typename ContainerType, typename NotificationType>
	void AddEvent(ContainerType& Container, const NotificationType& Notification)
	{
		Container.Add({Notification, NextIndex++});
		++TotalNotificationsReceived;
	}

	void OnLobbyJoined(const FLobbyJoined& Notification) { AddEvent(LobbyJoined, Notification); }
	void OnLobbyLeft(const FLobbyLeft& Notification) { AddEvent(LobbyLeft, Notification); }
	void OnLobbyMemberJoined(const FLobbyMemberJoined& Notification) { AddEvent(LobbyMemberJoined, Notification); }
	void OnLobbyMemberLeft(const FLobbyMemberLeft& Notification) { AddEvent(LobbyMemberLeft, Notification); }
	void OnLobbyLeaderChanged(const FLobbyLeaderChanged& Notification) { AddEvent(LobbyLeaderChanged, Notification); }
	void OnLobbySchemaChanged(const FLobbySchemaChanged& Notification) { AddEvent(LobbySchemaChanged, Notification); }
	void OnLobbyAttributesChanged(const FLobbyAttributesChanged& Notification) { AddEvent(LobbyAttributesChanged, Notification); }
	void OnLobbyMemberAttributesChanged(const FLobbyMemberAttributesChanged& Notification) { AddEvent(LobbyMemberAttributesChanged, Notification); }
	void OnLobbyInvitationAdded(const FLobbyInvitationAdded& Notification) { AddEvent(LobbyInvitationAdded, Notification); }
	void OnLobbyInvitationRemoved(const FLobbyInvitationRemoved& Notification) { AddEvent(LobbyInvitationRemoved, Notification); }

	FOnlineEventDelegateHandle LobbyJoinedHandle;
	FOnlineEventDelegateHandle LobbyLeftHandle;
	FOnlineEventDelegateHandle LobbyMemberJoinedHandle;
	FOnlineEventDelegateHandle LobbyMemberLeftHandle;
	FOnlineEventDelegateHandle LobbyLeaderChangedHandle;
	FOnlineEventDelegateHandle LobbySchemaChangedHandle;
	FOnlineEventDelegateHandle LobbyAttributesChangedHandle;
	FOnlineEventDelegateHandle LobbyMemberAttributesChangedHandle;
	FOnlineEventDelegateHandle LobbyInvitationAddedHandle;
	FOnlineEventDelegateHandle LobbyInvitationRemovedHandle;
	int32 NextIndex = 0;
	uint32 TotalNotificationsReceived = 0;
};

//--------------------------------------------------------------------------------------------------
// Unit testing
//--------------------------------------------------------------------------------------------------

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FClientLobbyDataTest, 
	"System.Engine.Online.ClientLobbyDataTest",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FClientLobbyDataTest::RunTest(const FString& Parameters)
{
	FSchemaId LobbySchemaId1 = TEXT("Lobby1");
	FSchemaId LobbySchemaId2 = TEXT("Lobby2");
	FSchemaCategoryId LobbyCategoryId = TEXT("Lobby");
	FSchemaCategoryId LobbyMemberCategoryId = TEXT("LobbyMember");
	FSchemaServiceDescriptorId LobbyServiceDescriptorId = TEXT("LobbyServiceDescriptor");
	FSchemaServiceAttributeId LobbyServiceAttributeId1 = TEXT("LobbyServiceAttributeId1");
	FSchemaServiceAttributeId LobbyServiceAttributeId2 = TEXT("LobbyServiceAttributeId2");
	FSchemaServiceAttributeId LobbyServiceAttributeId3 = TEXT("LobbyServiceAttributeId3");
	FSchemaServiceDescriptorId LobbyMemberServiceDescriptorId = TEXT("LobbyMemberServiceDescriptor");
	FSchemaServiceAttributeId LobbyMemberServiceAttributeId1 = TEXT("LobbyMemberServiceAttributeId1");
	FSchemaServiceAttributeId LobbyMemberServiceAttributeId2 = TEXT("LobbyMemberServiceAttributeId2");
	const TArray<ESchemaServiceAttributeSupportedTypeFlags> AllSupportedServiceAttributeTypes = {
		ESchemaServiceAttributeSupportedTypeFlags::Bool,
		ESchemaServiceAttributeSupportedTypeFlags::Int64,
		ESchemaServiceAttributeSupportedTypeFlags::Double,
		ESchemaServiceAttributeSupportedTypeFlags::String };
	const TArray<ESchemaServiceAttributeFlags> AllSupportedServiceAttributeFlags = {
		ESchemaServiceAttributeFlags::Searchable,
		ESchemaServiceAttributeFlags::Public,
		ESchemaServiceAttributeFlags::Private };

	FSchemaAttributeId SchemaCompatibilityAttributeId = TEXT("SchemaCompatibilityAttribute");
	FSchemaAttributeId LobbyAttributeId1 = TEXT("LobbyAttribute1");
	FSchemaAttributeId LobbyAttributeId2 = TEXT("LobbyAttribute2");
	FSchemaAttributeId LobbyMemberAttributeId1 = TEXT("LobbyMemberAttribute1");
	FSchemaAttributeId LobbyMemberAttributeId2 = TEXT("LobbyMemberAttribute2");

	FSchemaRegistryDescriptorConfig SchemaConfig;
	SchemaConfig.SchemaDescriptors.Add({ LobbyBaseSchemaId, FSchemaId(), { LobbyCategoryId, LobbyMemberCategoryId } });
	SchemaConfig.SchemaCategoryDescriptors.Add({ LobbyCategoryId, LobbyServiceDescriptorId });
	SchemaConfig.SchemaCategoryDescriptors.Add({ LobbyMemberCategoryId, LobbyMemberServiceDescriptorId });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbyBaseSchemaId, LobbyCategoryId, { SchemaCompatibilityAttributeId } });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbyBaseSchemaId, LobbyMemberCategoryId, {} });
	SchemaConfig.SchemaAttributeDescriptors.Add({ SchemaCompatibilityAttributeId, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

	SchemaConfig.SchemaDescriptors.Add({ LobbySchemaId1, LobbyBaseSchemaId });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbySchemaId1, LobbyCategoryId, { LobbyAttributeId1, LobbyAttributeId2 } });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbySchemaId1, LobbyMemberCategoryId, { LobbyMemberAttributeId1, LobbyMemberAttributeId2 } });

	SchemaConfig.SchemaDescriptors.Add({ LobbySchemaId2, LobbyBaseSchemaId });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbySchemaId2, LobbyCategoryId, { LobbyAttributeId1, LobbyAttributeId2 } });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ LobbySchemaId2, LobbyMemberCategoryId, { LobbyMemberAttributeId1, LobbyMemberAttributeId2 } });

	SchemaConfig.SchemaAttributeDescriptors.Add({ LobbyAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 64 });
	SchemaConfig.SchemaAttributeDescriptors.Add({ LobbyAttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 64 });
	SchemaConfig.SchemaAttributeDescriptors.Add({ LobbyMemberAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 64 });
	SchemaConfig.SchemaAttributeDescriptors.Add({ LobbyMemberAttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 64 });

	SchemaConfig.ServiceDescriptors.Add({ LobbyServiceDescriptorId, { LobbyServiceAttributeId1, LobbyServiceAttributeId2, LobbyServiceAttributeId3 } });
	SchemaConfig.ServiceAttributeDescriptors.Add({ LobbyServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 64 });
	SchemaConfig.ServiceAttributeDescriptors.Add({ LobbyServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 64 });
	SchemaConfig.ServiceAttributeDescriptors.Add({ LobbyServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 64 });
	SchemaConfig.ServiceDescriptors.Add({ LobbyMemberServiceDescriptorId, { LobbyMemberServiceAttributeId1, LobbyMemberServiceAttributeId2 } });
	SchemaConfig.ServiceAttributeDescriptors.Add({ LobbyMemberServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 64 });
	SchemaConfig.ServiceAttributeDescriptors.Add({ LobbyMemberServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 64 });

	TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();
	UTEST_TRUE("Schema config parsed successfully.", SchemaRegistry->ParseConfig(SchemaConfig));

	TSharedPtr<const FSchemaDefinition> LobbySchemaDefinition = SchemaRegistry->GetDefinition(LobbySchemaId1);
	UTEST_NOT_NULL("Lobby schema definition exists.", LobbySchemaDefinition.Get());

	FSchemaVariant SchemaCompatibilityAttributeValue(LobbySchemaDefinition->CompatibilityId);
	FSchemaVariant LobbyAttribute1Value1(TEXT("Attribute1-Value1"));
	FSchemaVariant LobbyAttribute1Value2(TEXT("Attribute1-Value2"));
	FSchemaVariant LobbyAttribute2Value1(TEXT("Attribute2-Value1"));
	FSchemaVariant LobbyAttribute2Value2(TEXT("Attribute2-Value2"));
	FSchemaVariant LobbyMemberAttribute1Value1(TEXT("MemberAttribute1-Value1"));
	FSchemaVariant LobbyMemberAttribute1Value2(TEXT("MemberAttribute1-Value2"));
	FSchemaVariant LobbyMemberAttribute2Value1(TEXT("MemberAttribute2-Value1"));
	FSchemaVariant LobbyMemberAttribute2Value2(TEXT("MemberAttribute2-Value2"));

	// 1. Applying snapshot with no local members does not fire events.
	{
		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		// 1.1 setup. Create lobby data from snapshot with initial attributes.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons)});
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.1.1. Initial snapshot does not trigger any events.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby owner.", ClientData.GetPublicData().OwnerAccountId, User1);
			UTEST_EQUAL("Verify lobby schema.", ClientData.GetPublicData().SchemaId, LobbySchemaId1);
			UTEST_EQUAL("Verify lobby max members.", ClientData.GetPublicData().MaxMembers, LobbyMaxMembers);
			UTEST_EQUAL("Verify lobby join policy.", ClientData.GetPublicData().JoinPolicy, LobbyJoinPolicy);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 1);
			UTEST_NOT_NULL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Find(LobbyAttributeId1));
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes[LobbyAttributeId1], LobbyAttribute1Value1);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 2);
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User1));
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members[User1]->AccountId, User1);
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User2));
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members[User2]->AccountId, User2);
			UTEST_EQUAL("Verify lobby member attributes.", ClientData.GetPublicData().Members[User1]->Attributes.Num(), 1);
			UTEST_NOT_NULL("Verify lobby member attributes.", ClientData.GetPublicData().Members[User1]->Attributes.Find(LobbyMemberAttributeId1));
			UTEST_EQUAL("Verify lobby member attributes.", ClientData.GetPublicData().Members[User1]->Attributes[LobbyMemberAttributeId1], LobbyMemberAttribute1Value1);
		}

		// 1.2 setup. Change owner. No member snapshots due to no detected member changes.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.2.1. Snapshot with changed lobby leader does not trigger lobby leader changed.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby owner.", ClientData.GetPublicData().OwnerAccountId, User2);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 1);
			const FSchemaVariant* LobbyAttribute1 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId1);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute1);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute1, LobbyAttribute1Value1);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 2);
			const TSharedRef<const FLobbyMember>* User1Data = ClientData.GetPublicData().Members.Find(User1);
			UTEST_NOT_NULL("Verify lobby members.", User1Data);
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User2));
			UTEST_EQUAL("Verify lobby member attributes.", (*User1Data)->Attributes.Num(), 1);
			const FSchemaVariant* LobbyMemberAttribute1 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId1);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute1);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
		}

		// 1.3 setup. Add lobby and lobby member attribute.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId2, LobbyMemberAttribute2Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId3, LobbyAttribute2Value1);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.3.1. Snapshot with changed lobby attribute does not trigger lobby attribute changed.
			// 1.3.2. Snapshot with changed lobby member attribute does not trigger lobby member attribute changed.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby owner.", ClientData.GetPublicData().OwnerAccountId, User2);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 2);
			const FSchemaVariant* LobbyAttribute1 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId1);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute1);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute1, LobbyAttribute1Value1);
			const FSchemaVariant* LobbyAttribute2 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute2);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute2, LobbyAttribute2Value1);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 2);
			const TSharedRef<const FLobbyMember>* User1Data = ClientData.GetPublicData().Members.Find(User1);
			UTEST_NOT_NULL("Verify lobby members.", User1Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User1Data)->Attributes.Num(), 2);
			const FSchemaVariant* LobbyMemberAttribute1 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId1);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute1);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
			const FSchemaVariant* LobbyMemberAttribute2 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute2);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute2, LobbyMemberAttribute2Value1);
			const TSharedRef<const FLobbyMember>* User2Data = ClientData.GetPublicData().Members.Find(User2);
			UTEST_NOT_NULL("Verify lobby members.", User2Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User2Data)->Attributes.Num(), 0);
		}

		// 1.3 setup. Modify lobby and lobby member attribute.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId2, LobbyMemberAttribute2Value2);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId3, LobbyAttribute2Value2);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.3.1. Snapshot with changed lobby attribute does not trigger lobby attribute changed.
			// 1.3.2. Snapshot with changed lobby member attribute does not trigger lobby member attribute changed.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby owner.", ClientData.GetPublicData().OwnerAccountId, User2);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 2);
			const FSchemaVariant* LobbyAttribute1 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId1);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute1);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute1, LobbyAttribute1Value1);
			const FSchemaVariant* LobbyAttribute2 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute2);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute2, LobbyAttribute2Value2);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 2);
			const TSharedRef<const FLobbyMember>* User1Data = ClientData.GetPublicData().Members.Find(User1);
			UTEST_NOT_NULL("Verify lobby members.", User1Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User1Data)->Attributes.Num(), 2);
			const FSchemaVariant* LobbyMemberAttribute1 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId1);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute1);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
			const FSchemaVariant* LobbyMemberAttribute2 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute2);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute2, LobbyMemberAttribute2Value2);
			const TSharedRef<const FLobbyMember>* User2Data = ClientData.GetPublicData().Members.Find(User2);
			UTEST_NOT_NULL("Verify lobby members.", User2Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User2Data)->Attributes.Num(), 0);
		}

		// 1.3 setup. Remove lobby and lobby member attribute.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.3.1. Snapshot with changed lobby attribute does not trigger lobby attribute changed.
			// 1.3.2. Snapshot with changed lobby member attribute does not trigger lobby member attribute changed.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby owner.", ClientData.GetPublicData().OwnerAccountId, User2);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 1);
			const FSchemaVariant* LobbyAttribute1 = ClientData.GetPublicData().Attributes.Find(LobbyAttributeId1);
			UTEST_NOT_NULL("Verify lobby attributes.", LobbyAttribute1);
			UTEST_EQUAL("Verify lobby attributes.", *LobbyAttribute1, LobbyAttribute1Value1);
			const TSharedRef<const FLobbyMember>* User1Data = ClientData.GetPublicData().Members.Find(User1);
			UTEST_NOT_NULL("Verify lobby members.", User1Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User1Data)->Attributes.Num(), 1);
			const FSchemaVariant* LobbyMemberAttribute1 = (*User1Data)->Attributes.Find(LobbyMemberAttributeId1);
			UTEST_NOT_NULL("Verify lobby member attributes.", LobbyMemberAttribute1);
			UTEST_EQUAL("Verify lobby member attributes.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
			const TSharedRef<const FLobbyMember>* User2Data = ClientData.GetPublicData().Members.Find(User2);
			UTEST_NOT_NULL("Verify lobby members.", User2Data);
			UTEST_EQUAL("Verify lobby member attributes.", (*User2Data)->Attributes.Num(), 0);
		}

		// 1.3 setup. User leaving the lobby.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.Members = {User1};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 1.3.3. Snapshot with members leaving does not trigger lobby member left or lobby left.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby attributes.", ClientData.GetPublicData().Attributes.Num(), 0);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 1);
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User1));
			UTEST_EQUAL("Verify lobby member attributes.", ClientData.GetPublicData().Members[User1]->Attributes.Num(), 0);
		}
	}

	// 2. Applying snapshot with local members present does fire events.
	{
		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		// 2.1 setup - initial lobby setup.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.1.1. Initial snapshot does not trigger lobby joined or lobby member joined. - Member in snapshot is local member to simulate lobby creation.
			UTEST_EQUAL("Snapshot does not trigger any events.", EventCapture.GetTotalNotificationsReceived(), 0);
			UTEST_EQUAL("Verify lobby members.", ClientData.GetPublicData().Members.Num(), 2);
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User1));
			UTEST_NOT_NULL("Verify lobby members.", ClientData.GetPublicData().Members.Find(User2));
		}

		// 2.2 setup - add local lobby member User1.
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = FLobbyClientMemberAttributeChanges();
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.2.1. Local member added to lobby through local change. Adding a local member will cause events to begin triggering.
			UTEST_EQUAL("Check lobby and member joined events were received.", EventCapture.GetTotalNotificationsReceived(), 3);
			UTEST_EQUAL("Check lobby joined event received.", EventCapture.LobbyJoined.Num(), 1);
			UTEST_EQUAL("Check lobby joined event received first.", EventCapture.LobbyJoined[0].GlobalIndex, 0);
			UTEST_EQUAL("Check lobby joined event is valid.", EventCapture.LobbyJoined[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check member joined events received.", EventCapture.LobbyMemberJoined.Num(), 2);
			UTEST_EQUAL("Check member joined event is valid.", EventCapture.LobbyMemberJoined[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_TRUE("Check member joined event is valid.", EventCapture.LobbyMemberJoined[0].Notification.Member->AccountId.IsValid());
		}

		// 2.3 setup - Change lobby leader.
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.3.1. Snapshot with changed lobby leader triggers lobby leader changed.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 1);
			UTEST_EQUAL("Check LobbySchemaChanged is valid.", EventCapture.LobbySchemaChanged.Num(), 0);
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged[0].Notification.Leader->AccountId, User2);
		}

		// 2.4 setup - Add lobby and member attribute
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId2, LobbyMemberAttribute2Value1);
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId3, LobbyAttribute2Value1);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.4.1. Snapshot with changed lobby attribute triggers lobby attribute changed for added attribute.
			// 2.4.2. Snapshot with changed lobby member attribute triggers lobby member attribute changed for added attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const FSchemaVariant* LobbyAttribute2 = EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Check LobbyAttributesChanged is valid.", LobbyAttribute2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", *LobbyAttribute2, LobbyAttribute2Value1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const FSchemaVariant* LobbyMemberAttribute2 = EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Check LobbyMemberAttributesChanged is valid.", LobbyMemberAttribute2);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", *LobbyMemberAttribute2, LobbyMemberAttribute2Value1);
		}

		// 2.5 setup - Modify lobby and member attribute
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId2, LobbyMemberAttribute2Value2);
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId3, LobbyAttribute2Value2);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.5.1. Snapshot with changed lobby attribute triggers lobby attribute changed for changed attribute.
			// 2.5.2. Snapshot with changed lobby member attribute triggers lobby member attribute changed for changed attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const TPair<FSchemaVariant, FSchemaVariant>* LobbyAttribute2 = EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Check LobbyAttributesChanged is valid.", LobbyAttribute2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", *LobbyAttribute2, (TPair<FSchemaVariant, FSchemaVariant>(LobbyAttribute2Value1, LobbyAttribute2Value2)));
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const TPair<FSchemaVariant, FSchemaVariant>* LobbyMemberAttribute2 = EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Check LobbyMemberAttributesChanged is valid.", LobbyMemberAttribute2);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", *LobbyMemberAttribute2, (TPair<FSchemaVariant, FSchemaVariant>(LobbyMemberAttribute2Value1, LobbyMemberAttribute2Value2)));
		}

		// 2.6 setup - Clear lobby and member attribute
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = { User1, User2 };

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 2.6.1. Snapshot with changed lobby attribute triggers lobby attribute changed for changed attribute.
			// 2.6.2. Snapshot with changed lobby member attribute triggers lobby member attribute changed for changed attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 1);
			UTEST_TRUE("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Contains(LobbyAttributeId2));
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 1);
			UTEST_TRUE("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Contains(LobbyMemberAttributeId2));
		}

		// 2.7 setup - kick local user from lobby.
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User2;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			LeaveReasons.Add(User1, ELobbyMemberLeaveReason::Kicked);

			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check local member left.", CommitResult.LeavingLocalMembers.Num(), 1);
			UTEST_EQUAL("Check local member left.", CommitResult.LeavingLocalMembers[0], User1);
		}

		{
			// 2.7.1. Snapshot with members leaving triggers lobby member left and lobby left.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 3);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft.Num(), 2);
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* User1LeaveNotification = EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId == User1 ? &EventCapture.LobbyMemberLeft[0] : &EventCapture.LobbyMemberLeft[1];
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* User2LeaveNotification = EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId == User2 ? &EventCapture.LobbyMemberLeft[0] : &EventCapture.LobbyMemberLeft[1];
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Reason, ELobbyMemberLeaveReason::Kicked);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Member->AccountId, User2);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Reason, ELobbyMemberLeaveReason::Left);
			UTEST_EQUAL("Check LobbyLeft valid.", EventCapture.LobbyLeft.Num(), 1);
			UTEST_EQUAL("Check LobbyLeft valid.", EventCapture.LobbyLeft[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyLeft ordering.", EventCapture.LobbyLeft[0].GlobalIndex, EventCapture.GetTotalNotificationsReceived() - 1);
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* LastMemberLeaveNotification = User1LeaveNotification->GlobalIndex > User2LeaveNotification->GlobalIndex ? User1LeaveNotification : User2LeaveNotification;
			UTEST_EQUAL("Check LobbyMemberLeft ordering.", LastMemberLeaveNotification->GlobalIndex, EventCapture.LobbyLeft[0].GlobalIndex - 1);
		}
	}

	// 3. Snapshot supplemental
	{
		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		// 3.1 setup - initialize lobby data with initial snapshot.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 3.1 setup continued. Add local member to enable notifications.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = FLobbyClientMemberAttributeChanges();
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 3.1 setup continued. Change snapshot for the test. User 2 leaves with no explicit reason.
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.Members = {User1};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 3.1. Test that Member left is fired even when a leave reason is not specified.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 1);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId, User2);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Reason, ELobbyMemberLeaveReason::Disconnected);
		}

		// 3.2 setup
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.Members = {User1};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 3.2. Applying member snapshots for an account id not in the lobby members list does not fire lobby member joined.
			UTEST_EQUAL("Check no events received.", EventCapture.GetTotalNotificationsReceived(), 0);
		}

		// 3.3 setup
		{
			EventCapture.Empty();

			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 3.3.1 Check lobby member joined notification received for remote member after local member joined.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", EventCapture.LobbyMemberJoined.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", EventCapture.LobbyMemberJoined[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberJoined valid.", EventCapture.LobbyMemberJoined[0].Notification.Member->AccountId, User2);
		}
	}

	// 4. Applying lobby changes produces expected notifications.
	{
		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		// 4.1 setup. Local member in snapshot with remote member to simulate lobby join.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		UTEST_EQUAL("Check no events received.", EventCapture.GetTotalNotificationsReceived(), 0);

		// 4.1 Attempt to update lobby data when user is not the owner fails.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.Attributes = { { { LobbyAttributeId1, LobbyAttribute1Value2 } }, {} };
			LobbyChanges.OwnerAccountId = User2;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User2, MoveTemp(LobbyChanges) });
			UTEST_FALSE("Check PrepareClientChanges failed as expected.", PrepareResult.IsOk());
		}

		// 4.2 setup. Join local User1 to the lobby to enable notifications.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = FLobbyClientMemberAttributeChanges();
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 4.2.1. Local members added to lobby through local change triggers lobby joined and lobby member joined. On member joined triggered for remote member.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 3);
			UTEST_EQUAL("Check LobbyJoined valid.", EventCapture.LobbyJoined.Num(), 1);
			UTEST_EQUAL("Check LobbyJoined valid.", EventCapture.LobbyJoined[0].GlobalIndex, 0);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", EventCapture.LobbyJoined[0].Notification.Lobby->Attributes.Num(), 1);
			const FSchemaVariant* LobbyAttribute1 = EventCapture.LobbyJoined[0].Notification.Lobby->Attributes.Find(LobbyAttributeId1);
			UTEST_NOT_NULL("Check LobbyMemberJoined valid.", LobbyAttribute1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", *LobbyAttribute1, LobbyAttribute1Value1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", EventCapture.LobbyMemberJoined.Num(), 2);
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberJoined>* User1JoinNotification = EventCapture.LobbyMemberJoined[0].Notification.Member->AccountId == User1 ? &EventCapture.LobbyMemberJoined[0] : &EventCapture.LobbyMemberJoined[1];
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberJoined>* User2JoinNotification = EventCapture.LobbyMemberJoined[0].Notification.Member->AccountId == User2 ? &EventCapture.LobbyMemberJoined[0] : &EventCapture.LobbyMemberJoined[1];
			UTEST_EQUAL("Check LobbyMemberJoined valid.", User1JoinNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberJoined valid.", User1JoinNotification->Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", User1JoinNotification->Notification.Member->Attributes.Num(), 1);
			const FSchemaVariant* LobbyMemberAttribute1 = User1JoinNotification->Notification.Member->Attributes.Find(LobbyMemberAttributeId1);
			UTEST_NOT_NULL("Check LobbyMemberJoined valid.", LobbyMemberAttribute1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
			UTEST_EQUAL("Check LobbyMemberJoined valid.", User2JoinNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberJoined valid.", User2JoinNotification->Notification.Member->AccountId, User2);
		}

		// 4.3 setup - add a lobby and lobby member attribute
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.Attributes = { {{ LobbyAttributeId2, LobbyAttribute2Value1 }}, {} };
			LobbyChanges.MemberAttributes = { {{ LobbyMemberAttributeId2, LobbyMemberAttribute2Value1 }}, {} };
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 4.3.1. Local changes with changed lobby attribute triggers lobby attribute changed for adding attribute.
			// 4.3.2. Local changes with changed lobby member attribute triggers lobby member attribute changed for adding attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const FSchemaVariant* LobbyAttribute2 = EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Check LobbyAttributesChanged is valid.", LobbyAttribute2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", *LobbyAttribute2, LobbyAttribute2Value1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const FSchemaVariant* LobbyMemberAttribute2 = EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Check LobbyMemberAttributesChanged is valid.", LobbyMemberAttribute2);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", *LobbyMemberAttribute2, LobbyMemberAttribute2Value1);
		}

		// 4.4 setup - modify a lobby and lobby member attribute
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.Attributes = { {{ LobbyAttributeId2, LobbyAttribute2Value2 }}, {} };
			LobbyChanges.MemberAttributes = { {{ LobbyMemberAttributeId2, LobbyMemberAttribute2Value2 }}, {} };
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 4.4.1. Local changes with changed lobby attribute triggers lobby attribute changed for changing attribute.
			// 4.4.2. Local changes with changed lobby member attribute triggers lobby member attribute changed for changing attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const TPair<FSchemaVariant, FSchemaVariant>* LobbyAttribute2 = EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Find(LobbyAttributeId2);
			UTEST_NOT_NULL("Check LobbyAttributesChanged is valid.", LobbyAttribute2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", *LobbyAttribute2, (TPair<FSchemaVariant, FSchemaVariant>(LobbyAttribute2Value1, LobbyAttribute2Value2)));
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 0);
			const TPair<FSchemaVariant, FSchemaVariant>* LobbyMemberAttribute2 = EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Find(LobbyMemberAttributeId2);
			UTEST_NOT_NULL("Check LobbyMemberAttributesChanged is valid.", LobbyMemberAttribute2);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", *LobbyMemberAttribute2, (TPair<FSchemaVariant, FSchemaVariant>(LobbyMemberAttribute2Value1, LobbyMemberAttribute2Value2)));
		}

		// 4.5 setup - remove a lobby and lobby member attribute
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.Attributes = { {}, { LobbyAttributeId2 } };
			LobbyChanges.MemberAttributes = { {}, { LobbyMemberAttributeId2 } };
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 4.5.1. Local changes with changed lobby attribute triggers lobby attribute changed for removing attribute.
			// 4.5.2. Local changes with changed lobby member attribute triggers lobby member attribute changed for removing attribute.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 2);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Num(), 1);
			UTEST_TRUE("Check LobbyAttributesChanged is valid.", EventCapture.LobbyAttributesChanged[0].Notification.RemovedAttributes.Contains(LobbyAttributeId2));
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.AddedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.ChangedAttributes.Num(), 0);
			UTEST_EQUAL("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Num(), 1);
			UTEST_TRUE("Check LobbyMemberAttributesChanged is valid.", EventCapture.LobbyMemberAttributesChanged[0].Notification.RemovedAttributes.Contains(LobbyMemberAttributeId2));
		}

		// 4.6 setup - assign leadership to another user.
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.OwnerAccountId = User2;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		{
			// 4.6.1. Local changes with changed lobby leader triggers lobby leader changed.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 1);
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged.Num(), 1);
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyLeaderChanged is valid.", EventCapture.LobbyLeaderChanged[0].Notification.Leader->AccountId, User2);
		}

		// 4.7 Trying to both set member attributes and leave will fail.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = { {}, { LobbyMemberAttributeId1 } };
			LobbyChanges.LocalUserLeaveReason = ELobbyMemberLeaveReason::Left;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_FALSE("Check PrepareClientChanges failed as expected.", PrepareResult.IsOk());
		}

		// 4.8 Trying to change lobby attributes while leaving will fail.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.Attributes = { {}, { LobbyAttributeId1 } };
			LobbyChanges.LocalUserLeaveReason = ELobbyMemberLeaveReason::Left;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_FALSE("Check PrepareClientChanges failed as expected.", PrepareResult.IsOk());
		}

		// 4.9 Notifications are generated correctly for lobby cleanup when the last local member leaves.
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.LocalUserLeaveReason = ELobbyMemberLeaveReason::Left;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check local member left.", CommitResult.LeavingLocalMembers.Num(), 1);
			UTEST_EQUAL("Check local member left.", CommitResult.LeavingLocalMembers[0], User1);
		}

		{
			// 4.9.1. Local member removed through local change results in member leave for all members followed by lobby leave.
			UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 3);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft.Num(), 2);
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* User1LeaveNotification = EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId == User1 ? &EventCapture.LobbyMemberLeft[0] : &EventCapture.LobbyMemberLeft[1];
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* User2LeaveNotification = EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId == User2 ? &EventCapture.LobbyMemberLeft[0] : &EventCapture.LobbyMemberLeft[1];
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Member->AccountId, User1);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User1LeaveNotification->Notification.Reason, ELobbyMemberLeaveReason::Left);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Member->AccountId, User2);
			UTEST_EQUAL("Check LobbyMemberLeft valid.", User2LeaveNotification->Notification.Reason, ELobbyMemberLeaveReason::Left);
			UTEST_EQUAL("Check LobbyLeft valid.", EventCapture.LobbyLeft.Num(), 1);
			UTEST_EQUAL("Check LobbyLeft valid.", EventCapture.LobbyLeft[0].Notification.Lobby, ClientData.GetPublicDataPtr());
			UTEST_EQUAL("Check LobbyLeft ordering.", EventCapture.LobbyLeft[0].GlobalIndex, EventCapture.GetTotalNotificationsReceived() - 1);
			FLobbyEventCapture::TNotificationInfo<FLobbyMemberLeft>* LastMemberLeaveNotification = User1LeaveNotification->GlobalIndex > User2LeaveNotification->GlobalIndex ? User1LeaveNotification : User2LeaveNotification;
			UTEST_EQUAL("Check LobbyMemberLeft ordering.", LastMemberLeaveNotification->GlobalIndex, EventCapture.LobbyLeft[0].GlobalIndex - 1);
		}
	}

	// 5. Local changes supplemental - Local user kicks remote user.
	{
		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		// 5.1 setup. Create a lobby from a snapshot containing two users.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User2);
				MemberSnapshot.AccountId = User2;
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1, User2};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 5.1 setup continued. Join User 1 to the lobby assuming the service assigned role of owner.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = { {}, {} };
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 5.1 setup continued. User 1 kicks User 2.
		{
			EventCapture.Empty();

			FLobbyClientChanges LobbyChanges;
			LobbyChanges.KickedTargetMember = User2;
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User1, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 5.1 Test local lobby owner kicking remote member.
		UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 1);
		UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft.Num(), 1);
		UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Lobby, ClientData.GetPublicDataPtr());
		UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Member->AccountId, User2);
		UTEST_EQUAL("Check LobbyMemberLeft valid.", EventCapture.LobbyMemberLeft[0].Notification.Reason, ELobbyMemberLeaveReason::Kicked);
	}

	// 6. Local changes supplemental - Local user joins a lobby, joining user is not included in lobby data snapshot.
	{
		FLobbyEvents LobbyEvents;
		FLobbyEventCapture EventCapture(LobbyEvents);
		FLobbyId LobbyId(EOnlineServices::Null, 1);
		FLobbyClientData ClientData(LobbyId, SchemaRegistry);

		FAccountId User1(EOnlineServices::Null, 1);
		FAccountId User2(EOnlineServices::Null, 2);
		int32 LobbyMaxMembers = 5;
		ELobbyJoinPolicy LobbyJoinPolicy = ELobbyJoinPolicy::PublicAdvertised;

		// 6.1 setup. Create a lobby from a snapshot containing one user.
		{
			TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
			{
				FLobbyMemberServiceSnapshot& MemberSnapshot = LobbyMemberSnapshots.Add(User1);
				MemberSnapshot.AccountId = User1;
				MemberSnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyMemberServiceAttributeId1, LobbyMemberAttribute1Value1);
			}

			FLobbyServiceSnapshot LobbySnapshot;
			LobbySnapshot.OwnerAccountId = User1;
			LobbySnapshot.MaxMembers = LobbyMaxMembers;
			LobbySnapshot.JoinPolicy = LobbyJoinPolicy;
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId1, SchemaCompatibilityAttributeValue);
			LobbySnapshot.SchemaServiceSnapshot.Attributes.Add(LobbyServiceAttributeId2, LobbyAttribute1Value1);
			LobbySnapshot.Members = {User1};

			TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
			TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareResult = ClientData.PrepareServiceSnapshot({ MoveTemp(LobbySnapshot), MoveTemp(LobbyMemberSnapshots), MoveTemp(LeaveReasons) });
			UTEST_TRUE("Check PrepareServiceSnapshot succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitServiceSnapshot::Result CommitResult = ClientData.CommitServiceSnapshot({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 6.2 Join local user to the lobby and set attribute in derived schema.
		{
			FLobbyClientChanges LobbyChanges;
			LobbyChanges.MemberAttributes = { {{ LobbyMemberAttributeId1, LobbyMemberAttribute1Value2 }}, {} };
			TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareResult = ClientData.PrepareClientChanges({ User2, MoveTemp(LobbyChanges) });
			UTEST_TRUE("Check PrepareClientChanges succeeded.", PrepareResult.IsOk());

			FLobbyClientDataCommitClientChanges::Result CommitResult = ClientData.CommitClientChanges({ &LobbyEvents });
			UTEST_EQUAL("Check no local members left.", CommitResult.LeavingLocalMembers.Num(), 0);
		}

		// 6.3 Test lobby events received.
		UTEST_EQUAL("Check events received.", EventCapture.GetTotalNotificationsReceived(), 3);
		UTEST_EQUAL("Check LobbyJoined valid.", EventCapture.LobbyJoined.Num(), 1);
		UTEST_EQUAL("Check LobbyJoined valid.", EventCapture.LobbyJoined[0].GlobalIndex, 0);
		UTEST_EQUAL("Check LobbyMemberJoined[0] valid.", EventCapture.LobbyMemberJoined.Num(), 2);
		UTEST_EQUAL("Check LobbyMemberJoined[0] valid.", EventCapture.LobbyMemberJoined[0].GlobalIndex, 1);
		UTEST_EQUAL("Check LobbyMemberJoined[0] valid.", EventCapture.LobbyMemberJoined[0].Notification.Member->AccountId, User1);
		UTEST_EQUAL("Check LobbyMemberJoined[0] attributes are valid.", EventCapture.LobbyMemberJoined[0].Notification.Member->Attributes.Num(), 1);
		const FSchemaVariant* LobbyMemberAttribute1 = EventCapture.LobbyMemberJoined[0].Notification.Member->Attributes.Find(LobbyMemberAttributeId1);
		UTEST_NOT_NULL("Check LobbyMemberJoined[0] attributes are valid.", LobbyMemberAttribute1);
		UTEST_EQUAL("Check LobbyMemberJoined[0] attributes are valid.", *LobbyMemberAttribute1, LobbyMemberAttribute1Value1);
		UTEST_EQUAL("Check LobbyMemberJoined[1] valid.", EventCapture.LobbyMemberJoined[1].GlobalIndex, 2);
		UTEST_EQUAL("Check LobbyMemberJoined[1] valid.", EventCapture.LobbyMemberJoined[1].Notification.Member->AccountId, User2);
		UTEST_EQUAL("Check LobbyMemberJoined[1] attributes are valid.", EventCapture.LobbyMemberJoined[1].Notification.Member->Attributes.Num(), 1);
		const FSchemaVariant* LobbyMemberAttribute2 = EventCapture.LobbyMemberJoined[1].Notification.Member->Attributes.Find(LobbyMemberAttributeId1);
		UTEST_NOT_NULL("Check LobbyMemberJoined[1] attributes are valid.", LobbyMemberAttribute2);
		UTEST_EQUAL("Check LobbyMemberJoined[1] attributes are valid.", *LobbyMemberAttribute2, LobbyMemberAttribute1Value2);
	}

	return true;
}

//--------------------------------------------------------------------------------------------------
// Functional testing
//--------------------------------------------------------------------------------------------------

#if LOBBIES_FUNCTIONAL_TEST_ENABLED
namespace Private {

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&, typename SecondaryOpType::Params&&)> ConsumeStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, Function Func)
{
	return [Func](TOnlineAsyncOp<OpType>& InAsyncOp, typename SecondaryOpType::Params&& InParams) mutable
	{
		TPromise<void> Promise;
		auto Future = Promise.GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.Then([Promise = MoveTemp(Promise), Op = InAsyncOp.AsShared()](TFuture<TOnlineResult<SecondaryOpType>>&& Future) mutable
		{
			if (Future.Get().IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Future.Get().GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Future.Get().GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Completed secondary operation %ls.", SecondaryOpType::Name);
			}
			Promise.EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&)> ConsumeStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, Function Func, typename SecondaryOpType::Params&& InParams)
{
	return [Func, InParams = MoveTempIfPossible(InParams)](TOnlineAsyncOp<OpType>& InAsyncOp) mutable
	{
		TPromise<void> Promise;
		auto Future = Promise.GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.Then([Promise = MoveTemp(Promise), Op = InAsyncOp.AsShared()](TFuture<TOnlineResult<SecondaryOpType>>&& Future) mutable
		{
			if (Future.Get().IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Future.Get().GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Future.Get().GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeStepResult] Completed secondary operation %ls.", SecondaryOpType::Name);
			}
			Promise.EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&, typename SecondaryOpType::Params&&)> CaptureStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, const FString& ResultKey, Function Func)
{
	return [ResultKey, Func](TOnlineAsyncOp<OpType>& InAsyncOp, typename SecondaryOpType::Params&& InParams) mutable
	{
		TPromise<void> Promise;
		auto Future = Promise.GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.Then([Promise = MoveTemp(Promise), Op = InAsyncOp.AsShared(), ResultKey](TFuture<TOnlineResult<SecondaryOpType>>&& Future) mutable
		{
			if (Future.Get().IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Future.Get().GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Future.Get().GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Captured secondary operation %ls as %ls", SecondaryOpType::Name, *ResultKey);
				Op->Data.Set(ResultKey, MoveTempIfPossible(Future.Get().GetOkValue()));
			}
			Promise.EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&)> CaptureStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, const FString& ResultKey, Function Func, typename SecondaryOpType::Params&& InParams)
{
	return [ResultKey, Func, InParams = MoveTempIfPossible(InParams)](TOnlineAsyncOp<OpType>& InAsyncOp) mutable
	{
		TPromise<void> Promise;
		auto Future = Promise.GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.Then([Promise = MoveTemp(Promise), Op = InAsyncOp.AsShared(), ResultKey](TFuture<TOnlineResult<SecondaryOpType>>&& Future) mutable
		{
			TOnlineResult<SecondaryOpType> Result = Future.Consume();
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Result.GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Result.GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureStepResult] Captured secondary operation %ls as %ls", SecondaryOpType::Name, *ResultKey);
				Op->Data.Set(ResultKey, MoveTempIfPossible(Result.GetOkValue()));
			}
			Promise.EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&, typename SecondaryOpType::Params&&)> ConsumeOperationStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, Function Func)
{
	return [Func](TOnlineAsyncOp<OpType>& InAsyncOp, typename SecondaryOpType::Params&& InParams) mutable -> TFuture<void>
	{
		auto Promise = MakeShared<TPromise<void>>();
		auto Future = Promise->GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.OnComplete([Promise, Op = InAsyncOp.AsShared()](const TOnlineResult<SecondaryOpType>& Result) mutable -> void
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Result.GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Result.GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Completed secondary operation %ls.", SecondaryOpType::Name);
			}
			Promise->EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&)> ConsumeOperationStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, Function Func, typename SecondaryOpType::Params&& InParams)
{
	return [Func, InParams = MoveTempIfPossible(InParams)](TOnlineAsyncOp<OpType>& InAsyncOp) mutable
	{
		auto Promise = MakeShared<TPromise<void>>();
		auto Future = Promise->GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.OnComplete([Promise, Op = InAsyncOp.AsShared()](const TOnlineResult<SecondaryOpType>& Result) mutable
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Result.GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Result.GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::ConsumeOperationStepResult] Completed secondary operation %ls.", SecondaryOpType::Name);
			}
			Promise->EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&, typename SecondaryOpType::Params&&)> CaptureOperationStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, const FString& ResultKey, Function Func)
{
	return [ResultKey, Func](TOnlineAsyncOp<OpType>& InAsyncOp, typename SecondaryOpType::Params&& InParams) mutable -> TFuture<void>
	{
		auto Promise = MakeShared<TPromise<void>>();
		auto Future = Promise->GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.OnComplete([Promise, Op = InAsyncOp.AsShared(), ResultKey](const TOnlineResult<SecondaryOpType>& Result) mutable -> void
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Result.GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Result.GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Captured secondary operation %ls as %ls", SecondaryOpType::Name, *ResultKey);
				Op->Data.Set(ResultKey, Result.GetOkValue());
			}
			Promise->EmplaceValue();
		});

		return Future;
	};
}

template <typename SecondaryOpType, typename OpType, typename Function>
TFunction<TFuture<void>(TOnlineAsyncOp<OpType>&)> CaptureOperationStepResult(TOnlineAsyncOp<OpType>& InAsyncOp, const FString& ResultKey, Function Func, typename SecondaryOpType::Params&& InParams)
{
	return [ResultKey, Func, InParams = MoveTempIfPossible(InParams)](TOnlineAsyncOp<OpType>& InAsyncOp) mutable
	{
		auto Promise = MakeShared<TPromise<void>>();
		auto Future = Promise->GetFuture();

		UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Starting secondary operation %ls.", SecondaryOpType::Name);

		Func(MoveTempIfPossible(InParams))
		.OnComplete([Promise, Op = InAsyncOp.AsShared(), ResultKey](const TOnlineResult<SecondaryOpType>& Result) mutable
		{
			if (Result.IsError())
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Failed secondary operation %ls. Error: %ls", SecondaryOpType::Name, *Result.GetErrorValue().GetLogString());
				Op->SetError(Errors::RequestFailure(MoveTempIfPossible(Result.GetErrorValue())));
			}
			else
			{
				UE_LOGF(LogOnlineServices, Log, "[LobbiesFunctionalTest::CaptureOperationStepResult] Captured secondary operation %ls as %ls", SecondaryOpType::Name, *ResultKey);
				Op->Data.Set(ResultKey, Result.GetOkValue());
			}
			Promise->EmplaceValue();
		});

		return Future;
	};
}

// todo: TFuture<void> does not work as expected with "Then".
TFuture<int> AwaitSleepFor(double Seconds)
{
	TSharedRef<TPromise<int>> Promise = MakeShared<TPromise<int>>();
	TFuture<int> Future = Promise->GetFuture();

	FTSTicker::GetCoreTicker().AddTicker(
	TEXT("LobbiesFunctionalTest::AwaitSleepFor"),
	Seconds,
	[Promise](float)
	{
		Promise->EmplaceValue();
		return false;
	});

	return Future;
}

// todo: TFuture<void> does not work as expected with "Then".
TFuture<int> AwaitNextGameTick()
{
	return AwaitSleepFor(0.f);
}

TFuture<TDefaultErrorResultInternal<FLobbyId>> AwaitInvitation(
	FLobbyEvents& LobbyEvents,
	FAccountId TargetAccountId,
	FLobbyId LobbyId,
	float TimeoutSeconds)
{
	// todo: Make this nicer - current implementation has issue on shutdown.
	struct AwaitInvitationState
	{
		TPromise<TDefaultErrorResultInternal<FLobbyId>> Promise;
		FOnlineEventDelegateHandle OnLobbyInvitationAddedHandle;
		FTSTicker::FDelegateHandle OnAwaitExpiredHandle;
		bool bSignaled = false;
	};

	TSharedRef<AwaitInvitationState> AwaitState = MakeShared<AwaitInvitationState>();

	AwaitState->OnLobbyInvitationAddedHandle = LobbyEvents.OnLobbyInvitationAdded.Add(
	[TargetAccountId, LobbyId, AwaitState](const FLobbyInvitationAdded& Notification)
	{
		FOnlineEventDelegateHandle DelegateHandle = MoveTemp(AwaitState->OnLobbyInvitationAddedHandle);

		if (!AwaitState->bSignaled)
		{
			if (Notification.Lobby->LobbyId == LobbyId && Notification.LocalAccountId == TargetAccountId)
			{
				FTSTicker::RemoveTicker(AwaitState->OnAwaitExpiredHandle);
				AwaitState->bSignaled = true;

				AwaitNextGameTick()
				.Then([AwaitState, LobbyId](TFuture<int>&&)
				{
					AwaitState->Promise.EmplaceValue(LobbyId);
				});
			}
		}
	});

	AwaitState->OnAwaitExpiredHandle = FTSTicker::GetCoreTicker().AddTicker(
	TEXT("LobbiesFunctionalTest::AwaitInvitation"),
	TimeoutSeconds,
	[AwaitState](float)
	{
		if (!AwaitState->bSignaled)
		{
			// Todo: Errors.
			AwaitState->bSignaled = true;
			AwaitState->OnLobbyInvitationAddedHandle.Unbind();
			AwaitState->Promise.EmplaceValue(Errors::NotImplemented());
			FTSTicker::RemoveTicker(AwaitState->OnAwaitExpiredHandle);
		}

		return false;
	});

	return AwaitState->Promise.GetFuture();
}

struct FFunctionalTestLoginUser
{
	static constexpr TCHAR Name[] = TEXT("FunctionalTestLoginUser");

	struct Params
	{
		IAuth* AuthInterface = nullptr;
		FPlatformUserId PlatformUserId;
		FName Type;
		FString Id;
		FString Token;
	};

	struct Result
	{
		TSharedPtr<FAccountInfo> AccountInfo;
	};
};

struct FFunctionalTestLogoutUser
{
	static constexpr TCHAR Name[] = TEXT("FunctionalTestLogoutUser");

	struct Params
	{
		IAuth* AuthInterface = nullptr;
		FPlatformUserId PlatformUserId;
	};

	struct Result
	{
	};
};

struct FFunctionalTestLogoutAllUsers
{
	static constexpr TCHAR Name[] = TEXT("FunctionalTestLogoutAllUsers");

	struct Params
	{
		IAuth* AuthInterface = nullptr;
	};

	struct Result
	{
	};
};

TFuture<TOnlineResult<FFunctionalTestLoginUser>> FunctionalTestLoginUser(FFunctionalTestLoginUser::Params&& Params)
{
	if (!Params.AuthInterface)
	{
		return MakeFulfilledPromise<TOnlineResult<FFunctionalTestLoginUser>>(Errors::MissingInterface()).GetFuture();
	}

	TSharedRef<TPromise<TOnlineResult<FFunctionalTestLoginUser>>> Promise = MakeShared<TPromise<TOnlineResult<FFunctionalTestLoginUser>>>();
	TFuture<TOnlineResult<FFunctionalTestLoginUser>> Future = Promise->GetFuture();

	FAuthLogin::Params LoginParams;
	LoginParams.PlatformUserId = Params.PlatformUserId;
	LoginParams.CredentialsType = Params.Type;
	LoginParams.CredentialsId = Params.Id;
	LoginParams.CredentialsToken.Set<FString>(Params.Token);

	Params.AuthInterface->Login(MoveTemp(LoginParams))
	.OnComplete([Promise, PlatformUserId = Params.PlatformUserId](const TOnlineResult<FAuthLogin>& LoginResult)
	{
		if (LoginResult.IsError())
		{
			Promise->EmplaceValue(Errors::RequestFailure());
		}
		else
		{
			Promise->EmplaceValue(FFunctionalTestLoginUser::Result{LoginResult.GetOkValue().AccountInfo});
		}
	});

	return Future;
}

TFuture<TOnlineResult<FFunctionalTestLogoutUser>> FunctionalTestLogoutUser(FFunctionalTestLogoutUser::Params&& Params)
{
	if (!Params.AuthInterface)
	{
		return MakeFulfilledPromise<TOnlineResult<FFunctionalTestLogoutUser>>(Errors::MissingInterface()).GetFuture();
	}

	FAuthGetLocalOnlineUserByPlatformUserId::Params GetAccountParams;
	GetAccountParams.PlatformUserId = Params.PlatformUserId;
	TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> Result = Params.AuthInterface->GetLocalOnlineUserByPlatformUserId(MoveTemp(GetAccountParams));
	if (Result.IsError())
	{
		// Treat invalid user as success.
		if (Result.GetErrorValue() == Errors::InvalidUser())
		{
			return MakeFulfilledPromise<TOnlineResult<FFunctionalTestLogoutUser>>(FFunctionalTestLogoutUser::Result{}).GetFuture();
		}
		else
		{
			return MakeFulfilledPromise<TOnlineResult<FFunctionalTestLogoutUser>>(Errors::Unknown(MoveTemp(Result.GetErrorValue()))).GetFuture();
		}
	}

	TSharedRef<FAccountInfo> AccountInfo = Result.GetOkValue().AccountInfo;
	if (AccountInfo->LoginStatus == ELoginStatus::NotLoggedIn)
	{
		return MakeFulfilledPromise<TOnlineResult<FFunctionalTestLogoutUser>>(Errors::InvalidUser()).GetFuture();
	}

	TSharedRef<TPromise<TOnlineResult<FFunctionalTestLogoutUser>>> Promise = MakeShared<TPromise<TOnlineResult<FFunctionalTestLogoutUser>>>();
	TFuture<TOnlineResult<FFunctionalTestLogoutUser>> Future = Promise->GetFuture();

	FAuthLogout::Params LogoutParams;
	LogoutParams.LocalAccountId = AccountInfo->AccountId;

	Params.AuthInterface->Logout(MoveTemp(LogoutParams))
	.OnComplete([Promise, PlatformUserId = Params.PlatformUserId](const TOnlineResult<FAuthLogout>& LogoutResult)
	{
		if (LogoutResult.IsError())
		{
			Promise->EmplaceValue(Errors::RequestFailure());
		}
		else
		{
			Promise->EmplaceValue(FFunctionalTestLogoutUser::Result{});
		}
	});

	return Future;
}

TFuture<TOnlineResult<FFunctionalTestLogoutAllUsers>> FunctionalTestLogoutAllUsers(FFunctionalTestLogoutAllUsers::Params&& Params)
{
	TArray<TFuture<TOnlineResult<FFunctionalTestLogoutUser>>> LogoutFutures;
	for (int32 index = 0; index < MAX_LOCAL_PLAYERS; ++index)
	{
		LogoutFutures.Emplace(FunctionalTestLogoutUser(FFunctionalTestLogoutUser::Params{Params.AuthInterface, FPlatformMisc::GetPlatformUserForUserIndex(index)}));
	}

	TSharedRef<TPromise<TOnlineResult<FFunctionalTestLogoutAllUsers>>> Promise = MakeShared<TPromise<TOnlineResult<FFunctionalTestLogoutAllUsers>>>();
	TFuture<TOnlineResult<FFunctionalTestLogoutAllUsers>> Future = Promise->GetFuture();

	WhenAll(MoveTempIfPossible(LogoutFutures))
	.Then([Promise = MoveTemp(Promise)](TFuture<TArray<TOnlineResult<FFunctionalTestLogoutUser>>>&& Results)
	{
		bool HasAnyError = false;
		for (const TOnlineResult<FFunctionalTestLogoutUser>& Result : Results.Get())
		{
			HasAnyError |= Result.IsError();
		}

		if (HasAnyError)
		{
			Promise->EmplaceValue(Errors::RequestFailure());
		}
		else
		{
			Promise->EmplaceValue(FFunctionalTestLogoutAllUsers::Result{});
		}
	});

	return Future;
}

template <typename OperationType>
class FBindOperation
{
private:
	class FBindImplBase
	{
	public:
		virtual ~FBindImplBase() = default;
		virtual TOnlineAsyncOpHandle<OperationType> operator()(typename OperationType::Params&& Params) const = 0;
	};

	template <typename ObjectType, typename Callable>
	class FBindImpl : public FBindImplBase
	{
	public:
		FBindImpl(ObjectType* Object, Callable Func)
			: Object(Object)
			, Func(Func)
		{
		}

		virtual ~FBindImpl() = default;

		virtual TOnlineAsyncOpHandle<OperationType> operator()(typename OperationType::Params&& Params) const
		{
			return (Object->*Func)(MoveTempIfPossible(Params));
		}

	private:
		ObjectType* Object;
		Callable Func;
	};

public:
	template <typename ObjectType, typename Callable>
	FBindOperation(ObjectType* Object, Callable Func)
		: Impl(MakeShared<FBindImpl<ObjectType, Callable>>(Object, MoveTempIfPossible(Func)))
	{
	}

	TOnlineAsyncOpHandle<OperationType> operator()(typename OperationType::Params&& Params) const
	{
		check(Impl);
		return (*Impl)(MoveTempIfPossible(Params));
	}

private:
	TSharedPtr<FBindImplBase> Impl;
};

} // Private

struct FFunctionalTestConfig
{
	FString TestAccount1Type;
	FString TestAccount1Id;
	FString TestAccount1Token;

	FString TestAccount2Type;
	FString TestAccount2Id;
	FString TestAccount2Token;

	float InvitationWaitSeconds = 10.f;
	float FindMatchReplicationDelay = 5.f;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FFunctionalTestConfig)
	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount1Type),
	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount1Id),
	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount1Token),

	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount2Type),
	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount2Id),
	ONLINE_STRUCT_FIELD(FFunctionalTestConfig, TestAccount2Token)
END_ONLINE_STRUCT_META()

/* Meta */ }

/**
 * Running the functional test below requires that the below config be added to DefaultEngine.ini.
 * 
 * [OnlineServices.Lobbies]
 * +SchemaDescriptors=(Id="FunctionalTestLobbies", ParentId="LobbyBase")
 * !SchemaCategoryAttributeDescriptors=ClearArray
 * +SchemaCategoryAttributeDescriptors=(SchemaId="LobbyBase", CategoryId="Lobby", AttributeIds=("SchemaCompatibilityId", "LobbyCreateTime"))
 * +SchemaCategoryAttributeDescriptors=(SchemaId="LobbyBase", CategoryId="LobbyMember")
 * +SchemaCategoryAttributeDescriptors=(SchemaId="FunctionalTestLobbies", CategoryId="Lobby", AttributeIds=("LobbyAttributeId1"))
 * +SchemaCategoryAttributeDescriptors=(SchemaId="FunctionalTestLobbies", CategoryId="LobbyMember", AttributeIds=("LobbyMemberAttributeId1"))
 * +SchemaAttributeDescriptors=(Id="SchemaCompatibilityId", Type="Int64", Flags=("Public", "SchemaCompatibilityId"))
 * +SchemaAttributeDescriptors=(Id="LobbyCreateTime", Type="Int64", Flags=("Public", "Searchable"))
 * +SchemaAttributeDescriptors=(Id="LobbyAttributeId1", Type="String", Flags=("Public"), MaxSize=64)
 * +SchemaAttributeDescriptors=(Id="LobbyMemberAttributeId1", Type="String", Flags=("Public"), MaxSize=64)
 */

TOnlineAsyncOpHandle<FFunctionalTestLobbies> RunLobbyFunctionalTest(IAuth& AuthInterface, FLobbiesCommon& LobbiesCommon, FLobbyEvents& LobbyEvents)
{
	static const FString ConfigName = TEXT("FunctionalTest");
	static const FString User1KeyName = TEXT("User1");
	static const FString User2KeyName = TEXT("User2");
	static const FString CreateLobbyKeyName = TEXT("CreateLobby");
	static const FString FindLobbyKeyName = TEXT("FindLobby");
	static const FString LobbyEventCaptureKeyName = TEXT("LobbyEventCapture");
	static const FString ConfigNameKeyName = TEXT("Config");
	static const FString SearchKeyName = TEXT("SearchKey");

	static const FSchemaId LobbySchemaId = TEXT("FunctionalTestLobbies");
	static const FSchemaAttributeId LobbyCreateTimeAttributeId = TEXT("LobbyCreateTime");
	static const FSchemaAttributeId LobbyAttributeId1 = TEXT("LobbyAttributeId1");
	static const FSchemaAttributeId LobbyMemberAttributeId1 = TEXT("LobbyMemberAttributeId1");
	static const FSchemaVariant LobbyAttribute1Value1 = TEXT("LobbyAttribute1Value1");
	static const FSchemaVariant LobbyAttribute1Value2 = TEXT("LobbyAttribute1Value2");
	static const FSchemaVariant LobbyMemberAttribute1Value1 = TEXT("LobbyMemberAttribute1Value1");
	static const FSchemaVariant LobbyMemberAttribute1Value2 = TEXT("LobbyMemberAttribute1Value2");
	static const FName LocalLobbyName = TEXT("test");

	struct FSearchParams
	{
		int64 LobbyCreateTime = 0;
	};

	TSharedRef<FFunctionalTestConfig> TestConfig = MakeShared<FFunctionalTestConfig>();
	LobbiesCommon.LoadConfig(*TestConfig, ConfigName);

	TOnlineAsyncOpRef<FFunctionalTestLobbies> Op = LobbiesCommon.GetOp<FFunctionalTestLobbies>(FFunctionalTestLobbies::Params{});

	// Set up event capturing.
	Op->Data.Set(LobbyEventCaptureKeyName, MakeShared<FLobbyEventCapture>(LobbyEvents));

	Op->Then(Private::ConsumeStepResult<Private::FFunctionalTestLogoutAllUsers>(*Op, &Private::FunctionalTestLogoutAllUsers, Private::FFunctionalTestLogoutAllUsers::Params{&AuthInterface}))
	.Then(Private::CaptureStepResult<Private::FFunctionalTestLoginUser>(*Op, User1KeyName, &Private::FunctionalTestLoginUser, Private::FFunctionalTestLoginUser::Params{&AuthInterface, FPlatformMisc::GetPlatformUserForUserIndex(0), *TestConfig->TestAccount1Type, TestConfig->TestAccount1Id, TestConfig->TestAccount1Token}))
	.Then(Private::CaptureStepResult<Private::FFunctionalTestLoginUser>(*Op, User2KeyName, &Private::FunctionalTestLoginUser, Private::FFunctionalTestLoginUser::Params{&AuthInterface, FPlatformMisc::GetPlatformUserForUserIndex(1), *TestConfig->TestAccount2Type, TestConfig->TestAccount2Id, TestConfig->TestAccount2Token}))

	//----------------------------------------------------------------------------------------------
	// Test 1:
	//    Step 1: Create a lobby with primary user.
	//    Step 2: Modify lobby attribute.
	//    Step 3: Modify lobby member attribute.
	//    Step 4: Leave lobby with primary user.
	//----------------------------------------------------------------------------------------------

	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);

		// Create a lobby
		FCreateLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LocalName = LocalLobbyName;
		Params.SchemaId = LobbySchemaId;
		Params.MaxMembers = 2;
		Params.JoinPolicy = ELobbyJoinPolicy::InvitationOnly;
		Params.Attributes.Add(LobbyAttributeId1, LobbyAttribute1Value1);
		Params.UserAttributes.Add(LobbyMemberAttributeId1, LobbyMemberAttribute1Value1);
		return Params;
	})
	.Then(Private::CaptureOperationStepResult<FCreateLobby>(*Op, CreateLobbyKeyName, Private::FBindOperation<FCreateLobby>(&LobbiesCommon, &FLobbiesCommon::CreateLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		// Check both lobby joined events were received and in the correct order.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyJoined.Num() != 1 || EventCapture->LobbyMemberJoined.Num() != 1
			|| EventCapture->LobbyJoined[0].GlobalIndex > EventCapture->LobbyMemberJoined[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}

		// Check lobby attributes are set to the expected values.
		if (!CreateResult.Lobby->Attributes.OrderIndependentCompareEqual(
			TMap<FSchemaAttributeId, FSchemaVariant>{{LobbyAttributeId1, LobbyAttribute1Value1}}))
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}

		const TSharedRef<const FLobbyMember>* MemberData = CreateResult.Lobby->Members.Find(User1Info.AccountInfo->AccountId);
		if (CreateResult.Lobby->Members.Num() == 1 && MemberData != nullptr)
		{
			if (!(**MemberData).Attributes.OrderIndependentCompareEqual(
				TMap<FSchemaAttributeId, FSchemaVariant>{{LobbyMemberAttributeId1, LobbyMemberAttribute1Value1}}))
			{
				InAsyncOp.SetError(Errors::Cancelled());
			}
		}
		else
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FModifyLobbyAttributes::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		Params.UpdatedAttributes = {{LobbyAttributeId1, LobbyAttribute1Value2}};
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FModifyLobbyAttributes>(*Op, Private::FBindOperation<FModifyLobbyAttributes>(&LobbiesCommon, &FLobbiesCommon::ModifyLobbyAttributes)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check that modification event was received.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->LobbyAttributesChanged.Num() == 1)
		{
			const FLobbyAttributesChanged& Notification = EventCapture->LobbyAttributesChanged[0].Notification;

			TSet<FSchemaAttributeId> ChangedKeys;
			Notification.ChangedAttributes.GetKeys(ChangedKeys);

			if (!ChangedKeys.Difference(TSet<FSchemaAttributeId>{LobbyAttributeId1}).IsEmpty() ||
				!Notification.Lobby->Attributes.OrderIndependentCompareEqual(TMap<FSchemaAttributeId, FSchemaVariant>{{LobbyAttributeId1, LobbyAttribute1Value2}}))
			{
				InAsyncOp.SetError(Errors::Cancelled());
			}
		}
		else
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FModifyLobbyMemberAttributes::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		Params.UpdatedAttributes = {{LobbyMemberAttributeId1, LobbyMemberAttribute1Value2}};
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FModifyLobbyMemberAttributes>(*Op, Private::FBindOperation<FModifyLobbyMemberAttributes>(&LobbiesCommon, &FLobbiesCommon::ModifyLobbyMemberAttributes)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);

		// Check that modification event was received.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->LobbyMemberAttributesChanged.Num() == 1)
		{
			const FLobbyMemberAttributesChanged& Notification = EventCapture->LobbyMemberAttributesChanged[0].Notification;
			if (const TSharedRef<const FLobbyMember>* MemberData = Notification.Lobby->Members.Find(User1Info.AccountInfo->AccountId))
			{
				TSet<FSchemaAttributeId> ChangedKeys;
				Notification.ChangedAttributes.GetKeys(ChangedKeys);

				if (!ChangedKeys.Difference(TSet<FSchemaAttributeId>{LobbyMemberAttributeId1}).IsEmpty() ||
					!(**MemberData).Attributes.OrderIndependentCompareEqual(TMap<FSchemaAttributeId, FSchemaVariant>{{LobbyMemberAttributeId1, LobbyMemberAttribute1Value2}}))
				{
					InAsyncOp.SetError(Errors::Cancelled());
				}
			}
			else
			{
				InAsyncOp.SetError(Errors::Cancelled());
			}
		}
		else
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FLeaveLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FLeaveLobby>(*Op, Private::FBindOperation<FLeaveLobby>(&LobbiesCommon, &FLobbiesCommon::LeaveLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check for expected events.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyLeft.Num() != 1 || EventCapture->LobbyMemberLeft.Num() != 1
			|| EventCapture->LobbyLeft[0].GlobalIndex < EventCapture->LobbyMemberLeft[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})

	//----------------------------------------------------------------------------------------------
	// Test 2:
	//    Step 1: Create a lobby with primary user.
	//    Step 2: Primary user invites secondary user.
	//    Step 3: Join lobby with secondary user.
	//    Step 4: Leave lobby with secondary user.
	//    Step 5: Leave lobby with primary user.
	//----------------------------------------------------------------------------------------------

	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);

		FCreateLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LocalName = LocalLobbyName;
		Params.SchemaId = LobbySchemaId;
		Params.MaxMembers = 2;
		Params.JoinPolicy = ELobbyJoinPolicy::InvitationOnly;
		//Params.Attributes;
		return Params;
	})
	.Then(Private::CaptureOperationStepResult<FCreateLobby>(*Op, CreateLobbyKeyName, Private::FBindOperation<FCreateLobby>(&LobbiesCommon, &FLobbiesCommon::CreateLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check both lobby joined events were received and in the correct order.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyJoined.Num() != 1 || EventCapture->LobbyMemberJoined.Num() != 1
			|| EventCapture->LobbyJoined[0].GlobalIndex > EventCapture->LobbyMemberJoined[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FInviteLobbyMember::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		Params.TargetAccountId = User2Info.AccountInfo->AccountId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FInviteLobbyMember>(*Op, Private::FBindOperation<FInviteLobbyMember>(&LobbiesCommon, &FLobbiesCommon::InviteLobbyMember)))
	.Then([TestConfig, LobbyEventsPtr = &LobbyEvents](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);
		return Private::AwaitInvitation(*LobbyEventsPtr, User2Info.AccountInfo->AccountId, CreateLobbyResult.Lobby->LobbyId, TestConfig->InvitationWaitSeconds);
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp, TDefaultErrorResultInternal<FLobbyId>&& Result)
	{
		if (Result.IsError())
		{
			InAsyncOp.SetError(Errors::Cancelled(Result.GetErrorValue()));
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FJoinLobby::Params Params;
		Params.LocalAccountId = User2Info.AccountInfo->AccountId;
		Params.LocalName = LocalLobbyName;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FJoinLobby>(*Op, Private::FBindOperation<FJoinLobby>(&LobbiesCommon, &FLobbiesCommon::JoinLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check both lobby joined events were received and in the correct order.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 1 || EventCapture->LobbyMemberJoined.Num() != 1)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FLeaveLobby::Params Params;
		Params.LocalAccountId = User2Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FLeaveLobby>(*Op, Private::FBindOperation<FLeaveLobby>(&LobbiesCommon, &FLobbiesCommon::LeaveLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check for expected events.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 1 || EventCapture->LobbyMemberLeft.Num() != 1)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FLeaveLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FLeaveLobby>(*Op, Private::FBindOperation<FLeaveLobby>(&LobbiesCommon, &FLobbiesCommon::LeaveLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check for expected events.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyLeft.Num() != 1 || EventCapture->LobbyMemberLeft.Num() != 1
			|| EventCapture->LobbyLeft[0].GlobalIndex < EventCapture->LobbyMemberLeft[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})

	//----------------------------------------------------------------------------------------------
	// Test 3: Simple search
	//    Step 1: Create a lobby with primary user.
	//    Step 2: Secondary user searches for lobbies.
	//    Step 3: Join lobby with secondary user.
	//    Step 4: Leave lobby with secondary user.
	//    Step 5: Leave lobby with primary user.
	//----------------------------------------------------------------------------------------------

	// todo: additional functional tests for search.
	// find by lobby id, find by user, find by attributes. Search types are mutually exclusive and
	// should return invalid args if multiple search types are passed for a search.

	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);

		TSharedRef<FSearchParams> SearchParams = MakeShared<FSearchParams>();
		SearchParams->LobbyCreateTime = static_cast<int64>(FPlatformTime::Seconds());
		InAsyncOp.Data.Set(SearchKeyName, TSharedRef<FSearchParams>(SearchParams));

		FCreateLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LocalName = LocalLobbyName;
		Params.SchemaId = LobbySchemaId;
		Params.MaxMembers = 2;
		Params.JoinPolicy = ELobbyJoinPolicy::PublicAdvertised;
		Params.Attributes = {{ LobbyCreateTimeAttributeId, SearchParams->LobbyCreateTime }};
		return Params;
	})
	.Then(Private::CaptureOperationStepResult<FCreateLobby>(*Op, CreateLobbyKeyName, Private::FBindOperation<FCreateLobby>(&LobbiesCommon, &FLobbiesCommon::CreateLobby)))
	.Then([TestConfig](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check both lobby joined events were received and in the correct order.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyJoined.Num() != 1 || EventCapture->LobbyMemberJoined.Num() != 1
			|| EventCapture->LobbyJoined[0].GlobalIndex > EventCapture->LobbyMemberJoined[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}

		// Wait some time so lobby creation can propagate before searching for it on another client.
		return Private::AwaitSleepFor(TestConfig->FindMatchReplicationDelay);
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp, int)
	{
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);
		const TSharedRef<FSearchParams>& SearchParams = GetOpDataChecked<TSharedRef<FSearchParams>>(InAsyncOp, SearchKeyName);

		// Search for lobby from create. Searching by attribute will also limit results by bucket id.
		FFindLobbies::Params Params;
		Params.LocalAccountId = User2Info.AccountInfo->AccountId;
		Params.Filters = {{LobbyCreateTimeAttributeId, ESchemaAttributeComparisonOp::Equals, SearchParams->LobbyCreateTime}};
		return Params;
	})
	.Then(Private::CaptureOperationStepResult<FFindLobbies>(*Op, FindLobbyKeyName, Private::FBindOperation<FFindLobbies>(&LobbiesCommon, &FLobbiesCommon::FindLobbies)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);
		const FFindLobbies::Result& FindResults = GetOpDataChecked<FFindLobbies::Result>(InAsyncOp, FindLobbyKeyName);

		if (FindResults.Lobbies.Num() != 1)
		{
			InAsyncOp.SetError(Errors::Cancelled());
			return FJoinLobby::Params{};
		}

		FJoinLobby::Params Params;
		Params.LocalAccountId = User2Info.AccountInfo->AccountId;
		Params.LocalName = LocalLobbyName;
		Params.LobbyId = FindResults.Lobbies[0]->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FJoinLobby>(*Op, Private::FBindOperation<FJoinLobby>(&LobbiesCommon, &FLobbiesCommon::JoinLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check both lobby joined events were received and in the correct order.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 1 || EventCapture->LobbyMemberJoined.Num() != 1)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User2Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User2KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FLeaveLobby::Params Params;
		Params.LocalAccountId = User2Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FLeaveLobby>(*Op, Private::FBindOperation<FLeaveLobby>(&LobbiesCommon, &FLobbiesCommon::LeaveLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check for expected events.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 1|| EventCapture->LobbyMemberLeft.Num() != 1)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Prepare for next test check.
		GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName)->Empty();

		const Private::FFunctionalTestLoginUser::Result& User1Info = GetOpDataChecked<Private::FFunctionalTestLoginUser::Result>(InAsyncOp, User1KeyName);
		const FCreateLobby::Result& CreateLobbyResult = GetOpDataChecked<FCreateLobby::Result>(InAsyncOp, CreateLobbyKeyName);

		FLeaveLobby::Params Params;
		Params.LocalAccountId = User1Info.AccountInfo->AccountId;
		Params.LobbyId = CreateLobbyResult.Lobby->LobbyId;
		return Params;
	})
	.Then(Private::ConsumeOperationStepResult<FLeaveLobby>(*Op, Private::FBindOperation<FLeaveLobby>(&LobbiesCommon, &FLobbiesCommon::LeaveLobby)))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		// Check for expected events.
		const TSharedRef<FLobbyEventCapture>& EventCapture = GetOpDataChecked<TSharedRef<FLobbyEventCapture>>(InAsyncOp, LobbyEventCaptureKeyName);
		if (EventCapture->GetTotalNotificationsReceived() != 2
			|| EventCapture->LobbyLeft.Num() != 1 || EventCapture->LobbyMemberLeft.Num() != 1
			|| EventCapture->LobbyLeft[0].GlobalIndex < EventCapture->LobbyMemberLeft[0].GlobalIndex)
		{
			InAsyncOp.SetError(Errors::Cancelled());
		}
	})

	//----------------------------------------------------------------------------------------------
	// Complete
	//----------------------------------------------------------------------------------------------
	// Complete the test by logging out all users.
	.Then(Private::ConsumeStepResult<Private::FFunctionalTestLogoutAllUsers>(*Op, &Private::FunctionalTestLogoutAllUsers, Private::FFunctionalTestLogoutAllUsers::Params{ &AuthInterface }))
	.Then([](TOnlineAsyncOp<FFunctionalTestLobbies>& InAsyncOp)
	{
		InAsyncOp.SetResult(FFunctionalTestLobbies::Result{});
	})
	.Enqueue(LobbiesCommon.GetServices().GetParallelQueue());

	return Op->GetHandle();
}

#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

/* UE::Online */ }
