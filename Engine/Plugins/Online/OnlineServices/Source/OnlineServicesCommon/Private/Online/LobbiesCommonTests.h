// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/LobbiesCommonTypes.h"

namespace UE::Online {

class IAuth;
class FLobbiesCommon;
struct FLobbyEvents;

#if LOBBIES_FUNCTIONAL_TEST_ENABLED

TOnlineAsyncOpHandle<FFunctionalTestLobbies> RunLobbyFunctionalTest(IAuth& AuthInterface, FLobbiesCommon& LobbiesCommon, FLobbyEvents& LobbyEvents);

#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

/* UE::Online */ }
