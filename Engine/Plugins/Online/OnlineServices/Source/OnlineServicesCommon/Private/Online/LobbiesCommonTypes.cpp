// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LobbiesCommonTypes.h"

#include "Online/OnlineServicesLog.h"

namespace UE::Online {

FSchemaId LobbyBaseSchemaId = TEXT("LobbyBase");
FSchemaCategoryId LobbySchemaCategoryId = TEXT("Lobby");
FSchemaCategoryId LobbyMemberSchemaCategoryId = TEXT("LobbyMember");

FLobbyClientData::FLobbyClientData(FLobbyId LobbyId, const TSharedRef<const FSchemaRegistry>& InSchemaRegistry)
	: SchemaRegistry(InSchemaRegistry)
	, InternalPublicData(MakeShared<FLobbyInternal>(InSchemaRegistry))
{
	InternalPublicData->LobbyId = LobbyId;
}

TSharedPtr<const FLobbyMemberInternal> FLobbyClientData::GetMemberData(FAccountId MemberAccountId) const
{
	const TSharedRef<FLobbyMemberInternal>* FoundMemberData = MemberDataStorage.Find(MemberAccountId);
	return FoundMemberData ? *FoundMemberData : TSharedPtr<const FLobbyMemberInternal>();
}

TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> FLobbyClientData::PrepareServiceSnapshot(FLobbyClientDataPrepareServiceSnapshot::Params&& Params) const
{
	FPreparedServiceChanges NewPreparedServiceChanges;

	// Reset any pending changes. No other changes can occur while applying a service snapshot.
	ResetPreparedChanges();

	// Prepare lobby attribute changes.
	TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareLobbyServiceSnapshotResult =
		InternalPublicData->SchemaData.PrepareServiceSnapshot({ TOptional<FSchemaId>(), MoveTemp(Params.LobbySnapshot.SchemaServiceSnapshot) });
	if (PrepareLobbyServiceSnapshotResult.IsError())
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareServiceSnapshot] Failed to prepare lobby service snapshot: Lobby[%ls], Result[%ls]",
			*ToLogString(InternalPublicData->LobbyId), *PrepareLobbyServiceSnapshotResult.GetErrorValue().GetLogString());
		return TOnlineResult<FLobbyClientDataPrepareServiceSnapshot>(MoveTemp(PrepareLobbyServiceSnapshotResult.GetErrorValue()));
	}

	// Get schema id to use when applying member changes.
	// Preparing the lobby snapshot may have resulted in a schema change.
	TOptional<FSchemaId> NewDerivedSchemaId = PrepareLobbyServiceSnapshotResult.GetOkValue().DerivedSchemaId;

	// Process members who left.
	// Build list of leaving members by iterating over the known members.
	for (const TPair<FAccountId, TSharedRef<FLobbyMemberInternal>>& MemberData : MemberDataStorage)
	{
		// Check if member exists in new snapshot.
		if (!Params.LobbySnapshot.Members.Contains(MemberData.Key))
		{
			// Member has left - check if they gave an explicit reason.
			if (const ELobbyMemberLeaveReason* MemberLeaveReason = Params.LeaveReasons.Find(MemberData.Key))
			{
				NewPreparedServiceChanges.LeavingMembers.Add({MemberData.Value, *MemberLeaveReason});
			}
			else
			{
				const ELobbyMemberLeaveReason DefaultLeaveReason = ELobbyMemberLeaveReason::Disconnected;
				NewPreparedServiceChanges.LeavingMembers.Add({ MemberData.Value, DefaultLeaveReason });
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareServiceSnapshot] Member left lobby without giving reason, defaulting to %ls: Lobby[%ls], Member[%ls]",
					*ToLogString(InternalPublicData->LobbyId), LexToString(DefaultLeaveReason), *ToLogString(MemberData.Key));
			}
		}
	}

	// Process member joins and attribute changes.
	// Joining members are expected to have a member data snapshot.
	for (TPair<FAccountId, FLobbyMemberServiceSnapshot>& MemberSnapshotPair : Params.LobbyMemberSnapshots)
	{
		if (!Params.LobbySnapshot.Members.Contains(MemberSnapshotPair.Key))
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareServiceSnapshot] Member update ignored for unknown member: Lobby[%ls], Member[%ls]",
				*ToLogString(InternalPublicData->LobbyId), *ToLogString(MemberSnapshotPair.Key));
		}
		else
		{
			FLobbyMemberServiceSnapshot& MemberSnapshot = MemberSnapshotPair.Value;
			FLobbyMemberInternal* PreparingMemberData = nullptr;

			// Check whether this member is already in the lobby.
			if (const TSharedRef<FLobbyMemberInternal>* MemberData = MemberDataStorage.Find(MemberSnapshot.AccountId))
			{
				NewPreparedServiceChanges.UpdatedMemberDataStorage.Add(*MemberData);
				PreparingMemberData = &MemberData->Get();
			}
			else
			{
				// New member, create an entry for them.
				TSharedRef<FLobbyMemberInternal> NewMemberData = MakeShared<FLobbyMemberInternal>(SchemaRegistry);
				NewMemberData->AccountId = MemberSnapshot.AccountId;
				NewPreparedServiceChanges.NewMemberDataStorage.Add(NewMemberData);
				PreparingMemberData = &NewMemberData.Get();
			}

			// Try to prepare the lobby member snapshot.
			TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareLobbyMemberServiceSnapshotResult =
				PreparingMemberData->SchemaData.PrepareServiceSnapshot({ NewDerivedSchemaId, MoveTemp(MemberSnapshot.SchemaServiceSnapshot) });
			if (PrepareLobbyMemberServiceSnapshotResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareServiceSnapshot] Failed to prepare lobby member service snapshot: Lobby[%ls], Member[%ls], Result[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(MemberSnapshot.AccountId), *PrepareLobbyMemberServiceSnapshotResult.GetErrorValue().GetLogString());
				return TOnlineResult<FLobbyClientDataPrepareServiceSnapshot>(MoveTemp(PrepareLobbyMemberServiceSnapshotResult.GetErrorValue()));
			}
		}
	}

	// Check whether lobby ownership changed and is valid.
	if (InternalPublicData->OwnerAccountId != Params.LobbySnapshot.OwnerAccountId)
	{
		if (!Params.LobbySnapshot.Members.Contains(Params.LobbySnapshot.OwnerAccountId))
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareServiceSnapshot] Lobby owner chage failed - lobby member data not found: Lobby[%ls], Member[%ls]",
				*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LobbySnapshot.OwnerAccountId));
			return TOnlineResult<FLobbyClientDataPrepareServiceSnapshot>(Errors::InvalidState());
		}

		NewPreparedServiceChanges.OwnerAccountId = Params.LobbySnapshot.OwnerAccountId;
	}

	// Check for max members change.
	if (InternalPublicData->MaxMembers != Params.LobbySnapshot.MaxMembers)
	{
		NewPreparedServiceChanges.MaxMembers = Params.LobbySnapshot.MaxMembers;
	}

	// Check for join policy change.
	if (InternalPublicData->JoinPolicy != Params.LobbySnapshot.JoinPolicy)
	{
		NewPreparedServiceChanges.JoinPolicy = Params.LobbySnapshot.JoinPolicy;
	}
	
	// Store prepared changes.
	PreparedServiceChanges = MoveTemp(NewPreparedServiceChanges);

	return TOnlineResult<FLobbyClientDataPrepareServiceSnapshot>(FLobbyClientDataPrepareServiceSnapshot::Result{});
}

FLobbyClientDataCommitServiceSnapshot::Result FLobbyClientData::CommitServiceSnapshot(FLobbyClientDataCommitServiceSnapshot::Params&& Params)
{
	FLobbyClientDataCommitServiceSnapshot::Result CommitResult;
	const bool bDispatchNotifications = Params.LobbyEvents != nullptr && !LocalMembers.IsEmpty();

	if (!PreparedServiceChanges)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::CommitServiceSnapshot] Commit failed, no pending changes found: Lobby[%ls]",
			*ToLogString(InternalPublicData->LobbyId));
		return CommitResult;
	}

	// Commit schema changes.
	FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitLobbyServiceSnapshotResult = InternalPublicData->SchemaData.CommitServiceSnapshot();
	FSchemaServiceClientChanges& LobbyServiceClientChanges = CommitLobbyServiceSnapshotResult.ClientChanges;

	// Notify if schema changed.
	// Schema change notification must be processed first as the schema affects how the client processes attributes.
	if (LobbyServiceClientChanges.SchemaId)
	{
		FLobbySchemaChanged LobbySchemaChanged = { InternalPublicData, InternalPublicData->SchemaId, *LobbyServiceClientChanges.SchemaId };
		InternalPublicData->SchemaId = *LobbyServiceClientChanges.SchemaId;
		if (bDispatchNotifications)
		{
			Params.LobbyEvents->OnLobbySchemaChanged.Broadcast(LobbySchemaChanged);
		}
	}

	const bool bAnyLobbyAttributesChanged =
		!LobbyServiceClientChanges.AddedAttributes.IsEmpty() ||
		!LobbyServiceClientChanges.ChangedAttributes.IsEmpty() ||
		!LobbyServiceClientChanges.RemovedAttributes.IsEmpty();

	// Handle lobby attribute change notifications.
	if (bDispatchNotifications && bAnyLobbyAttributesChanged)
	{
		Params.LobbyEvents->OnLobbyAttributesChanged.Broadcast({
			InternalPublicData,
			MoveTemp(LobbyServiceClientChanges.AddedAttributes),
			MoveTemp(LobbyServiceClientChanges.ChangedAttributes),
			MoveTemp(LobbyServiceClientChanges.RemovedAttributes) });
	}

	// Handle joining members.
	for (TSharedRef<FLobbyMemberInternal> NewMember : PreparedServiceChanges->NewMemberDataStorage)
	{
		// Commit attribute changes.
		NewMember->SchemaData.CommitServiceSnapshot();
		
		// Add member to lobby structures.
		AddMember(NewMember);

		// Notify.
		if (bDispatchNotifications)
		{
			Params.LobbyEvents->OnLobbyMemberJoined.Broadcast({ InternalPublicData, NewMember });
		}
	}

	// Handle updated members.
	for (TSharedRef<FLobbyMemberInternal> UpdatedMember : PreparedServiceChanges->UpdatedMemberDataStorage)
	{
		// Commit attribute changes.
		FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitLobbyMemberServiceSnapshotResult = UpdatedMember->SchemaData.CommitServiceSnapshot();
		FSchemaServiceClientChanges& LobbyMemberServiceClientChanges = CommitLobbyMemberServiceSnapshotResult.ClientChanges;

		const bool bAnyLobbyMemberAttributesChanged =
			!LobbyMemberServiceClientChanges.AddedAttributes.IsEmpty() ||
			!LobbyMemberServiceClientChanges.ChangedAttributes.IsEmpty() ||
			!LobbyMemberServiceClientChanges.RemovedAttributes.IsEmpty();

		// Handle lobby attribute changes.
		if (bDispatchNotifications && bAnyLobbyMemberAttributesChanged)
		{
			Params.LobbyEvents->OnLobbyMemberAttributesChanged.Broadcast({
				InternalPublicData,
				UpdatedMember,
				MoveTemp(LobbyMemberServiceClientChanges.AddedAttributes),
				MoveTemp(LobbyMemberServiceClientChanges.ChangedAttributes),
				MoveTemp(LobbyMemberServiceClientChanges.RemovedAttributes) });
		}
	}

	// Handle leaving members.
	for (TPair<TSharedRef<FLobbyMemberInternal>, ELobbyMemberLeaveReason> LeavingMemberPair : PreparedServiceChanges->LeavingMembers)
	{
		RemoveMember(LeavingMemberPair.Key);

		if (LeavingMemberPair.Key->bIsLocalMember)
		{
			CommitResult.LeavingLocalMembers.Add(LeavingMemberPair.Key->AccountId);
		}

		if (bDispatchNotifications)
		{
			Params.LobbyEvents->OnLobbyMemberLeft.Broadcast({ InternalPublicData, LeavingMemberPair.Key, LeavingMemberPair.Value });
		}
	}

	// Check for ownership change.
	if (PreparedServiceChanges->OwnerAccountId)
	{
		// We check validity of the member data because at the time when the snapshot is retrieved containing a host change, that new host might have already left the session
		if (TSharedRef<FLobbyMemberInternal>* OwnerMemberData = MemberDataStorage.Find(*PreparedServiceChanges->OwnerAccountId))
		{
			InternalPublicData->OwnerAccountId = *PreparedServiceChanges->OwnerAccountId;

			if (bDispatchNotifications)
			{
				Params.LobbyEvents->OnLobbyLeaderChanged.Broadcast(FLobbyLeaderChanged{ InternalPublicData, *OwnerMemberData });
			}
		}
		else
		{
			InternalPublicData->OwnerAccountId = FAccountId();
		}		
	}

	// Check for max member change.
	if (PreparedServiceChanges->MaxMembers)
	{
		InternalPublicData->MaxMembers = *PreparedServiceChanges->MaxMembers;
	}

	// Check for join policy change.
	if (PreparedServiceChanges->JoinPolicy)
	{
		InternalPublicData->JoinPolicy = *PreparedServiceChanges->JoinPolicy;
	}

	// If no local members remain in the lobby process lobby removal.
	if (bDispatchNotifications && LocalMembers.IsEmpty())
	{
		TArray<TSharedRef<FLobbyMemberInternal>> LeavingRemoteMembers;
		MemberDataStorage.GenerateValueArray(LeavingRemoteMembers);

		for (TSharedRef<FLobbyMemberInternal>& LeavingRemoteMember : LeavingRemoteMembers)
		{
			RemoveMember(LeavingRemoteMember);
			Params.LobbyEvents->OnLobbyMemberLeft.Broadcast(FLobbyMemberLeft{ InternalPublicData, LeavingRemoteMember, ELobbyMemberLeaveReason::Left });
		}

		Params.LobbyEvents->OnLobbyLeft.Broadcast(FLobbyLeft{ InternalPublicData });
	}

	// Clear out update.
	ResetPreparedChanges();
	return CommitResult;
}

TOnlineResult<FLobbyClientDataPrepareClientChanges> FLobbyClientData::PrepareClientChanges(FLobbyClientDataPrepareClientChanges::Params&& Params) const
{
	FPreparedClientChanges NewPreparedClientChanges;
	FLobbyClientDataPrepareClientChanges::Result PrepareResult;

	// Reset any pending changes. No other changes can occur while applying a service snapshot.
	ResetPreparedChanges();

	// The lobby schema may be changed by applying the update. Track the schema that is switched to so that the member attributes use the correct schema.
	FSchemaId LobbySchemaId = InternalPublicData->SchemaId;

	NewPreparedClientChanges.LocalAccountId = Params.LocalAccountId;

	// Prepare lobby changes.
	{
		// Only the lobby owner may change some lobby data through local changes.
		bool bLobbyDataChanged = false;

		// Check for local name change.
		if (Params.ClientChanges.LocalName && InternalPublicData->LocalName != *Params.ClientChanges.LocalName)
		{
			if (InternalPublicData->LocalName != FName())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Local name chage failed - local name cannot be changed once set: Lobby[%ls], CurrentName[%ls], NewName[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *InternalPublicData->LocalName.ToString().ToLower(), *Params.ClientChanges.LocalName->ToString().ToLower());
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}

			NewPreparedClientChanges.LocalName = *Params.ClientChanges.LocalName;
		}

		// Check for join policy change.
		if (Params.ClientChanges.JoinPolicy && InternalPublicData->JoinPolicy != Params.ClientChanges.JoinPolicy)
		{
			bLobbyDataChanged = true;
			NewPreparedClientChanges.JoinPolicy = *Params.ClientChanges.JoinPolicy;
			PrepareResult.ServiceChanges.JoinPolicy = *Params.ClientChanges.JoinPolicy;
		}

		// Check for leader change.
		if (Params.ClientChanges.OwnerAccountId && InternalPublicData->OwnerAccountId != Params.ClientChanges.OwnerAccountId)
		{
			if (!MemberDataStorage.Contains(*Params.ClientChanges.OwnerAccountId))
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Lobby owner chage failed - new owner is not in lobby: Lobby[%ls], NewOwner[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(*Params.ClientChanges.OwnerAccountId));
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}

			bLobbyDataChanged = true;
			NewPreparedClientChanges.OwnerAccountId = *Params.ClientChanges.OwnerAccountId;
			PrepareResult.ServiceChanges.OwnerAccountId = *Params.ClientChanges.OwnerAccountId;
		}

		// Check for user to kick.
		if (Params.ClientChanges.KickedTargetMember)
		{
			if (!MemberDataStorage.Contains(*Params.ClientChanges.KickedTargetMember))
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Kick target member failed - target is not in the lobby: Lobby[%ls], KickedTargetMember[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(*Params.ClientChanges.KickedTargetMember));
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}

			bLobbyDataChanged = true;
			NewPreparedClientChanges.KickedTargetMember = *Params.ClientChanges.KickedTargetMember;
			PrepareResult.ServiceChanges.KickedTargetMember = *Params.ClientChanges.KickedTargetMember;
		}

		// Check for attribute changes.
		if (Params.ClientChanges.LobbySchema || Params.ClientChanges.Attributes)
		{
			bLobbyDataChanged = true;

			// Prepare lobby attribute changes.
			FSchemaCategoryInstancePrepareClientChanges::Params PrepareClientChangesParams;
			if (Params.ClientChanges.LobbySchema)
			{
				PrepareClientChangesParams.ClientChanges.SchemaId = *Params.ClientChanges.LobbySchema;

				// Update to using the new schema when applying a member update below.
				LobbySchemaId = *Params.ClientChanges.LobbySchema;
			}
			if (Params.ClientChanges.Attributes)
			{
				PrepareClientChangesParams.ClientChanges.UpdatedAttributes = MoveTemp(Params.ClientChanges.Attributes->UpdatedAttributes);
				PrepareClientChangesParams.ClientChanges.RemovedAttributes = MoveTemp(Params.ClientChanges.Attributes->RemovedAttributes);
			}

			TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult =
				InternalPublicData->SchemaData.PrepareClientChanges(MoveTemp(PrepareClientChangesParams));
			if (PrepareClientChangesResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Failed to prepare lobby attribute changes: Lobby[%ls], Result[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *PrepareClientChangesResult.GetErrorValue().GetLogString());
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(MoveTemp(PrepareClientChangesResult.GetErrorValue()));
			}

			// Get attribute changes to be written to the service.
			PrepareResult.ServiceChanges.UpdatedAttributes = MoveTemp(PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes);
			PrepareResult.ServiceChanges.RemovedAttributes = MoveTemp(PrepareClientChangesResult.GetOkValue().ServiceChanges.RemovedAttributes);

			NewPreparedClientChanges.bLobbyAttributesChanged = true;
		}

		if (bLobbyDataChanged)
		{
			// Check that the local user can make lobby changes.
			if (Params.LocalAccountId != InternalPublicData->OwnerAccountId)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Failed to prepare lobby changes. LocalAccountId is not the lobby owner: Lobby[%ls], LocalAccountId[%ls], OwnerAccountId[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LocalAccountId), *ToLogString(InternalPublicData->OwnerAccountId));
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}

			// Check that the local user is not trying to change the lobby while leaving.
			if (Params.ClientChanges.LocalUserLeaveReason)
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Failed to prepare lobby changes. Unable to modify the lobby while leaving: Lobby[%ls], LocalAccountId[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LocalAccountId));
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}
		}
	}

	// Prepare lobby member changes.
	{
		// Check that the local user is not joining and leaving.
		if (Params.ClientChanges.MemberAttributes && Params.ClientChanges.LocalUserLeaveReason)
		{
			UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Failed to prepare lobby member changes. Member cannot be updated and removed: Lobby[%ls], LocalAccountId[%ls]",
				*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LocalAccountId));
			return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
		}

		// Check whether the local user changed their attributes.
		if (Params.ClientChanges.MemberAttributes)
		{
			FLobbyMemberInternal* PreparingMemberData = nullptr;

			// Check if this is a new or existing member.
			if (const TSharedRef<FLobbyMemberInternal>* MemberData = MemberDataStorage.Find(Params.LocalAccountId))
			{
				NewPreparedClientChanges.UpdatedLocalMember = *MemberData;
				PreparingMemberData = &MemberData->Get();

				// If the member data exists and has not been flagged as a local user, then they
				// are joining after a service snapshot with them present has been applied.
				if (!LocalMembers.Contains(Params.LocalAccountId))
				{
					NewPreparedClientChanges.JoiningLocalMember = *MemberData;
				}
			}
			else
			{
				// New member who does not have a service snapshot, create an entry for them using the current lobby schema.
				TSharedRef<FLobbyMemberInternal> NewMemberData = MakeShared<FLobbyMemberInternal>(SchemaRegistry, LobbySchemaId);
				NewMemberData->AccountId = Params.LocalAccountId;
				NewPreparedClientChanges.JoiningLocalMember = NewMemberData;
				PreparingMemberData = &NewMemberData.Get();
			}

			// Prepare lobby member attribute changes.
			FSchemaCategoryInstancePrepareClientChanges::Params PrepareClientChangesParams;
			if (LobbySchemaId != InternalPublicData->SchemaId)
			{
				PrepareClientChangesParams.ClientChanges.SchemaId = LobbySchemaId;
			}
			PrepareClientChangesParams.ClientChanges.UpdatedAttributes = MoveTemp(Params.ClientChanges.MemberAttributes->UpdatedAttributes);
			PrepareClientChangesParams.ClientChanges.RemovedAttributes = MoveTemp(Params.ClientChanges.MemberAttributes->RemovedAttributes);

			TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult =
				PreparingMemberData->SchemaData.PrepareClientChanges(MoveTemp(PrepareClientChangesParams));
			if (PrepareClientChangesResult.IsError())
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Failed to prepare lobby member attribute changes: Lobby[%ls], LocalMember[%ls], Result[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LocalAccountId), *PrepareClientChangesResult.GetErrorValue().GetLogString());
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(MoveTemp(PrepareClientChangesResult.GetErrorValue()));
			}

			// Get attribute changes to be written to the service.
			PrepareResult.ServiceChanges.UpdatedMemberAttributes = MoveTemp(PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes);
			PrepareResult.ServiceChanges.RemovedMemberAttributes = MoveTemp(PrepareClientChangesResult.GetOkValue().ServiceChanges.RemovedAttributes);
		}

		// Check whether the local user left.
		if (Params.ClientChanges.LocalUserLeaveReason)
		{
			if (!MemberDataStorage.Contains(Params.LocalAccountId))
			{
				UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::PrepareClientChanges] Local user leave faild - user is not in the lobby: Lobby[%ls], LocalAccountId[%ls]",
					*ToLogString(InternalPublicData->LobbyId), *ToLogString(Params.LocalAccountId));
				return TOnlineResult<FLobbyClientDataPrepareClientChanges>(Errors::InvalidParams());
			}

			NewPreparedClientChanges.LocalUserLeaveReason = *Params.ClientChanges.LocalUserLeaveReason;
		}
	}

	PreparedClientChanges = NewPreparedClientChanges;
	return TOnlineResult<FLobbyClientDataPrepareClientChanges>(MoveTemp(PrepareResult));
}

FLobbyClientDataCommitClientChanges::Result FLobbyClientData::CommitClientChanges(FLobbyClientDataCommitClientChanges::Params&& Params)
{
	FLobbyClientDataCommitClientChanges::Result CommitResult;
	bool bDispatchNotifications = Params.LobbyEvents != nullptr && !LocalMembers.IsEmpty();

	if (!PreparedClientChanges)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FLobbyClientData::CommitClientChanges] Commit failed, no pending changes found: Lobby[%ls]",
			*ToLogString(InternalPublicData->LobbyId));
		return CommitResult;
	}

	// Commit lobby changes.
	{
		// Assign local name.
		if (PreparedClientChanges->LocalName)
		{
			InternalPublicData->LocalName = *PreparedClientChanges->LocalName;
		}

		// Assign join policy.
		if (PreparedClientChanges->JoinPolicy)
		{
			InternalPublicData->JoinPolicy = *PreparedClientChanges->JoinPolicy;
		}

		// Assign join policy
		if (PreparedClientChanges->OwnerAccountId)
		{
			InternalPublicData->OwnerAccountId = *PreparedClientChanges->OwnerAccountId;

			if (bDispatchNotifications)
			{
				TSharedRef<FLobbyMemberInternal> MemberData = MemberDataStorage.FindChecked(InternalPublicData->OwnerAccountId);
				Params.LobbyEvents->OnLobbyLeaderChanged.Broadcast({ InternalPublicData, MemberData });
			}
		}

		// Remove kicked member.
		if (PreparedClientChanges->KickedTargetMember)
		{
			TSharedRef<FLobbyMemberInternal> MemberData = MemberDataStorage.FindChecked(*PreparedClientChanges->KickedTargetMember);
			RemoveMember(MemberData);

			if (bDispatchNotifications)
			{
				Params.LobbyEvents->OnLobbyMemberLeft.Broadcast({ InternalPublicData, MemberData, ELobbyMemberLeaveReason::Kicked });
			}
		}

		// Commit schema changes if needed.
		if (PreparedClientChanges->bLobbyAttributesChanged)
		{
			FSchemaCategoryInstanceCommitClientChanges::Result CommitLobbyServiceSnapshotResult = InternalPublicData->SchemaData.CommitClientChanges();
			FSchemaServiceClientChanges& LobbyServiceClientChanges = CommitLobbyServiceSnapshotResult.ClientChanges;

			// Notify if schema changed.
			// Schema change notification must be processed first as the schema affects how the client processes attributes.
			if (LobbyServiceClientChanges.SchemaId)
			{
				FLobbySchemaChanged LobbySchemaChanged = { InternalPublicData, InternalPublicData->SchemaId, *LobbyServiceClientChanges.SchemaId };
				InternalPublicData->SchemaId = *LobbyServiceClientChanges.SchemaId;
				if (bDispatchNotifications)
				{
					Params.LobbyEvents->OnLobbySchemaChanged.Broadcast(LobbySchemaChanged);
				}
			}

			const bool bAnyLobbyAttributesChanged =
				!LobbyServiceClientChanges.AddedAttributes.IsEmpty() ||
				!LobbyServiceClientChanges.ChangedAttributes.IsEmpty() ||
				!LobbyServiceClientChanges.RemovedAttributes.IsEmpty();

			// Handle lobby attribute change notifications.
			if (bDispatchNotifications && bAnyLobbyAttributesChanged)
			{
				Params.LobbyEvents->OnLobbyAttributesChanged.Broadcast({
					InternalPublicData,
					MoveTemp(LobbyServiceClientChanges.AddedAttributes),
					MoveTemp(LobbyServiceClientChanges.ChangedAttributes),
					MoveTemp(LobbyServiceClientChanges.RemovedAttributes) });
			}
		}
	}

	// Check for an updating local member.
	if (PreparedClientChanges->UpdatedLocalMember)
	{
		TSharedRef<FLobbyMemberInternal> UpdatedMember = PreparedClientChanges->UpdatedLocalMember.ToSharedRef();

		// Commit attribute changes.
		FSchemaCategoryInstanceCommitClientChanges::Result CommitLobbyMemberServiceSnapshotResult = UpdatedMember->SchemaData.CommitClientChanges();
		FSchemaServiceClientChanges& LobbyMemberServiceClientChanges = CommitLobbyMemberServiceSnapshotResult.ClientChanges;

		const bool bAnyLobbyMemberAttributesChanged =
			!LobbyMemberServiceClientChanges.AddedAttributes.IsEmpty() ||
			!LobbyMemberServiceClientChanges.ChangedAttributes.IsEmpty() ||
			!LobbyMemberServiceClientChanges.RemovedAttributes.IsEmpty();

		// Handle lobby attribute changes.
		if (bDispatchNotifications && bAnyLobbyMemberAttributesChanged)
		{
			Params.LobbyEvents->OnLobbyMemberAttributesChanged.Broadcast({
				InternalPublicData,
				UpdatedMember,
				MoveTemp(LobbyMemberServiceClientChanges.AddedAttributes),
				MoveTemp(LobbyMemberServiceClientChanges.ChangedAttributes),
				MoveTemp(LobbyMemberServiceClientChanges.RemovedAttributes) });
		}
	}

	// Commit lobby member changes.
	{
		// Check for a new joining local member.
		if (PreparedClientChanges->JoiningLocalMember)
		{
			// Check whether notifications should be turned on.
			if (LocalMembers.IsEmpty())
			{
				bDispatchNotifications = Params.LobbyEvents != nullptr;
				if (bDispatchNotifications)
				{
					// Signal that the lobby has been joined.
					Params.LobbyEvents->OnLobbyJoined.Broadcast({ InternalPublicData });

					// Dispatch a notification for each existing remote member.
					for (TPair<FAccountId, TSharedRef<FLobbyMemberInternal>>& MemberPair : MemberDataStorage)
					{
						Params.LobbyEvents->OnLobbyMemberJoined.Broadcast({ InternalPublicData, MemberPair.Value });
					}
				}
			}

			TSharedRef<FLobbyMemberInternal> JoinedMember = PreparedClientChanges->JoiningLocalMember.ToSharedRef();

			JoinedMember->bIsLocalMember = true;
			// todo.
			//NewMemberData->PlatformAccountId = ;
			//NewMemberData->PlatformDisplayName = ;

			const bool bWasAlreadyJoined = MemberDataStorage.Contains(JoinedMember->AccountId);

			// Add to member tracking.
			AddMember(JoinedMember);

			// Commit attribute changes.
			JoinedMember->SchemaData.CommitClientChanges();

			// Send join notification for the local user.
			if (bDispatchNotifications && !bWasAlreadyJoined)
			{
				Params.LobbyEvents->OnLobbyMemberJoined.Broadcast({ InternalPublicData, JoinedMember });
			}
		}

		// Check if the local user left.
		if (PreparedClientChanges->LocalUserLeaveReason)
		{
			TSharedRef<FLobbyMemberInternal> LeavingMember = MemberDataStorage.FindChecked(PreparedClientChanges->LocalAccountId);
			RemoveMember(LeavingMember);
			CommitResult.LeavingLocalMembers.Add(PreparedClientChanges->LocalAccountId);

			if (bDispatchNotifications)
			{
				Params.LobbyEvents->OnLobbyMemberLeft.Broadcast({ InternalPublicData, LeavingMember, *PreparedClientChanges->LocalUserLeaveReason });

				if (LocalMembers.IsEmpty())
				{
					// Fire leave events for remote members.
					TArray<TSharedRef<FLobbyMemberInternal>> LeavingRemoteMembers;
					MemberDataStorage.GenerateValueArray(LeavingRemoteMembers);

					for (TSharedRef<FLobbyMemberInternal>& LeavingRemoteMember : LeavingRemoteMembers)
					{
						RemoveMember(LeavingMember);
						Params.LobbyEvents->OnLobbyMemberLeft.Broadcast(FLobbyMemberLeft{ InternalPublicData, LeavingRemoteMember, ELobbyMemberLeaveReason::Left });
					}

					// Notify lobby has been left.
					Params.LobbyEvents->OnLobbyLeft.Broadcast({ InternalPublicData });
				}
			}
		}
	}

	ResetPreparedChanges();
	return CommitResult;
}

void FLobbyClientData::ResetPreparedChanges() const
{
	PreparedServiceChanges.Reset();
	PreparedClientChanges.Reset();
}

void FLobbyClientData::AddMember(const TSharedRef<FLobbyMemberInternal>& Member)
{
	MemberDataStorage.Add(Member->AccountId, Member);
	InternalPublicData->Members.Add(Member->AccountId, Member);

	if (Member->bIsLocalMember)
	{
		LocalMembers.Add(Member->AccountId);
	}
}

void FLobbyClientData::RemoveMember(const TSharedRef<FLobbyMemberInternal>& Member)
{
	MemberDataStorage.Remove(Member->AccountId);
	InternalPublicData->Members.Remove(Member->AccountId);

	if (Member->bIsLocalMember)
	{
		LocalMembers.Remove(Member->AccountId);
	}
}

/* UE::Online */ }
