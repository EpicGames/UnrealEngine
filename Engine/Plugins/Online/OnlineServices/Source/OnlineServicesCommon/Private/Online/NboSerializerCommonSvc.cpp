// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/NboSerializerCommonSvc.h"
#include "Online/NboSerializer.h"
#include "Online/SessionsCommon.h"

/**
 * Serializes data in network byte order form into a buffer
 */
namespace UE::Online {

namespace NboSerializerCommonSvc {

/** SerializeToBuffer methods */

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSchemaVariant& Data)
{
	Packet << (uint8)Data.VariantType;

	switch (Data.VariantType)
	{
	case ESchemaAttributeType::Bool: Packet << Data.GetBoolean(); break;
	case ESchemaAttributeType::Double: Packet << Data.GetDouble(); break;
	case ESchemaAttributeType::Int64: Packet << (int32)Data.GetInt64(); break;
	case ESchemaAttributeType::String: Packet << Data.GetString(); break;
	}
}

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FCustomSessionSetting& CustomSessionSetting)
{
	SerializeToBuffer(Packet, CustomSessionSetting.Data);
	Packet << CustomSessionSetting.ID;
	Packet << (uint8)CustomSessionSetting.Visibility;
}

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FCustomSessionSettingsMap& CustomSessionSettingsMap)
{
	// First count the number of advertised keys
	int32 NumAdvertisedProperties = 0;
	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : CustomSessionSettingsMap)
	{
		const FCustomSessionSetting& Setting = Entry.Value;
		if (Setting.Visibility == ESchemaAttributeVisibility::Public)
		{
			NumAdvertisedProperties++;
		}
	}

	// Add the count of advertised keys and the data
	Packet << NumAdvertisedProperties;

	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : CustomSessionSettingsMap)
	{
		const FCustomSessionSetting& Setting = Entry.Value;
		if (Setting.Visibility == ESchemaAttributeVisibility::Public)
		{
			Packet << Entry.Key;
			SerializeToBuffer(Packet, Setting);
		}
	}
}

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionInfo& SessionInfo)
{
	Packet << SessionInfo.bAllowSanctionedPlayers;
	Packet << SessionInfo.bAntiCheatProtected;
	Packet << SessionInfo.bIsDedicatedServerSession;
	Packet << SessionInfo.bIsLANSession;
	Packet << SessionInfo.SessionIdOverride;
	// SessionInfo.SessionId will be serialized in implementations as user id types will vary
}

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionSettings& SessionSettings)
{
	Packet << SessionSettings.bAllowNewMembers;
	SerializeToBuffer(Packet, SessionSettings.CustomSettings);
	Packet << (uint8)SessionSettings.JoinPolicy;
	Packet << SessionSettings.NumMaxConnections;
	Packet << SessionSettings.SchemaName;
}

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionCommon& Session)
{
	// Session.OwnerAccountId will be serialized in implementations as user id types will vary
	SerializeToBuffer(Packet, Session.SessionInfo);
	SerializeToBuffer(Packet, Session.SessionSettings);
	// Session.SessionMembers will be serialized in implementations as user id types will vary
}

/** SerializeFromBuffer methods */

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSchemaVariant& Data)
{
	uint8 VariantType = 0;
	Packet >> VariantType;
	Data.VariantType = (ESchemaAttributeType)VariantType;

	switch (Data.VariantType)
	{
	case ESchemaAttributeType::Bool:
	{
		uint8 Read;
		Packet >> Read;
		Data.Set(!!Read);
		break;
	}
	case ESchemaAttributeType::Double:
	{
		double Read = 0.0;
		Packet >> Read;
		Data.Set(Read);
		break;
	}
	case ESchemaAttributeType::Int64:
	{
		int32 Read = 0;
		Packet >> Read;
		Data.Set((int64)Read);
		break;
	}
	case ESchemaAttributeType::String:
	{
		FString Read;
		Packet >> Read;
		Data.Set(Read);
		break;
	}
	}
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FCustomSessionSetting& CustomSessionSetting)
{
	SerializeFromBuffer(Packet, CustomSessionSetting.Data);
	Packet >> CustomSessionSetting.ID;
	uint8 VisibilityNum = 0;
	Packet >> VisibilityNum;
	CustomSessionSetting.Visibility = (ESchemaAttributeVisibility)VisibilityNum;
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FCustomSessionSettingsMap& CustomSessionSettingsMap)
{
	int32 NumEntries = 0;
	Packet >> NumEntries;

	for (int32 Index = 0; Index < NumEntries; ++Index)
	{
		FSchemaAttributeId Key;
		Packet >> Key;

		FCustomSessionSetting Value;
		SerializeFromBuffer(Packet, Value);

		CustomSessionSettingsMap.Emplace(Key, Value);
	}
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionInfo& SessionInfo)
{
	Packet >> SessionInfo.bAllowSanctionedPlayers;
	Packet >> SessionInfo.bAntiCheatProtected;
	Packet >> SessionInfo.bIsDedicatedServerSession;
	Packet >> SessionInfo.bIsLANSession;
	Packet >> SessionInfo.SessionIdOverride;

	// SessionInfo.SessionId will be deserialized in implementations as user id types will vary
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionSettings& SessionSettings)
{
	Packet >> SessionSettings.bAllowNewMembers;

	SerializeFromBuffer(Packet, SessionSettings.CustomSettings);

	uint8 Read = 0;
	Packet >> Read;
	SessionSettings.JoinPolicy = (ESessionJoinPolicy)Read;

	Packet >> SessionSettings.NumMaxConnections;
	Packet >> SessionSettings.SchemaName;
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionCommon& Session)
{
	// Session.OwnerUserId will be deserialized in implementations as user id types will vary
	SerializeFromBuffer(Packet, Session.SessionInfo);
	SerializeFromBuffer(Packet, Session.SessionSettings);
	// Session.SessionMembers will be deserialized in implementations as user id types will vary
}

/* NboSerializerCommonSvc */ }

/* UE::Online */ }
