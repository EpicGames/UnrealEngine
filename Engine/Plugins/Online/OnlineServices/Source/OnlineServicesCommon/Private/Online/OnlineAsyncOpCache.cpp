// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineAsyncOpCache.h"
#include "Online/OnlineExecHandler.h" // IWYU pragma: keep
#include "Online/OnlineServicesCommon.h"

namespace UE::Online {

TSharedRef<FOnlineAsyncOpCache> FOnlineAsyncOpCache::GetSharedThis()
{
	return TSharedRef<FOnlineAsyncOpCache>(Services.AsShared(), this);
}

void FOnlineAsyncOpCache::ClearAllCallbacks()
{
	for (const TUniquePtr<IWrappedOperation>& WrappedOp : IndependentOperations)
	{
		WrappedOp->ClearCallback();
	}

	for (const TPair<FAccountId, TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>>& UserOperationPair : UserOperations)
	{
		ClearCallbacks(UserOperationPair.Value);
	}

	ClearCallbacks(Operations);
}

void FOnlineAsyncOpCache::CancelAll()
{
	// Move to temp array instead of using IndependentOperations directly because Cancel will remove from that array during for loop
	TArray<TUniquePtr<IWrappedOperation>> TempIndependentOperations = MoveTemp(IndependentOperations);
	for (const TUniquePtr<IWrappedOperation>& WrappedOp : TempIndependentOperations)
	{
		WrappedOp->Cancel();
	}

	for (const TPair<FAccountId, TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>>& UserOperationPair : UserOperations)
	{
		CancelOperations(UserOperationPair.Value);
	}

	CancelOperations(Operations);
}

bool FOnlineAsyncOpCache::HasAnyRunningOperation() const
{
	for (const TUniquePtr<IWrappedOperation>& WrappedOp : IndependentOperations)
	{
		if (WrappedOp->GetAsyncOpState() < EAsyncOpState::Complete)
		{
			return true;
		}
	}

	for (const TPair<FAccountId, TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>>& UserOperationPair : UserOperations)
	{
		if (HasAnyRunningOperationIn(UserOperationPair.Value))
		{
			return true;
		}
	}

	return HasAnyRunningOperationIn(Operations);
}

bool FOnlineAsyncOpCache::HasAnyRunningOperationIn(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations) const
{
	for (const TPair<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& OperationPair : InOperations)
	{
		const TUniquePtr<IWrappedOperation>& WrappedOp = OperationPair.Value;
		if (WrappedOp->GetAsyncOpState() < EAsyncOpState::Complete)
		{
			return true;
		}
	}

	return false;
}

void FOnlineAsyncOpCache::ClearCallbacks(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations)
{
	for (const TPair<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& OperationPair : InOperations)
	{
		const TUniquePtr<IWrappedOperation>& WrappedOp = OperationPair.Value;
		WrappedOp->ClearCallback();
	}
}

void FOnlineAsyncOpCache::CancelOperations(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations)
{
	bool bAllCanceled = false;
	while (!bAllCanceled)
	{
		bAllCanceled = true;

		for (const TPair<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& OperationPair : InOperations)
		{
			const TUniquePtr<IWrappedOperation>& WrappedOp = OperationPair.Value;
			if (WrappedOp->GetAsyncOpState() < EAsyncOpState::Complete)
			{
				WrappedOp->Cancel();
				bAllCanceled = false;
				// InOperations changed during Cancel(), can't continue here in the for loop.
				break;
			}
		}
	}
}

/* UE::Online*/ }
