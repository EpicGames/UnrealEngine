// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineConfig.h"
#include "Misc/AsciiSet.h"
#include "Misc/ConfigCacheIni.h"

DEFINE_LOG_CATEGORY_STATIC(LogOnlineServicesConfig, Log, VeryVerbose);

namespace
{
using namespace UE::Online;

constexpr FAsciiSet Equals(TEXT("="));
constexpr FAsciiSet StringChars(TEXT("abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789_+"));
	
void SkipWhitespace(const TCHAR*& Str)
{
	while (FChar::IsWhitespace(*Str))
	{
		Str++;
	}
}

const TCHAR* ParseField(const TCHAR* InStr, TMap<FString, FString>& OutMap)
{
	const TCHAR* Start = InStr;
	const TCHAR* Str = FAsciiSet::FindFirstOrEnd(Start, Equals);
	if (*Str)
	{
		int32 Len = UE_PTRDIFF_TO_INT32(Str - Start);
		FString FieldName = FString::ConstructFromPtrSize(Start, Len);

		Str++;
		Start = Str;
		// Quoted string
		if (*Str == TCHAR('\"'))
		{
			Start++;
			do { Str++; } while (*Str && *Str != TCHAR('\"'));
			if (*Str != TCHAR('\"'))
			{
				UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched quoted string: %ls", InStr);
				return nullptr;
			}

			Len = UE_PTRDIFF_TO_INT32(Str - Start);
			FString FieldValue= FString::ConstructFromPtrSize(Start, Len);

			OutMap.Emplace(MoveTemp(FieldName), MoveTemp(FieldValue));
			Str++;
		}
		// Struct
		else if (*Str == TCHAR('('))
		{
			int SubCount = 0;
			Str++;
			while (*Str && (SubCount > 0 || *Str != TCHAR(')')))
			{
				SkipWhitespace(Str);
				if (*Str == TCHAR('\"'))
				{
					do { Str++; } while (*Str && *Str != TCHAR('\"'));
					if (*Str != TCHAR('\"'))
					{
						UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched quoted string: %ls", InStr);
						return nullptr;
					}
				}
				else if (*Str == TCHAR('('))
				{
					SubCount++;
				}
				else if(*Str == TCHAR(')'))
				{
					SubCount--;
					if (SubCount < 0)
					{
						UE_LOGF(LogOnlineServicesConfig, Warning, "Too many closing parenthesis in: %ls", InStr);
						return nullptr;
					}
				}
				Str++;
			}
			Str++;

			Len = UE_PTRDIFF_TO_INT32(Str - Start);
			FString FieldValue = FString::ConstructFromPtrSize(Start, Len);

			OutMap.Emplace(MoveTemp(FieldName), MoveTemp(FieldValue));
		}
		// Unquoted string
		else
		{
			Str = FAsciiSet::Skip(Str, StringChars);
				
			Len = UE_PTRDIFF_TO_INT32(Str - Start);
			FString FieldValue = FString::ConstructFromPtrSize(Start, Len);

			OutMap.Emplace(MoveTemp(FieldName), MoveTemp(FieldValue));
		}

		// Skip comma.
		if (*Str == TCHAR(','))
		{
			Str++;
		}
		SkipWhitespace(Str);
	}

	return Str;
}
}

namespace UE::Online {

IOnlineConfigStructPtr FOnlineConfigStructGConfig::CreateStruct(const FString& InConfigValue)
{
	IOnlineConfigStructPtr Result;
	ParseStruct(GetData(InConfigValue), Result);
	return Result;
}

TArray<IOnlineConfigStructPtr> FOnlineConfigStructGConfig::CreateStructArray(const FString& InConfigValue)
{
	TArray<IOnlineConfigStructPtr> Result;

	if (!InConfigValue.IsEmpty())
	{
		const TCHAR* Str = GetData(InConfigValue);
		// Start of array
		if(*Str == TCHAR('('))
		{
			Str++;
			// Start of array struct element
			while (Str && *Str == TCHAR('('))
			{	
				IOnlineConfigStructPtr StructPtr;
				Str = ParseStruct(Str, StructPtr);
				if (StructPtr)
				{
					Result.Emplace(MoveTemp(StructPtr));
				}
			}
			if (Str && *Str != TCHAR(')'))
			{
				UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched opening parenthesis: %ls", Str);
			}
		}
	}

	return Result;
}

const TCHAR* FOnlineConfigStructGConfig::ParseStruct(const TCHAR* InStr, IOnlineConfigStructPtr& OutPtr)
{
	const TCHAR* Str = InStr;
	if (*Str && *Str == TCHAR('('))
	{
		Str++;
		FOnlineConfigStructGConfig Struct { FPrivateToken{} };
		while (*Str && *Str != TCHAR(')'))
		{
			Str = ParseField(Str, Struct.StructMembers);
			if (!Str)
			{
				return nullptr;
			}
		}
		if (*Str != TCHAR(')'))
		{
			UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched opening parenthesis: %ls", Str);
			return nullptr;
		}
		Str++;
		// Skip comma.
		if (*Str == TCHAR(','))
		{
			Str++;
		}
		SkipWhitespace(Str);

		OutPtr = MakeShared<FOnlineConfigStructGConfig>(MoveTemp(Struct));
	}
	else
	{
		UE_LOGF(LogOnlineServicesConfig, Warning, "Failed to parse struct: %ls", Str);
		return nullptr;
	}

	return Str;
}

TArray<FString> FOnlineConfigStructGConfig::CreateValueArray(const FString& InConfigValue)
{
	TArray<FString> Result;

	if (!InConfigValue.IsEmpty())
	{
		const TCHAR* Str = GetData(InConfigValue);
		// Start of array
		if (*Str == TCHAR('('))
		{
			Str++;
			while (*Str && *Str != TCHAR(')'))
			{
				const TCHAR* Start = Str;

				// Quoted string
				if (*Str == TCHAR('\"'))
				{
					Start++;
					do { Str++; } while (*Str && *Str != TCHAR('\"'));
					if (*Str != TCHAR('\"'))
					{
						UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched quoted string: %ls", Str);
						return Result;
					}

					const int32 Len = UE_PTRDIFF_TO_INT32(Str - Start);
					FString FieldValue = FString::ConstructFromPtrSize(Start, Len);
					Result.Emplace(MoveTemp(FieldValue));
					Str++;
				}
				// Unquoted string
				else
				{
					Str = FAsciiSet::Skip(Str, StringChars);

					const int32 Len = UE_PTRDIFF_TO_INT32(Str - Start);
					FString FieldValue = FString::ConstructFromPtrSize(Start, Len);
					Result.Emplace(MoveTemp(FieldValue));
				}

				// Skip comma.
				if (*Str == TCHAR(','))
				{
					Str++;
				}
				SkipWhitespace(Str);
			}
			if (*Str != TCHAR(')'))
			{
				UE_LOGF(LogOnlineServicesConfig, Warning, "Unmatched opening parenthesis: %ls", Str);
			}
		}
	}

	return Result;
}

bool FOnlineConfigStructGConfig::GetValue(const TCHAR* Key, FString& Value)
{
	if (const FString* Found = StructMembers.Find(Key))
	{
		Value = *Found;
		return true;
	}
	return false;
}

int32 FOnlineConfigStructGConfig::GetValue(const TCHAR* Key, TArray<FString>& Value)
{
	if (const FString* Found = StructMembers.Find(Key))
	{
		Value = CreateValueArray(*Found);
		return Value.Num();
	}
	return 0;
}

bool FOnlineConfigStructGConfig::GetValue(const TCHAR* Key, IOnlineConfigStructPtr& Value)
{
	if (const FString* Found = StructMembers.Find(Key))
	{
		if (IOnlineConfigStructPtr ConfigStructPtr = CreateStruct(*Found))
		{
			Value = ConfigStructPtr;
			return true;
		}
	}
	return false;
}

int32 FOnlineConfigStructGConfig::GetValue(const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value)
{
	if (const FString* Found = StructMembers.Find(Key))
	{
		Value = CreateStructArray(*Found);
		return Value.Num();
	}
	return 0;
}

bool FOnlineConfigProviderGConfig::GetValue(const TCHAR* Section, const TCHAR* Key, FString& Value)
{
	return GConfig->GetValue(Section, Key, Value, ConfigFile);
}

int32 FOnlineConfigProviderGConfig::GetValue(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value)
{
	return GConfig->GetValue(Section, Key, Value, ConfigFile);
}

bool FOnlineConfigProviderGConfig::GetValue(const TCHAR* Section, const TCHAR* Key, IOnlineConfigStructPtr& Value)
{
	FString ConfigValue;
	if (GConfig->GetValue(Section, Key, ConfigValue, ConfigFile))
	{
		if (IOnlineConfigStructPtr StructPtr = FOnlineConfigStructGConfig::CreateStruct(ConfigValue))
		{
			Value = MoveTemp(StructPtr);
			return true;
		}
	}

	Value = nullptr;
	return false;
}

int32 FOnlineConfigProviderGConfig::GetValue(const TCHAR* Section, const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value)
{
	Value.Empty();
	TArray<FString> ConfigValues;
	if (GConfig->GetValue(Section, Key, ConfigValues, ConfigFile))
	{
		Value.Reserve(ConfigValues.Num());
		for (const FString& ConfigValue : ConfigValues)
		{
			if (IOnlineConfigStructPtr StructPtr = FOnlineConfigStructGConfig::CreateStruct(ConfigValue))
			{
				Value.Emplace(MoveTemp(StructPtr));
			}
		}
	}
	return Value.Num();
}

/* UE::Online */ }
