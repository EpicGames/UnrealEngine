// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesCommon.h"

#include "Online/IOnlineComponent.h"
#include "Online/OnlineAsyncOpCache_Meta.inl"
#include "Online/OnlineExecHandler.h"

namespace UE::Online {

struct FOnlineServicesCommonConfig
{
	int32 MaxConcurrentOperations = 16;
	float SecondsToSleepForOutstandingOperations = 0.01f;
	float SecondsToFlushForOutstandingOperations = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FOnlineServicesCommonConfig)
	ONLINE_STRUCT_FIELD(FOnlineServicesCommonConfig, MaxConcurrentOperations),
	ONLINE_STRUCT_FIELD(FOnlineServicesCommonConfig, SecondsToSleepForOutstandingOperations),
	ONLINE_STRUCT_FIELD(FOnlineServicesCommonConfig, SecondsToFlushForOutstandingOperations)
END_ONLINE_STRUCT_META()

}

uint32 FOnlineServicesCommon::NextInstanceIndex = 0;

FOnlineServicesCommon::FOnlineServicesCommon(const FString& InServiceConfigName, FName InInstanceName, FName InInstanceConfigName)
	: OpCache(InServiceConfigName, *this)
	, InstanceIndex(NextInstanceIndex++)
	, InstanceName(InInstanceName)
	, InstanceConfigName(InInstanceConfigName)
	, ConfigProvider(MakeUnique<FOnlineConfigProviderGConfig>(GEngineIni))
	, ServiceConfigName(InServiceConfigName)
	, SerialQueue(ParallelQueue)
	, bPreShutdownComplete(false)
{
}

FOnlineServicesCommon::~FOnlineServicesCommon()
{
}

void FOnlineServicesCommon::Init()
{
	OpCache.SetLoadConfigFn(
		[this](FOperationConfig& OperationConfig, const TArray<FString>& SectionHierarchy)
		{
			return LoadConfig(OperationConfig, SectionHierarchy);
		});

	RegisterComponents();
	Initialize();
	PostInitialize();

	UE_LOGF(LogOnlineServices, Log, "%p %ls online services instance initialize", this, LexToString(GetServicesProvider()));
}

void FOnlineServicesCommon::Flush(EAsyncOpFlushReason FlushReason)
{
	UE_LOGF(LogOnlineServices, Log, "%p %ls online services instance flushing remain operations", this, LexToString(GetServicesProvider()));

	FOnlineServicesCommonConfig Config;
	LoadConfig(Config);

	float SecondsToSleep = Config.SecondsToSleepForOutstandingOperations;
	double CurrentTime = FPlatformTime::Seconds();
	double LastFlushTickTime = CurrentTime;
	double BeginWaitTime = CurrentTime;
	double TimeToWaitBeforeCanceling = Config.SecondsToFlushForOutstandingOperations;

	while (OpCache.HasAnyRunningOperation() && (CurrentTime - BeginWaitTime) < TimeToWaitBeforeCanceling)
	{
		FlushTick(CurrentTime - LastFlushTickTime);
		LastFlushTickTime = CurrentTime;
		FPlatformProcess::Sleep(SecondsToSleep);
		CurrentTime = FPlatformTime::Seconds();
	}

	if (OpCache.HasAnyRunningOperation())
	{
		UE_LOGF(LogOnlineServices, Warning, "%p %ls online services flush timed out, cancelling all operations", this, LexToString(GetServicesProvider()));

		if (FlushReason == EAsyncOpFlushReason::Shutdown)
		{
			OpCache.ClearAllCallbacks();
		}

		OpCache.CancelAll();
	}
}

void FOnlineServicesCommon::FlushTick(float DeltaSeconds)
{
	Tick(DeltaSeconds);

	// In case any AsyncOp relies on CoreTicker
	FTSTicker::GetCoreTicker().Tick(DeltaSeconds);

	// TaskGraph also needs to be ticked to process messages that come back to the game thread
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
}

void FOnlineServicesCommon::Destroy()
{
	if (OpCache.HasAnyRunningOperation())
	{
		Flush(EAsyncOpFlushReason::Shutdown);
	}

	UE_LOGF(LogOnlineServices, Log, "%p %ls online services instance destroy", this, LexToString(GetServicesProvider()));

	PreShutdown();

	Shutdown();
}

IAchievementsPtr FOnlineServicesCommon::GetAchievementsInterface()
{
	return IAchievementsPtr(AsShared(), Get<IAchievements>());
}

ICommercePtr FOnlineServicesCommon::GetCommerceInterface()
{
	return ICommercePtr(AsShared(), Get<ICommerce>());
}


IAuthPtr FOnlineServicesCommon::GetAuthInterface()
{
	return IAuthPtr(AsShared(), Get<IAuth>());
}

IUserInfoPtr FOnlineServicesCommon::GetUserInfoInterface()
{
	return IUserInfoPtr(AsShared(), Get<IUserInfo>());
}

ISocialPtr FOnlineServicesCommon::GetSocialInterface()
{
	return ISocialPtr(AsShared(), Get<ISocial>());
}

IPresencePtr FOnlineServicesCommon::GetPresenceInterface()
{
	return IPresencePtr(AsShared(), Get<IPresence>());
}

IExternalUIPtr FOnlineServicesCommon::GetExternalUIInterface()
{
	return IExternalUIPtr(AsShared(), Get<IExternalUI>());
}

ILeaderboardsPtr FOnlineServicesCommon::GetLeaderboardsInterface()
{
	return ILeaderboardsPtr(AsShared(), Get<ILeaderboards>());
}

ILobbiesPtr FOnlineServicesCommon::GetLobbiesInterface()
{
	return ILobbiesPtr(AsShared(), Get<ILobbies>());
}

ISessionsPtr FOnlineServicesCommon::GetSessionsInterface()
{
	return ISessionsPtr(AsShared(), Get<ISessions>());
}

IStatsPtr FOnlineServicesCommon::GetStatsInterface()
{
	return IStatsPtr(AsShared(), Get<IStats>());
}

IConnectivityPtr FOnlineServicesCommon::GetConnectivityInterface()
{
	return IConnectivityPtr(AsShared(), Get<IConnectivity>());
}

IPrivilegesPtr FOnlineServicesCommon::GetPrivilegesInterface()
{
	return IPrivilegesPtr(AsShared(), Get<IPrivileges>());
}

ITitleFilePtr FOnlineServicesCommon::GetTitleFileInterface()
{
	return ITitleFilePtr(AsShared(), Get<ITitleFile>());
}

IUserFilePtr FOnlineServicesCommon::GetUserFileInterface()
{
	return IUserFilePtr(AsShared(), Get<IUserFile>());
}

TOnlineResult<FGetResolvedConnectString> FOnlineServicesCommon::GetResolvedConnectString(FGetResolvedConnectString::Params&& Params)
{
	return TOnlineResult<FGetResolvedConnectString>(Errors::NotImplemented());
}

FName FOnlineServicesCommon::GetInstanceName() const
{
	return InstanceName;
}

FName FOnlineServicesCommon::GetInstanceConfigName() const
{
	return InstanceConfigName;
}

void FOnlineServicesCommon::AssignBaseInterfaceSharedPtr(const FOnlineTypeName& TypeName, void* OutBaseInterfaceSP)
{
	Components.AssignBaseSharedPtr(TypeName, OutBaseInterfaceSP);
}

void FOnlineServicesCommon::RegisterComponents()
{
}

void FOnlineServicesCommon::Initialize()
{
	Components.Visit(&IOnlineComponent::Initialize);
}

void FOnlineServicesCommon::PostInitialize()
{
	Components.Visit(&IOnlineComponent::PostInitialize);

	LoadCommonConfig();
}

void FOnlineServicesCommon::UpdateConfig()
{
	Components.Visit(&IOnlineComponent::UpdateConfig);

	LoadCommonConfig();
}

bool FOnlineServicesCommon::Tick(float DeltaSeconds)
{
	Components.Visit(&IOnlineComponent::Tick, DeltaSeconds);

	ParallelQueue.Tick(DeltaSeconds);

	return true;
}

void FOnlineServicesCommon::PreShutdown()
{
	Components.Visit(&IOnlineComponent::PreShutdown);

	bPreShutdownComplete = true;
}

void FOnlineServicesCommon::Shutdown()
{
	Components.Visit(&IOnlineComponent::Shutdown);
}

FOnlineAsyncOpQueueParallel& FOnlineServicesCommon::GetParallelQueue()
{
	return ParallelQueue;
}

FOnlineAsyncOpQueue& FOnlineServicesCommon::GetSerialQueue()
{
	return SerialQueue;
}

FOnlineAsyncOpQueue& FOnlineServicesCommon::GetSerialQueue(const FAccountId& AccountId)
{
	TUniquePtr<FOnlineAsyncOpQueueSerial>* Queue = PerUserSerialQueue.Find(AccountId);
	if (Queue == nullptr)
	{
		Queue = &PerUserSerialQueue.Emplace(AccountId, MakeUnique<FOnlineAsyncOpQueueSerial>(ParallelQueue));
	}

	return **Queue;
}

void FOnlineServicesCommon::RegisterExecHandler(const FString& Name, TUniquePtr<IOnlineExecHandler>&& Handler)
{
	ExecCommands.Emplace(Name, MoveTemp(Handler));
}

#if UE_ALLOW_EXEC_COMMANDS
bool FOnlineServicesCommon::Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("OnlineServices")))
	{
		int Index = 0;
		if (FParse::Value(Cmd, TEXT("Index="), Index) && Index == InstanceIndex)
		{
			FParse::Token(Cmd, false); // skip over Index=#

			FString Command;
			if (FParse::Token(Cmd, Command, false))
			{
				if (TUniquePtr<IOnlineExecHandler>* ExecHandler = ExecCommands.Find(Command))
				{
					return (*ExecHandler)->Exec(World, Cmd, Ar);
				}
			}
		}
		else if (FParse::Command(&Cmd, TEXT("List")))
		{
			Ar.Logf(TEXT("%u: ServiceConfigName=[%s] InstanceName=[%s] InstanceConfigName=[%s]"),
				InstanceIndex, *GetServiceConfigName(), *GetInstanceName().ToString(), *GetInstanceConfigName().ToString());
		}
	}
	return false;
}
#endif // UE_ALLOW_EXEC_COMMANDS

void FOnlineServicesCommon::LoadCommonConfig()
{
	FOnlineServicesCommonConfig Config;
	LoadConfig(Config);

	ParallelQueue.SetMaxConcurrentOperations(Config.MaxConcurrentOperations);
}

/* UE::Online */ }
