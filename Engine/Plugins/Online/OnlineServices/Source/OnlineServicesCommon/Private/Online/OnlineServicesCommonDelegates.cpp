// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesCommonDelegates.h"
#include "Online/OnlineServicesCommon.h"

namespace UE::Online {

FOnOnlineAsyncOpCompletedParams::FOnOnlineAsyncOpCompletedParams(FOnlineServicesCommon& InOnlineServicesCommon, const TOptional<FOnlineError>& InOnlineError)
	: OnlineServicesCommon(StaticCastWeakPtr<FOnlineServicesCommon>(InOnlineServicesCommon.AsWeak()))
	, OnlineError(InOnlineError)
{
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FOnOnlineAsyncOpCompletedV2 OnOnlineAsyncOpCompletedV2;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/* UE::Online */ }
