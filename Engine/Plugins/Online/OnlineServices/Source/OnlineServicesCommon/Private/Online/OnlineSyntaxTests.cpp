// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineComponent.h"

#define TEST_ASYNC_OP_CONTINUATION_SYNTAX 1
#define TEST_CONSTRUCT_DELEGATE_SYNTAX 1
#define TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT 0
#define TEST_ASYNCOPCACHE_SYNTAX 1
#define TEST_CONFIG_SYNTAX 1
#define TEST_TOLOGSTRING_SYNTAX 1

namespace UE::Online {

namespace Test {

struct ITestInterface
{
public:
};

struct FTestInterface : ITestInterface
{
public:
	using Super = ITestInterface;
};

struct FTestOp
{
	static constexpr TCHAR Name[] = TEXT("TestOp");

	struct Params
	{
	};

	struct Result
	{
	};
};

struct FTestUserOp
{
	static constexpr TCHAR Name[] = TEXT("TestUserOp");

	struct Params
	{
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FJoinableTestOp
{
	static constexpr TCHAR Name[] = TEXT("JoinableTestOp");

	struct Params
	{
	};

	struct Result
	{
	};
};

struct FJoinableTestUserOp
{
	static constexpr TCHAR Name[] = TEXT("JoinableTestUserOp");

	struct Params
	{
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FTestMutation
{
};

FTestMutation& operator+=(FTestMutation& A, const FTestMutation& B) { return A; }
FTestMutation& operator+=(FTestMutation& A, FTestMutation&& B) { return A; }

struct FMergeableTestOp
{
	static constexpr TCHAR Name[] = TEXT("MergeableTestOp");

	struct Params
	{
		FTestMutation Mutations;
	};

	struct Result
	{
	};
};

struct FMergeableTestUserOp
{
	static constexpr TCHAR Name[] = TEXT("MergeableTestUserOp");

	struct Params
	{
		FAccountId LocalAccountId;
		FTestMutation Mutations;
	};

	struct Result
	{
	};
};

/* Test */ }

namespace Meta {

BEGIN_ONLINE_STRUCT_META(Test::FTestOp::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(Test::FTestUserOp::Params)
	ONLINE_STRUCT_FIELD(Test::FTestUserOp::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(Test::FJoinableTestOp::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(Test::FJoinableTestUserOp::Params)
	ONLINE_STRUCT_FIELD(Test::FJoinableTestUserOp::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(Test::FMergeableTestOp::Params)
	ONLINE_STRUCT_FIELD(Test::FMergeableTestOp::Params, Mutations)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(Test::FMergeableTestUserOp::Params)
	ONLINE_STRUCT_FIELD(Test::FMergeableTestUserOp::Params, Mutations),
	ONLINE_STRUCT_FIELD(Test::FMergeableTestUserOp::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

/* Meta */ }

// confirm that a couple concepts are working as expected
static_assert(TModels_V<UE::Online::Meta::CSuperDefined, Test::FTestInterface>);
static_assert(!TModels_V<UE::Online::Meta::CSuperDefined, Test::ITestInterface>);
static_assert(!TModels_V<UE::Online::CLocalAccountIdDefined, Test::FTestOp::Params>);
static_assert(TModels_V<UE::Online::CLocalAccountIdDefined, Test::FTestUserOp::Params>);

/* UE::Online */ }


#if TEST_ASYNC_OP_CONTINUATION_SYNTAX

namespace UE::Online {

struct FTestOp
{
	static constexpr TCHAR Name[] = TEXT("TestOp");
	struct Params {};
	struct Result {};
};

struct FTestOp2
{
	static constexpr TCHAR Name[] = TEXT("TestOp2");
	struct Params {};
	struct Result {};
};

inline void TestAsyncOpContinuationSyntax(FOnlineServicesCommon& Services)
{
	TSharedRef<TOnlineAsyncOp<FTestOp>> SharedOp = MakeShared<TOnlineAsyncOp<FTestOp>>(Services, FTestOp::Params());
	TOnlineAsyncOp<FTestOp>& Op = SharedOp.Get();
	TOnlineChainableAsyncOp<FTestOp, void> ChainableOp = Op;
	if (true /* condition to enable optional steps */)
	{
		ChainableOp = ChainableOp.Then([](FOnlineAsyncOp&) {})
			.Then([](FOnlineAsyncOp&) { return MakeFulfilledPromise<void>().GetFuture(); });
	}
	ChainableOp.Then([](FOnlineAsyncOp&) {})
		.Then([](FOnlineAsyncOp&) { return MakeFulfilledPromise<void>().GetFuture(); })
		.Then([](FOnlineAsyncOp&) {})
		.Then([](TOnlineAsyncOp<FTestOp>&) { return 1; })
		.Then([](TOnlineAsyncOp<FTestOp>&, int) { return MakeFulfilledPromise<FString>().GetFuture(); })
		.Then([](TOnlineAsyncOp<FTestOp>&, FString) { return FString(); })
		.Then([](TOnlineAsyncOp<FTestOp>&, const FString&) { return FString(); })
		.Then([](TOnlineAsyncOp<FTestOp>&, FString) {})
		.Then([](TOnlineAsyncOp<FTestOp>&) { return 1.0f; })
		.Then([](TOnlineAsyncOp<FTestOp>&, float) { return MakeFulfilledPromise<void>().GetFuture(); })
		.Then([](TOnlineAsyncOp<FTestOp>&) { return MakeFulfilledPromise<void>().GetFuture(); })
		.Then([](TOnlineAsyncOp<FTestOp>&, TPromise<int>&& Promise) { Promise.SetValue(1); })
		.Then([](TOnlineAsyncOp<FTestOp>&, int, TPromise<FString>&& Promise) { Promise.SetValue(FString()); })
		.Then([](TOnlineAsyncOp<FTestOp>&, const FString&, TPromise<int>&& Promise) { Promise.SetValue(1); })
		.Then([](TOnlineAsyncOp<FTestOp>&, int) { return TArray<int>{ 1, 2, 3, 4 }; })
		.ForEach([](TOnlineAsyncOp<FTestOp>&, int Value) { return Value; })
		.ForEach([](TOnlineAsyncOp<FTestOp>&, int Value, TPromise<int>&& Promise) { Promise.SetValue(Value); })
		.ForEach([](TOnlineAsyncOp<FTestOp>&, int Value) { return MakeFulfilledPromise<int>(Value).GetFuture(); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value) { return TArray<int>(Value); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value, TPromise<TArray<int>>&& Promise) { return Promise.SetValue(TArray<int>(Value)); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value) { return MakeFulfilledPromise<TArray<int>>(Value).GetFuture(); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value) { return 1; })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value) { return MakeFulfilledPromise<int>(1).GetFuture(); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value, TPromise<int>&& Promise) { Promise.SetValue(1); })
		.ForEachN(2, [](TOnlineAsyncOp<FTestOp>&, TArrayView<int> Value) { TArray<int> IntArray = { 1, 2 }; return TArray<int>(IntArray); })
		.Then([](TOnlineAsyncOp<FTestOp>&, const TArray<int>&) { return 1; })
		.Then([](TOnlineAsyncOp<FTestOp>&, int) {})
		.Then([](TOnlineAsyncOp<FTestOp>&) { return TContinuationResult<int>::Complete(1); })
		.Then([](TOnlineAsyncOp<FTestOp>&, int) { return MakeFulfilledPromise<TContinuationResult<int>>(TContinuationResult<int>::Complete(1)).GetFuture(); })
		.Then([](TOnlineAsyncOp<FTestOp>&, int) { return MakeFulfilledPromise<TContinuationResult<void>>(TContinuationResult<void>::Complete()).GetFuture(); })
		.Then([](TOnlineAsyncOp<FTestOp>&) { return TContinuationResult<void>::Complete(); })
		.Enqueue(Services.GetParallelQueue());
	Op.GetHandle();
	Op.GetWrappedHandle<FTestOp2>([](const TOnlineResult<FTestOp>& Result) { return TOnlineResult<FTestOp2>(FTestOp2::Result()); });
}

/* UE::Online */ }

#endif // TEST_ASYNC_OP_CONTINUATION_SYNTAX


#if TEST_CONSTRUCT_DELEGATE_SYNTAX

class FSPTest : public TSharedFromThis<FSPTest, ESPMode::NotThreadSafe>
{
public:
	void FN(int, const FString&) {}
	void FN2(int, const FString&, float) {}
	void CFN(int, const FString&) const {}
	void CFN2(int, const FString&, float) const {}
};

class FTSSPTest : public TSharedFromThis<FTSSPTest, ESPMode::ThreadSafe>
{
public:
	void FN(int, const FString&) {}
	void FN2(int, const FString&, float) {}
	void CFN(int, const FString&) const {}
	void CFN2(int, const FString&, float) const {}
};

class FRawTest
{
public:
	void FN(int, const FString&) {}
	void FN2(int, const FString&, float) {}
	void CFN(int, const FString&) const {}
	void CFN2(int, const FString&, float) const {}
};

#if TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class UMyObject : public UObject
{
public:
	void FN(int, const FString&) {}
	void FN2(int, const FString&, float) {}
	void CFN(int, const FString&) const {}
	void CFN2(int, const FString&, float) const {}
};
#endif // TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT

inline void FN(int, const FString&) {}
inline void FN2(int, const FString&, float) {}

class FOnlineServicesTest : public UE::Online::FOnlineServicesCommon
{
public:
	using Super = UE::Online::FOnlineServicesCommon;

	FOnlineServicesTest(FName InInstanceName, FName InInstanceConfigName) : Super(TEXT("Test"), InInstanceName, InInstanceConfigName) {}

	virtual UE::Online::EOnlineServices GetServicesProvider() const override { return UE::Online::EOnlineServices::Null; }

	virtual void RegisterComponents() override;
};

class FOnlineComponentTest : public UE::Online::TOnlineComponent<UE::Online::Test::ITestInterface>
{
public:
	using Super = UE::Online::TOnlineComponent<UE::Online::Test::ITestInterface>;

	FOnlineComponentTest(FOnlineServicesTest& InOwningSubsystem) : Super(TEXT("Test"), InOwningSubsystem) {}

	void FN(int, const FString&) {}
	void FN2(int, const FString&, float) {}
	void CFN(int, const FString&) const {}
	void CFN2(int, const FString&, float) const {}
};

void FOnlineServicesTest::RegisterComponents()
{
	Components.Register<FOnlineComponentTest>(*this);
}

inline void DelegateTest()
{
	using UE::Online::Private::ConstructDelegate;

	TSharedRef<FSPTest, ESPMode::NotThreadSafe> SP = MakeShared<FSPTest, ESPMode::NotThreadSafe>();
	TSharedRef<const FSPTest, ESPMode::NotThreadSafe> ConstSP = ConstCastSharedRef<const FSPTest>(SP);
	TSharedRef<FTSSPTest, ESPMode::ThreadSafe> TSSP = MakeShared<FTSSPTest, ESPMode::ThreadSafe>();
	TSharedRef<const FTSSPTest, ESPMode::ThreadSafe> ConstTSSP = ConstCastSharedRef<const FTSSPTest>(TSSP);
	TSharedRef<FRawTest, ESPMode::NotThreadSafe> SPRaw = MakeShared<FRawTest, ESPMode::NotThreadSafe>();
	TSharedRef<const FRawTest, ESPMode::NotThreadSafe> ConstSPRaw = ConstCastSharedRef<const FRawTest>(SPRaw);
	TSharedRef<FRawTest, ESPMode::ThreadSafe> TSSPRaw = MakeShared<FRawTest, ESPMode::ThreadSafe>();
	TSharedRef<const FRawTest, ESPMode::ThreadSafe> ConstTSSPRaw = ConstCastSharedRef<const FRawTest>(TSSPRaw);
	FRawTest* Raw = &SPRaw.Get();
	const FRawTest* ConstRaw = Raw;
#if TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
	UMyObject* UObject = NewObject<UMyObject>();
	const UMyObject* ConstUObject = UObject;
#endif // TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT

	TSharedRef<FOnlineServicesTest> ServicesTest = MakeShared<FOnlineServicesTest>(NAME_None, NAME_None);
	FOnlineComponentTest* TestComponent = ServicesTest->Get<FOnlineComponentTest>();
	const FOnlineComponentTest* ConstTestComponent = ServicesTest->Get<FOnlineComponentTest>();

	// CreateStatic
	TDelegate<void(int, const FString&)> StaticDelegate1 = ConstructDelegate<void(int, const FString&)>(FN);
	TDelegate<void(int, const FString&)> StaticDelegate2 = ConstructDelegate<void(int, const FString&)>(&FN);
	TDelegate<void(int, const FString&)> StaticDelegate3 = ConstructDelegate<void(int, const FString&)>(FN2, 1.0f);
	TDelegate<void(int, const FString&)> StaticDelegate4 = ConstructDelegate<void(int, const FString&)>(&FN2, 1.0f);
	// CreateLambda
	TDelegate<void(int, const FString&)> LambdaDelegate = ConstructDelegate<void(int, const FString&)>([](int, const FString&) {});
	// CreateUObject
#if TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
	TDelegate<void(int, const FString&)> UObjectDelegate1 = ConstructDelegate<void(int, const FString&)>(UObject, &UMyObject::FN);
	TDelegate<void(int, const FString&)> UObjectDelegate2 = ConstructDelegate<void(int, const FString&)>(UObject, &UMyObject::FN2, 1.0f);
	TDelegate<void(int, const FString&)> UObjectDelegate3 = ConstructDelegate<void(int, const FString&)>(UObject, &UMyObject::CFN);
	TDelegate<void(int, const FString&)> UObjectDelegate4 = ConstructDelegate<void(int, const FString&)>(UObject, &UMyObject::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> UObjectDelegate5 = ConstructDelegate<void(int, const FString&)>(ConstUObject, &UMyObject::CFN);
	TDelegate<void(int, const FString&)> UObjectDelegate6 = ConstructDelegate<void(int, const FString&)>(ConstUObject, &UMyObject::CFN2, 1.0f);
	// CreateUFunction
	TDelegate<void(int, const FString&)> UFunctionDelegate1 = ConstructDelegate<void(int, const FString&)>(UObject, FName(TEXT("FN")));
	TDelegate<void(int, const FString&)> UFunctionDelegate2 = ConstructDelegate<void(int, const FString&)>(UObject, FName(TEXT("FN2")), 1.0f);
#endif // TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
	// CreateThreadSafeSP
	TDelegate<void(int, const FString&)> TSSPDelegate1 = ConstructDelegate<void(int, const FString&)>(TSSP, &FTSSPTest::FN);
	TDelegate<void(int, const FString&)> TSSPDelegate2 = ConstructDelegate<void(int, const FString&)>(TSSP, &FTSSPTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPDelegate3 = ConstructDelegate<void(int, const FString&)>(&TSSP.Get(), &FTSSPTest::FN);
	TDelegate<void(int, const FString&)> TSSPDelegate4 = ConstructDelegate<void(int, const FString&)>(&TSSP.Get(), &FTSSPTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPDelegate5 = ConstructDelegate<void(int, const FString&)>(TSSP, &FTSSPTest::CFN);
	TDelegate<void(int, const FString&)> TSSPDelegate6 = ConstructDelegate<void(int, const FString&)>(TSSP, &FTSSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPDelegate7 = ConstructDelegate<void(int, const FString&)>(&TSSP.Get(), &FTSSPTest::CFN);
	TDelegate<void(int, const FString&)> TSSPDelegate8 = ConstructDelegate<void(int, const FString&)>(&TSSP.Get(), &FTSSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPDelegate9 = ConstructDelegate<void(int, const FString&)>(ConstTSSP, &FTSSPTest::CFN);
	TDelegate<void(int, const FString&)> TSSPDelegate10 = ConstructDelegate<void(int, const FString&)>(ConstTSSP, &FTSSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPDelegate11 = ConstructDelegate<void(int, const FString&)>(&ConstTSSP.Get(), &FTSSPTest::CFN);
	TDelegate<void(int, const FString&)> TSSPDelegate12 = ConstructDelegate<void(int, const FString&)>(&ConstTSSP.Get(), &FTSSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPRawDelegate1 = ConstructDelegate<void(int, const FString&)>(TSSPRaw, &FRawTest::FN);
	TDelegate<void(int, const FString&)> TSSPRawDelegate2 = ConstructDelegate<void(int, const FString&)>(TSSPRaw, &FRawTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPRawDelegate3 = ConstructDelegate<void(int, const FString&)>(TSSPRaw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> TSSPRawDelegate4 = ConstructDelegate<void(int, const FString&)>(TSSPRaw, &FRawTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TSSPRawDelegate5 = ConstructDelegate<void(int, const FString&)>(ConstTSSPRaw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> TSSPRawDelegate6 = ConstructDelegate<void(int, const FString&)>(ConstTSSPRaw, &FRawTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TComponentDelegate1 = ConstructDelegate<void(int, const FString&)>(TestComponent, &FOnlineComponentTest::FN);
	TDelegate<void(int, const FString&)> TComponentDelegate2 = ConstructDelegate<void(int, const FString&)>(TestComponent, &FOnlineComponentTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> TComponentDelegate3 = ConstructDelegate<void(int, const FString&)>(TestComponent, &FOnlineComponentTest::CFN);
	TDelegate<void(int, const FString&)> TComponentDelegate4 = ConstructDelegate<void(int, const FString&)>(TestComponent, &FOnlineComponentTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> TComponentDelegate5 = ConstructDelegate<void(int, const FString&)>(ConstTestComponent, &FOnlineComponentTest::CFN);
	TDelegate<void(int, const FString&)> TComponentDelegate6 = ConstructDelegate<void(int, const FString&)>(ConstTestComponent, &FOnlineComponentTest::CFN2, 1.0f);
	// CreateSP
	TDelegate<void(int, const FString&)> SPDelegate1 = ConstructDelegate<void(int, const FString&)>(SP, &FSPTest::FN);
	TDelegate<void(int, const FString&)> SPDelegate2 = ConstructDelegate<void(int, const FString&)>(SP, &FSPTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> SPDelegate3 = ConstructDelegate<void(int, const FString&)>(&SP.Get(), &FSPTest::FN);
	TDelegate<void(int, const FString&)> SPDelegate4 = ConstructDelegate<void(int, const FString&)>(&SP.Get(), &FSPTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> SPDelegate5 = ConstructDelegate<void(int, const FString&)>(SP, &FSPTest::CFN);
	TDelegate<void(int, const FString&)> SPDelegate6 = ConstructDelegate<void(int, const FString&)>(SP, &FSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> SPDelegate7 = ConstructDelegate<void(int, const FString&)>(&SP.Get(), &FSPTest::CFN);
	TDelegate<void(int, const FString&)> SPDelegate8 = ConstructDelegate<void(int, const FString&)>(&SP.Get(), &FSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> SPDelegate9 = ConstructDelegate<void(int, const FString&)>(ConstSP, &FSPTest::CFN);
	TDelegate<void(int, const FString&)> SPDelegate10 = ConstructDelegate<void(int, const FString&)>(ConstSP, &FSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> SPDelegate11 = ConstructDelegate<void(int, const FString&)>(&ConstSP.Get(), &FSPTest::CFN);
	TDelegate<void(int, const FString&)> SPDelegate12 = ConstructDelegate<void(int, const FString&)>(&ConstSP.Get(), &FSPTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> SPRawDelegate1 = ConstructDelegate<void(int, const FString&)>(SPRaw, &FRawTest::FN);
	TDelegate<void(int, const FString&)> SPRawDelegate2 = ConstructDelegate<void(int, const FString&)>(SPRaw, &FRawTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> SPRawDelegate3 = ConstructDelegate<void(int, const FString&)>(SPRaw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> SPRawDelegate4 = ConstructDelegate<void(int, const FString&)>(SPRaw, &FRawTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> SPRawDelegate5 = ConstructDelegate<void(int, const FString&)>(ConstSPRaw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> SPRawDelegate6 = ConstructDelegate<void(int, const FString&)>(ConstSPRaw, &FRawTest::CFN2, 1.0f);
	// CreateRaw
	TDelegate<void(int, const FString&)> RawDelegate1 = ConstructDelegate<void(int, const FString&)>(Raw, &FRawTest::FN);
	TDelegate<void(int, const FString&)> RawDelegate2 = ConstructDelegate<void(int, const FString&)>(Raw, &FRawTest::FN2, 1.0f);
	TDelegate<void(int, const FString&)> RawDelegate3 = ConstructDelegate<void(int, const FString&)>(Raw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> RawDelegate4 = ConstructDelegate<void(int, const FString&)>(Raw, &FRawTest::CFN2, 1.0f);
	TDelegate<void(int, const FString&)> RawDelegate5 = ConstructDelegate<void(int, const FString&)>(ConstRaw, &FRawTest::CFN);
	TDelegate<void(int, const FString&)> RawDelegate6 = ConstructDelegate<void(int, const FString&)>(ConstRaw, &FRawTest::CFN2, 1.0f);
	// CreateWeakLambda
#if TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
	TDelegate<void(int, const FString&)> WeakUObjectDelegate1 = ConstructDelegate<void(int, const FString&)>(UObject, [](int, const FString&) {});
//	TDelegate<void(int, const FString&)> WeakUObjectDelegate2 = ConstructDelegate<void(int, const FString&)>(ConstUObject, [](int, const FString&) {}); // CreateWeakObject doesn't support const UObject*
#endif // TEST_CONSTRUCT_DELEGATE_SYNTAX_UOBJECT
	// lambda with TWeakPtr
	TDelegate<void(int, const FString&)> WeakTSSPDelegate1 = ConstructDelegate<void(int, const FString&)>(TSSP, [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakTSSPDelegate2 = ConstructDelegate<void(int, const FString&)>(&TSSP.Get(), [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakSPDelegate1 = ConstructDelegate<void(int, const FString&)>(SP, [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakSPDelegate2 = ConstructDelegate<void(int, const FString&)>(&SP.Get(), [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakTSSPRawDelegate1 = ConstructDelegate<void(int, const FString&)>(TSSPRaw, [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakSPRawDelegate1 = ConstructDelegate<void(int, const FString&)>(SPRaw, [](int, const FString&) {});
	TDelegate<void(int, const FString&)> WeakTComponentDelegate = ConstructDelegate<void(int, const FString&)>(TestComponent, [](int, const FString&) {});
	// lambda with raw pointer: Error: When using non pointer to member functions, the first parameter can only be a UObject*, TSharedRef, or pointer to a class that derives from TSharedFromThis
//	TDelegate<void(int, const FString&)> WeakRawDelegate = ConstructDelegate<void(int, const FString&)>(Raw, [](int, const FString&) {});

	delete Raw;
}

#endif // TEST_CONSTRUCT_DELEGATE_SYNTAX


#if TEST_ASYNCOPCACHE_SYNTAX

namespace UE::Online {

namespace Test {

class FTestInterfaceCommon : public TOnlineComponent<ITestInterface>
{
public:
	using Super = ITestInterface;

	FTestInterfaceCommon(FOnlineServicesCommon& InServices)
		: TOnlineComponent(TEXT("Test"), InServices)
	{

	}
};

/* Test */ }

inline void TestAsyncOpCacheSyntax(FOnlineServicesCommon& Services, Test::FTestInterfaceCommon& Interface)
{
	Test::FTestOp::Params Params;
	TOnlineAsyncOpRef<Test::FTestOp> Op = Interface.GetOp<Test::FTestOp>(MoveTemp(Params));
	TOnlineAsyncOpRef<Test::FTestOp> Op2 = Services.GetOp<Test::FTestOp>(MoveTemp(Params));

	Test::FTestUserOp::Params UserParams;
	TOnlineAsyncOpRef<Test::FTestUserOp> UserOp = Interface.GetOp<Test::FTestUserOp>(MoveTemp(UserParams));
	TOnlineAsyncOpRef<Test::FTestUserOp> UserOp2 = Services.GetOp<Test::FTestUserOp>(MoveTemp(UserParams));

	Test::FJoinableTestOp::Params JoinableParams;
	TOnlineAsyncOpRef<Test::FJoinableTestOp> JoinableOp = Interface.GetJoinableOp<Test::FJoinableTestOp>(MoveTemp(JoinableParams));
	TOnlineAsyncOpRef<Test::FJoinableTestOp> JoinableOp2 = Services.GetJoinableOp<Test::FJoinableTestOp>(MoveTemp(JoinableParams));

	Test::FJoinableTestUserOp::Params JoinableUserParams;
	TOnlineAsyncOpRef<Test::FJoinableTestUserOp> JoinableUserOp = Interface.GetJoinableOp<Test::FJoinableTestUserOp>(MoveTemp(JoinableUserParams));
	TOnlineAsyncOpRef<Test::FJoinableTestUserOp> JoinableUserOp2 = Services.GetJoinableOp<Test::FJoinableTestUserOp>(MoveTemp(JoinableUserParams));

	Test::FMergeableTestOp::Params MergeableParams;
	TOnlineAsyncOpRef<Test::FMergeableTestOp> MergeableOp = Interface.GetMergeableOp<Test::FMergeableTestOp>(MoveTemp(MergeableParams));
	TOnlineAsyncOpRef<Test::FMergeableTestOp> MergeableOp2 = Services.GetMergeableOp<Test::FMergeableTestOp>(MoveTemp(MergeableParams));

	Test::FMergeableTestUserOp::Params MergeableUserParams;
	TOnlineAsyncOpRef<Test::FMergeableTestUserOp> MergeableUserOp = Interface.GetMergeableOp<Test::FMergeableTestUserOp>(MoveTemp(MergeableUserParams));
	TOnlineAsyncOpRef<Test::FMergeableTestUserOp> MergeableUserOp2 = Services.GetMergeableOp<Test::FMergeableTestUserOp>(MoveTemp(MergeableUserParams));
}

/* UE::Online */ }

#endif // TEST_ASYNCOPCACHE_SYNTAX

#if TEST_CONFIG_SYNTAX

enum class EConfigTestEnum
{
	A,
	B
};
void LexFromString(EConfigTestEnum& OutEnum, const TCHAR* InStr)
{
	if (FCString::Strcmp(InStr, TEXT("A")) == 0)
	{
		OutEnum = EConfigTestEnum::A;
	}
	else if (FCString::Strcmp(InStr, TEXT("B")) == 0)
	{
		OutEnum = EConfigTestEnum::B;
	}
}

struct FTestConfigInnerInner
{
	float Float;
};

struct FTestConfigInner
{
	float Float;
	FTestConfigInnerInner Object;
	TArray<FTestConfigInnerInner> ObjectArray;
	EConfigTestEnum Enum;
	TArray<EConfigTestEnum> EnumArray;
	TArray<FString> StringArray;
};

struct FTestConfig
{
	float Float;
	int Int;
	FString String;
	TArray<int> IntArray;
	FTestConfigInner Object;
	TArray<FTestConfigInner> ObjectArray;
	EConfigTestEnum Enum;
	TArray<EConfigTestEnum> EnumArray;
};

namespace UE::Online::Meta {

BEGIN_ONLINE_STRUCT_META(FTestConfigInnerInner)
	ONLINE_STRUCT_FIELD(FTestConfigInnerInner, Float)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTestConfigInner)
	ONLINE_STRUCT_FIELD(FTestConfigInner, Float),
	ONLINE_STRUCT_FIELD(FTestConfigInner, Object),
	ONLINE_STRUCT_FIELD(FTestConfigInner, ObjectArray),
	ONLINE_STRUCT_FIELD(FTestConfigInner, Enum),
	ONLINE_STRUCT_FIELD(FTestConfigInner, EnumArray),
	ONLINE_STRUCT_FIELD(FTestConfigInner, StringArray)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTestConfig)
	ONLINE_STRUCT_FIELD(FTestConfig, Float),
	ONLINE_STRUCT_FIELD(FTestConfig, Int),
	ONLINE_STRUCT_FIELD(FTestConfig, String),
	ONLINE_STRUCT_FIELD(FTestConfig, IntArray),
	ONLINE_STRUCT_FIELD(FTestConfig, Object),
	ONLINE_STRUCT_FIELD(FTestConfig, ObjectArray),
	ONLINE_STRUCT_FIELD(FTestConfig, Enum),
	ONLINE_STRUCT_FIELD(FTestConfig, EnumArray)
END_ONLINE_STRUCT_META()

/* UE::Online::Meta*/ }

void TestLoadConfigSyntax(UE::Online::IOnlineConfigProvider& ConfigProvider)
{
	FTestConfig TestConfig;
	// Test loading from a section (each field maps to a key in that section)
	UE::Online::LoadConfig(ConfigProvider, TEXT("Test"), TestConfig);
	// Test loading from a specific key in a section (the config value is in object syntax, each field maps to a key in that object)
	UE::Online::LoadConfig(ConfigProvider, TEXT("Test"), TEXT("Test"), TestConfig);
}

#endif // TEST_CONFIG_SYNTAX

#if TEST_TOLOGSTRING_SYNTAX

#include "Online/OnlineUtils.h"

namespace {

void TestToLogStringSyntax()
{
	using UE::Online::ToLogString;

	ToLogString(FString());
	ToLogString(UE::Online::FAccountInfo());
	ToLogString(UE::Online::FExternalAuthToken());
	ToLogString(UE::Online::FExternalServerAuthTicket());
	ToLogString(UE::Online::FVerifiedAuthSession());
	ToLogString(UE::Online::FVerifiedAuthTicket());
	ToLogString(UE::Online::FAuthLogin::Params());
	ToLogString(UE::Online::FAuthLogin::Result({MakeShared<UE::Online::FAccountInfo>()}));
	ToLogString(UE::Online::FAuthLogout::Params());
	ToLogString(UE::Online::FAuthLogout::Result());
	ToLogString(UE::Online::FAuthModifyAccountAttributes::Params());
	ToLogString(UE::Online::FAuthModifyAccountAttributes::Result());
	ToLogString(UE::Online::FAuthQueryExternalServerAuthTicket::Params());
	ToLogString(UE::Online::FAuthQueryExternalServerAuthTicket::Result());
	ToLogString(UE::Online::FAuthQueryExternalAuthToken::Params());
	ToLogString(UE::Online::FAuthQueryExternalAuthToken::Result());
	ToLogString(UE::Online::FAuthQueryVerifiedAuthTicket::Params());
	ToLogString(UE::Online::FAuthQueryVerifiedAuthTicket::Result());
	ToLogString(UE::Online::FAuthCancelVerifiedAuthTicket::Params());
	ToLogString(UE::Online::FAuthCancelVerifiedAuthTicket::Result());
	ToLogString(UE::Online::FAuthBeginVerifiedAuthSession::Params());
	ToLogString(UE::Online::FAuthBeginVerifiedAuthSession::Result());
	ToLogString(UE::Online::FAuthEndVerifiedAuthSession::Params());
	ToLogString(UE::Online::FAuthEndVerifiedAuthSession::Result());
	ToLogString(UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId::Params());
	ToLogString(UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId::Result({MakeShared<UE::Online::FAccountInfo>()}));
	ToLogString(UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Params());
	ToLogString(UE::Online::FAuthGetLocalOnlineUserByPlatformUserId::Result({MakeShared<UE::Online::FAccountInfo>()}));
	ToLogString(UE::Online::FAuthGetAllLocalOnlineUsers::Params());
	ToLogString(UE::Online::FAuthGetAllLocalOnlineUsers::Result());
	ToLogString(UE::Online::FAuthLoginStatusChanged({MakeShared<UE::Online::FAccountInfo>()}));
	ToLogString(UE::Online::FAuthPendingAuthExpiration({MakeShared<UE::Online::FAccountInfo>()}));
	ToLogString(UE::Online::FAuthAccountAttributesChanged({MakeShared<UE::Online::FAccountInfo>()}));
}

/* unnamed namespace */ }


#endif // TEST_TOLOGSTRING_SYNTAX
