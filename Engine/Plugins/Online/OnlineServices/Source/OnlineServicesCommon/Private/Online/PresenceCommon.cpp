// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PresenceCommon.h"
#include "Online/OnlineUtils.h"

namespace UE::Online {

FPresenceCommon::FPresenceCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Presence"), InServices)
	, Services(InServices)
{
}

void FPresenceCommon::RegisterCommands()
{
	RegisterCommand(&FPresenceCommon::QueryPresence);
	RegisterCommand(&FPresenceCommon::BatchQueryPresence);
	RegisterCommand(&FPresenceCommon::GetCachedPresence);
	RegisterCommand(&FPresenceCommon::UpdatePresence);
	RegisterCommand(&FPresenceCommon::PartialUpdatePresence);
}

void FPresenceCommon::UpdateConfig()
{
	TOnlineComponent::UpdateConfig();
}

TOnlineAsyncOpHandle<FQueryPresence> FPresenceCommon::QueryPresence(FQueryPresence::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryPresence> Operation = GetOp<FQueryPresence>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}
/*
      The common implementation of BatchQueryPresence sends out a bunch of asynchronous QueryPresence calls and combines them. 
*/ 
struct FBatchQueryPresenceHelperResult
{
public:
	FBatchQueryPresence::Result Result;
	FOnlineError ErrorState;
};

class FBatchQueryPresenceHelper
{
public:
	FBatchQueryPresenceHelper(TOnlineAsyncOpPtr<FBatchQueryPresence> InOp, TSharedPtr<IPresence> InPresence)
		: Op(InOp)
		, Presence(InPresence)
		, ErrorState(Errors::Success())
	{
		NumPresencesRemaining = InOp->GetParams().TargetAccountIds.Num();
	}

	TFuture<FBatchQueryPresenceHelperResult> GetPromise()
	{
		TOnlineAsyncOpPtr<FBatchQueryPresence> OpPtr = Op.Pin();
		if (OpPtr.IsValid())
		{
			for(FAccountId TargetAccountId : OpPtr->GetParams().TargetAccountIds)
			{
				FQueryPresence::Params Params;
				Params.LocalAccountId = OpPtr->GetParams().LocalAccountId;
				Params.TargetAccountId = TargetAccountId;
				Params.bListenToChanges = OpPtr->GetParams().bListenToChanges;

				Presence->QueryPresence(MoveTemp(Params)).OnComplete([this](const TOnlineResult<FQueryPresence> NewOp) mutable
				{
					if (NewOp.IsOk())
					{
						// push results
						Result.Presences.Add(NewOp.GetOkValue().Presence);
					}
					else
					{
						// error
						// we're only reporting the most recent one, more verbose logging on each individual error can be found in the logs
						ErrorState = NewOp.GetErrorValue();
					}


					// final calculations
					if (--NumPresencesRemaining <= 0)
					{
						Promise.SetValue(FBatchQueryPresenceHelperResult{ MoveTemp(Result), MoveTemp(ErrorState) });
					}
				});
			}
		}

		return Promise.GetFuture();
	}

	FBatchQueryPresence::Result Result;
	// Need to use weak ptr here otherwise there will be circular reference with Op->Steps->Lambda->Helper->Op
	TWeakPtr<TOnlineAsyncOp<FBatchQueryPresence>> Op;
	TSharedPtr<IPresence> Presence;
	FOnlineError ErrorState;
	int NumPresencesRemaining = 0;
	TPromise<FBatchQueryPresenceHelperResult> Promise;
};

// Base implementation for the batch query function makes multiple async calls to query and returns them as a single output
TOnlineAsyncOpHandle<FBatchQueryPresence> FPresenceCommon::BatchQueryPresence(FBatchQueryPresence::Params&& Params)
{
	TOnlineAsyncOpRef<FBatchQueryPresence> Op = GetOp<FBatchQueryPresence>(MoveTemp(Params));
	TSharedRef<FBatchQueryPresenceHelper> Helper = MakeShared<FBatchQueryPresenceHelper>(Op.ToSharedPtr(), this->AsShared());

	Op->Then([this, Helper](TOnlineAsyncOp<FBatchQueryPresence>& InAsyncOp) mutable
	{
		return Helper->GetPromise();
	})
	.Then([this, Helper](TOnlineAsyncOp<FBatchQueryPresence>& InAsyncOp, FBatchQueryPresenceHelperResult Result) mutable
	{
		if (Result.ErrorState == Errors::Success())
		{
			InAsyncOp.SetResult(MoveTemp(Result.Result));
		}
		else 
		{
			InAsyncOp.SetError(MoveTemp(Result.ErrorState));
		}
	})
	.Enqueue(Services.GetParallelQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetCachedPresence> FPresenceCommon::GetCachedPresence(FGetCachedPresence::Params&& Params)
{
	return TOnlineResult<FGetCachedPresence>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FUpdatePresence> FPresenceCommon::UpdatePresence(FUpdatePresence::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdatePresence> Operation = GetOp<FUpdatePresence>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FPartialUpdatePresence> FPresenceCommon::PartialUpdatePresence(FPartialUpdatePresence::Params&& Params)
{
	TOnlineResult<FGetCachedPresence> CachedPresence = GetCachedPresence({Params.LocalAccountId, Params.LocalAccountId});
	if (CachedPresence.IsOk())
	{
		TSharedRef<const FUserPresence> ExistingPresence = CachedPresence.GetOkValue().Presence;
		TSharedRef<FUserPresence> NewPresence = ApplyPresenceMutations(*ExistingPresence, Params.Mutations);

		TOnlineAsyncOpRef<FPartialUpdatePresence> Op = GetOp<FPartialUpdatePresence>(MoveTemp(Params));
		TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresenceResult = UpdatePresence({Op->GetParams().LocalAccountId, NewPresence});
		UpdatePresenceResult.OnComplete([this, Op](const TOnlineResult<FUpdatePresence> Result)
		{
			if (Result.IsOk())
			{
				Op->SetResult(FPartialUpdatePresence::Result());
			}
			else
			{
				FOnlineError Error = Result.GetErrorValue();
				Op->SetError(MoveTemp(Error));
			}
		});

		return Op->GetHandle();
	}

	TOnlineAsyncOpRef<FPartialUpdatePresence> Operation = GetOp<FPartialUpdatePresence>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineEvent<void(const FPresenceUpdated&)> FPresenceCommon::OnPresenceUpdated()
{
	return OnPresenceUpdatedEvent;
}

/* UE::Online */ }
