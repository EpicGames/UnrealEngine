// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PrivilegesCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FPrivilegesCommon::FPrivilegesCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Privileges"), InServices)
{
}

void FPrivilegesCommon::RegisterCommands()
{
	RegisterCommand(&FPrivilegesCommon::QueryUserPrivilege);
}

TOnlineAsyncOpHandle<FQueryUserPrivilege> FPrivilegesCommon::QueryUserPrivilege(FQueryUserPrivilege::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryUserPrivilege> Operation = GetOp<FQueryUserPrivilege>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

/* UE::Online */}
