// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {
	/** FSessionCommon */

	FSessionCommon& FSessionCommon::operator+=(const FSessionUpdate& SessionUpdate)
	{
		if (SessionUpdate.OwnerAccountId.IsSet())
		{
			OwnerAccountId = SessionUpdate.OwnerAccountId.GetValue();
		}

		if (SessionUpdate.SessionSettingsChanges.IsSet())
		{
			SessionSettings += SessionUpdate.SessionSettingsChanges.GetValue();
		}

		// Session Members

		for (const FAccountId& Key : SessionUpdate.RemovedSessionMembers)
		{
			SessionMembers.Remove(Key);
		}

		SessionMembers.Append(SessionUpdate.AddedSessionMembers);

		return *this;
	}

	FString FSessionCommon::ToLogString() const
	{
		return FString::Printf(TEXT("ISession: SessionId [%s]"), *UE::Online::ToLogString(SessionInfo.SessionId));
	}

	void FSessionCommon::DumpState() const
	{
		UE_LOGF(LogOnlineServices, Log, "[ISession]");
		DumpMemberData();
		DumpSessionInfo();
		DumpSessionSettings();
	}

	void FSessionCommon::DumpMemberData() const
	{
		UE_LOGF(LogOnlineServices, Log, "OwnerAccountId [%ls]", *UE::Online::ToLogString(OwnerAccountId));
		UE_LOGF(LogOnlineServices, Log, "SessionMemberIdsSet:");

		for (const FAccountId& SessionMemberId : SessionMembers)
		{
			UE_LOGF(LogOnlineServices, Log, "	[%ls]", *UE::Online::ToLogString(SessionMemberId));
		}
	}

	void FSessionCommon::DumpSessionInfo() const
	{
		UE_LOGF(LogOnlineServices, Log, "SessionInfo:");
		UE_LOGF(LogOnlineServices, Log, "	SessionId [%ls]", *UE::Online::ToLogString(SessionInfo.SessionId));
		UE_LOGF(LogOnlineServices, Log, "	bAllowSanctionedPlayers [%d]", SessionInfo.bAllowSanctionedPlayers);
		UE_LOGF(LogOnlineServices, Log, "	bAntiCheatProtected [%d]", SessionInfo.bAntiCheatProtected);
		UE_LOGF(LogOnlineServices, Log, "	bIsDedicatedServerSession [%d]", SessionInfo.bIsDedicatedServerSession);
		UE_LOGF(LogOnlineServices, Log, "	bIsLANSession [%d]", SessionInfo.bIsLANSession);
		UE_LOGF(LogOnlineServices, Log, "	SessionIdOverride [%ls]", *SessionInfo.SessionIdOverride);
	}

	void FSessionCommon::DumpSessionSettings() const
	{
		UE_LOGF(LogOnlineServices, Log, "SessionSettings:");
		UE_LOGF(LogOnlineServices, Log, "	bAllowNewMembers [%d]", SessionSettings.bAllowNewMembers);
		UE_LOGF(LogOnlineServices, Log, "	JoinPolicy [%ls]", LexToString(SessionSettings.JoinPolicy));
		UE_LOGF(LogOnlineServices, Log, "	NumMaxConnections [%d]", SessionSettings.NumMaxConnections);
		UE_LOGF(LogOnlineServices, Log, "	SchemaName [%ls]", *SessionSettings.SchemaName.ToString());
		UE_LOGF(LogOnlineServices, Log, "	CustomSettings:");

		for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : SessionSettings.CustomSettings)
		{
			UE_LOGF(LogOnlineServices, Log, "		Key: [%ls]", *Entry.Key.ToString());
			UE_LOGF(LogOnlineServices, Log, "		Value:	Visibility [%ls]", LexToString(Entry.Value.Visibility));
			UE_LOGF(LogOnlineServices, Log, "				ID [%d]", Entry.Value.ID);
			UE_LOGF(LogOnlineServices, Log, "				Data [%ls]", *LexToString(Entry.Value.Data));
		}
	}

	/** FSessionInvitecommon */

	FSessionInviteCommon::FSessionInviteCommon(const FAccountId& InRecipientId, const FAccountId& InSenderId, const FSessionInviteId& InInviteId, const FOnlineSessionId& InSessionId)
		: RecipientId(InRecipientId)
		, SenderId(InSenderId)
		, InviteId(InInviteId)
		, SessionId(InSessionId)
	{

	}

	FString FSessionInviteCommon::ToLogString() const
	{
		return FString::Printf(TEXT("ISessionInvite: RecipientId [%s], SenderId [%s], InviteId [%s], SessionId [%s]"),
			*UE::Online::ToLogString(RecipientId),*UE::Online::ToLogString(SenderId), *UE::Online::ToLogString(InviteId), *UE::Online::ToLogString(SessionId));
	}

	/** FSessionsCommon */

	FSessionsCommon::FSessionsCommon(FOnlineServicesCommon& InServices)
		: TOnlineComponent(TEXT("Sessions"), InServices)
	{
	}

	void FSessionsCommon::Initialize()
	{
		TOnlineComponent<ISessions>::Initialize();
	}

	void FSessionsCommon::RegisterCommands()
	{
		TOnlineComponent<ISessions>::RegisterCommands();

		RegisterCommand(&FSessionsCommon::GetAllSessions);
		RegisterCommand(&FSessionsCommon::GetSessionByName);
		RegisterCommand(&FSessionsCommon::GetSessionName);
		RegisterCommand(&FSessionsCommon::GetSessionById);
		RegisterCommand(&FSessionsCommon::GetPresenceSession);
		RegisterCommand(&FSessionsCommon::IsPresenceSession);
		RegisterCommand(&FSessionsCommon::SetPresenceSession);
		RegisterCommand(&FSessionsCommon::ClearPresenceSession);
		RegisterCommand(&FSessionsCommon::CreateSession);
		RegisterCommand(&FSessionsCommon::UpdateSessionSettings);
		RegisterCommand(&FSessionsCommon::LeaveSession);
		RegisterCommand(&FSessionsCommon::FindSessions);
		RegisterCommand(&FSessionsCommon::StartMatchmaking);
		RegisterCommand(&FSessionsCommon::JoinSession);
		RegisterCommand(&FSessionsCommon::AddSessionMember);
		RegisterCommand(&FSessionsCommon::RemoveSessionMember);
		RegisterCommand(&FSessionsCommon::SendSessionInvite);
		RegisterCommand(&FSessionsCommon::GetSessionInviteById);
		RegisterCommand(&FSessionsCommon::GetAllSessionInvites);
		RegisterCommand(&FSessionsCommon::RejectSessionInvite);
	}

	// Synchronous methods

#define CHECK_PARAMS_ID_HANDLE(IdHandle, MethodName) \
	if (!IdHandle.IsValid()) \
	{ \
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] Value of [%ls] invalid.", UTF8_TO_TCHAR(__FUNCTION__), UTF8_TO_TCHAR(#IdHandle)); \
		return TOnlineResult<MethodName>(Errors::InvalidParams()); \
	} \

#define CHECK_PARAMS_FNAME(Name, MethodName) \
	if (Name.IsNone()) \
	{ \
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] Value of [%ls] invalid.", UTF8_TO_TCHAR(__FUNCTION__), UTF8_TO_TCHAR(#Name)); \
		return TOnlineResult<MethodName>(Errors::InvalidParams()); \
	} \

	TOnlineResult<FGetAllSessions> FSessionsCommon::GetAllSessions(FGetAllSessions::Params&& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FGetAllSessions)

		TArray<TSharedRef<const ISession>> Sessions;

		if (const TArray<FName>* UserSessions = NamedSessionUserMap.Find(Params.LocalAccountId))
		{
			for (const FName& SessionName : *UserSessions)
			{
				const FOnlineSessionId& SessionId = LocalSessionsByName.FindChecked(SessionName);
				const TSharedRef<FSessionCommon>& Session = AllSessionsById.FindChecked(SessionId);

				Sessions.Add(Session);
			}
		}

		return TOnlineResult<FGetAllSessions>({ MoveTemp(Sessions) });
	}

	TOnlineResult<FGetSessionByName> FSessionsCommon::GetSessionByName(FGetSessionByName::Params&& Params) const
	{
		TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.LocalName });
		if (GetMutableSessionByNameResult.IsOk())
		{
			return TOnlineResult<FGetSessionByName>({ GetMutableSessionByNameResult.GetOkValue().Session });
		}
		else
		{
			return TOnlineResult<FGetSessionByName>(GetMutableSessionByNameResult.GetErrorValue());
		}	
	}

	TOnlineResult<FGetSessionName> FSessionsCommon::GetSessionName(FGetSessionName::Params&& Params) const
	{
		if (const FName* SessionName = LocalSessionsByName.FindKey(Params.SessionId))
		{
			return TOnlineResult<FGetSessionName>({ *SessionName });
		}
		else
		{
			return TOnlineResult<FGetSessionName>(Errors::NotFound());
		}
	}
	

	TOnlineResult<FGetSessionById> FSessionsCommon::GetSessionById(FGetSessionById::Params&& Params) const
	{
		TOnlineResult<FGetMutableSessionById> GetMutableSessionByIdResult = GetMutableSessionById({ Params.SessionId });
		if (GetMutableSessionByIdResult.IsOk())
		{
			return TOnlineResult<FGetSessionById>({ GetMutableSessionByIdResult.GetOkValue().Session });
		}
		else
		{
			return TOnlineResult<FGetSessionById>(GetMutableSessionByIdResult.GetErrorValue());
		}
	}

	TOnlineResult<FGetPresenceSession> FSessionsCommon::GetPresenceSession(FGetPresenceSession::Params&& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FGetPresenceSession);

		if (const FOnlineSessionId* PresenceSessionId = PresenceSessionsUserMap.Find(Params.LocalAccountId))
		{
			return TOnlineResult<FGetPresenceSession>({ AllSessionsById.FindChecked(*PresenceSessionId) });
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No data found for user [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
			return TOnlineResult<FGetPresenceSession>({ Errors::InvalidState() });
		}
	}

	TOnlineResult<FIsPresenceSession> FSessionsCommon::IsPresenceSession(FIsPresenceSession::Params&& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FIsPresenceSession)
		CHECK_PARAMS_ID_HANDLE(Params.SessionId, FIsPresenceSession);

		if (const FOnlineSessionId* PresenceSessionId = PresenceSessionsUserMap.Find(Params.LocalAccountId))
		{
			return TOnlineResult<FIsPresenceSession>(FIsPresenceSession::Result{ Params.SessionId == (*PresenceSessionId) });
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No data found for user [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
			return TOnlineResult<FIsPresenceSession>({ Errors::InvalidState() });
		}
	}

	TOnlineResult<FSetPresenceSession> FSessionsCommon::SetPresenceSession(FSetPresenceSession::Params&& Params)
	{
		return SetPresenceSessionImpl(MoveTemp(Params));
	}

	TOnlineResult<FClearPresenceSession> FSessionsCommon::ClearPresenceSession(FClearPresenceSession::Params&& Params)
	{
		return ClearPresenceSessionImpl(MoveTemp(Params));
	}

	TOnlineResult<FSetPresenceSession> FSessionsCommon::SetPresenceSessionImpl(FSetPresenceSession::Params&& Params)
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FSetPresenceSession)
		CHECK_PARAMS_ID_HANDLE(Params.SessionId, FSetPresenceSession);

		FOnlineSessionId& PresenceSessionId = PresenceSessionsUserMap.FindOrAdd(Params.LocalAccountId);
		PresenceSessionId = Params.SessionId;

		return TOnlineResult<FSetPresenceSession>(FSetPresenceSession::Result{ });
	}

	TOnlineResult<FClearPresenceSession> FSessionsCommon::ClearPresenceSessionImpl(FClearPresenceSession::Params&& Params)
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FClearPresenceSession);

		PresenceSessionsUserMap.Remove(Params.LocalAccountId);

		return TOnlineResult<FClearPresenceSession>(FClearPresenceSession::Result{ });
	}

	TOnlineResult<FGetMutableSessionByName> FSessionsCommon::GetMutableSessionByName(FGetMutableSessionByName::Params&& Params) const
	{
		CHECK_PARAMS_FNAME(Params.LocalName, FGetMutableSessionByName)

		if (const FOnlineSessionId* SessionId = LocalSessionsByName.Find(Params.LocalName))
		{
			check(AllSessionsById.Contains(*SessionId));

			return TOnlineResult<FGetMutableSessionByName>({ AllSessionsById.FindChecked(*SessionId) });
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No session found for name [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *Params.LocalName.ToString());
			return TOnlineResult<FGetMutableSessionByName>(Errors::NotFound());
		}
	}

	TOnlineResult<FGetMutableSessionById> FSessionsCommon::GetMutableSessionById(FGetMutableSessionById::Params&& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.SessionId, FGetMutableSessionById)
		if (const TSharedRef<FSessionCommon>* FoundSession = AllSessionsById.Find(Params.SessionId))
		{
			return TOnlineResult<FGetMutableSessionById>({ *FoundSession });
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No session found for id [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
			return TOnlineResult<FGetMutableSessionById>(Errors::NotFound());
		}
	}

	TOnlineResult<FGetSessionInviteById> FSessionsCommon::GetSessionInviteById(FGetSessionInviteById::Params&& Params)
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FGetSessionInviteById)
		CHECK_PARAMS_ID_HANDLE(Params.SessionInviteId, FGetSessionInviteById)

		if (const TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>* UserMap = SessionInvitesUserMap.Find(Params.LocalAccountId))
		{
			if (const TSharedRef<FSessionInviteCommon>* SessionInvite = UserMap->Find(Params.SessionInviteId))
			{
				return TOnlineResult<FGetSessionInviteById>({ *SessionInvite });
			}
			else
			{
				UE_LOGF(LogOnlineServices, Verbose, "[%ls] No invite found for invite id [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionInviteId));
				return TOnlineResult<FGetSessionInviteById>(Errors::NotFound());
			}
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No invite data found for account id [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
			return TOnlineResult<FGetSessionInviteById>(Errors::InvalidState());
		}
	}

	TOnlineResult<FGetAllSessionInvites> FSessionsCommon::GetAllSessionInvites(FGetAllSessionInvites::Params&& Params)
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId, FGetAllSessionInvites)
		
		TArray<TSharedRef<const ISessionInvite>> SessionInvites;

		if (const TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>* UserMap = SessionInvitesUserMap.Find(Params.LocalAccountId))
		{
			for (const TPair<FSessionInviteId, TSharedRef<FSessionInviteCommon>>& Entry : *UserMap)
			{
				SessionInvites.Add(Entry.Value);
			}
		}

		return TOnlineResult<FGetAllSessionInvites>({ MoveTemp(SessionInvites) });
	}

#undef CHECK_PARAMS_ID_HANDLE
#undef CHECK_PARAMS_FNAME

	// Asynchronous methods

#define CHECK_PARAMS_ID_HANDLE(IdHandle) \
	if (!IdHandle.IsValid()) \
	{ \
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] Value of [%ls] invalid.", UTF8_TO_TCHAR(__FUNCTION__), UTF8_TO_TCHAR(#IdHandle)); \
		return TOptional<FOnlineError>(Errors::InvalidParams()); \
	} \

#define CHECK_PARAMS_FNAME(Name) \
	if (Name.IsNone()) \
	{ \
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] Value of [%ls] invalid.", UTF8_TO_TCHAR(__FUNCTION__), UTF8_TO_TCHAR(#Name)); \
		return TOptional<FOnlineError>(Errors::InvalidParams()); \
	} \

#define CHECK_STATE_EXISTING_SESSION_NAME(SessionName) \
	TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ SessionName }); \
	if (GetSessionByNameResult.IsError()) \
	{ \
		UE_LOGF(LogOnlineServices, Verbose, "[%ls] Could not find session with name [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *SessionName.ToString()); \
		return TOptional<FOnlineError>(Errors::InvalidState()); \
	} \

	template<typename MethodStruct>
	TOnlineAsyncOpHandle<MethodStruct> FSessionsCommon::ExecuteAsyncSessionsMethod(typename MethodStruct::Params&& Params, TFuture<TOnlineResult<MethodStruct>>(FSessionsCommon::* ImplFunc)(const typename MethodStruct::Params& Params), bool bUseParallelQueue)
	{
		TOnlineAsyncOpRef<MethodStruct> Op = GetOp<MethodStruct>(MoveTemp(Params));
		const typename MethodStruct::Params& OpParams = Op->GetParams();

		if (TOptional<FOnlineError> ParamsCheck = CheckParams(OpParams))
		{
			Op->SetError(MoveTemp(ParamsCheck.GetValue()));
			return Op->GetHandle();
		}

		Op->Then([this, WeakThis = AsWeak(), ImplFunc](TOnlineAsyncOp<MethodStruct>& Op)
			{
				if (TSharedPtr<ISessions> StrongThis = WeakThis.Pin())
				{
					const typename MethodStruct::Params& OpParams = Op.GetParams();

					if (TOptional<FOnlineError> StateCheck = CheckState(Op.GetParams()))
					{
						Op.SetError(MoveTemp(StateCheck.GetValue()));
						return;
					}

					(this->*ImplFunc)(OpParams)
						.Next([WeakOp = Op.AsWeak()](const TOnlineResult<MethodStruct>& Result)
							{
								if (TOnlineAsyncOpPtr<MethodStruct> StrongOp = WeakOp.Pin())
								{
									if (Result.IsOk())
									{
										StrongOp->SetResult(typename MethodStruct::Result(Result.GetOkValue()));
									}
									else
									{
										StrongOp->SetError(FOnlineError(Result.GetErrorValue()));
									}
								}
							});
				}
			})
			.Enqueue(bUseParallelQueue ? Services.GetParallelQueue() : GetSerialQueue());

		return Op->GetHandle();
	}

	// CreateSession

	TOnlineAsyncOpHandle<FCreateSession> FSessionsCommon::CreateSession(FCreateSession::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FCreateSession>(MoveTemp(Params), &FSessionsCommon::CreateSessionImpl);
	}

	TFuture<TOnlineResult<FCreateSession>> FSessionsCommon::CreateSessionImpl(const FCreateSession::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FCreateSession>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FCreateSession::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)
		CHECK_PARAMS_FNAME(Params.SessionSettings.SchemaName)

		if (Params.SessionSettings.NumMaxConnections == 0)
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] NumMaxConnections invalid, can't be 0.", UTF8_TO_TCHAR(__FUNCTION__));
			return Errors::InvalidParams();
		}

		TArray<FSchemaAttributeId> AttributeIds;
		Params.SessionSettings.CustomSettings.GenerateKeyArray(AttributeIds);
		for (const FSchemaAttributeId& AttributeId : AttributeIds)
		{
			CHECK_PARAMS_FNAME(AttributeId)
		}

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FCreateSession::Params& Params) const
	{
		TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ Params.SessionName });
		if (GetSessionByNameResult.IsOk())
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] Another session with the name [%ls] already exists.", UTF8_TO_TCHAR(__FUNCTION__), *Params.SessionName.ToString());
			return TOptional<FOnlineError>(Errors::InvalidState());
		}

		if (Params.bPresenceEnabled)
		{
			for (const TPair<FName, FOnlineSessionId>& Entry : LocalSessionsByName)
			{
				TOnlineResult<FGetPresenceSession> GetPresenceSessionResult = GetPresenceSession({ Params.LocalAccountId });
				if (GetPresenceSessionResult.IsOk())
				{
					UE_LOGF(LogOnlineServices, Verbose, "[%ls] Could not create session with bPresenceEnabled set to true when another already exists [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *Entry.Key.ToString());
					return Errors::InvalidState();
				}
			}
		}

		return TOptional<FOnlineError>();
	}

	// UpdateSessionSettings

	TOnlineAsyncOpHandle<FUpdateSessionSettings> FSessionsCommon::UpdateSessionSettings(FUpdateSessionSettings::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FUpdateSessionSettings>(MoveTemp(Params), &FSessionsCommon::UpdateSessionSettingsImpl);
	}

	TFuture<TOnlineResult<FUpdateSessionSettings>> FSessionsCommon::UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FUpdateSessionSettings>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FUpdateSessionSettings::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		if (Params.Mutations.SchemaName.IsSet())
		{
			CHECK_PARAMS_FNAME(Params.Mutations.SchemaName.GetValue())
		}

		if (Params.Mutations.NumMaxConnections.IsSet())
		{
			if (Params.Mutations.NumMaxConnections.GetValue() == 0)
			{
				UE_LOGF(LogOnlineServices, Verbose, "[%ls] NumMaxConnections invalid, can't be 0.", UTF8_TO_TCHAR(__FUNCTION__));
				return Errors::InvalidParams();
			}
		}

		TArray<FSchemaAttributeId> UpdatedAttributeIds;
		Params.Mutations.UpdatedCustomSettings.GenerateKeyArray(UpdatedAttributeIds);
		for (const FSchemaAttributeId& UpdatedAttributeId : UpdatedAttributeIds)
		{
			CHECK_PARAMS_FNAME(UpdatedAttributeId)
		}

		for (const FSchemaAttributeId& RemovedAttributeId : Params.Mutations.RemovedCustomSettings)
		{
			CHECK_PARAMS_FNAME(RemovedAttributeId)
		}

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FUpdateSessionSettings::Params& Params) const
	{
		CHECK_STATE_EXISTING_SESSION_NAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	// LeaveSession

	TOnlineAsyncOpHandle<FLeaveSession> FSessionsCommon::LeaveSession(FLeaveSession::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FLeaveSession>(MoveTemp(Params), &FSessionsCommon::LeaveSessionImpl);
	}

	TFuture<TOnlineResult<FLeaveSession>> FSessionsCommon::LeaveSessionImpl(const FLeaveSession::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FLeaveSession>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FLeaveSession::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FLeaveSession::Params& Params) const
	{
		CHECK_STATE_EXISTING_SESSION_NAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	// FindSessions

	TOnlineAsyncOpHandle<FFindSessions> FSessionsCommon::FindSessions(FFindSessions::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FFindSessions>(MoveTemp(Params), &FSessionsCommon::FindSessionsImpl, true);
	}

	TFuture<TOnlineResult<FFindSessions>> FSessionsCommon::FindSessionsImpl(const FFindSessions::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FFindSessions>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FFindSessions::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)

		if (Params.MaxResults == 0)
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] MaxResults can't be 0", UTF8_TO_TCHAR(__FUNCTION__));
			return TOptional<FOnlineError>(Errors::InvalidParams());
		}

		for (const FFindSessionsSearchFilter& Filter : Params.Filters)
		{
			CHECK_PARAMS_FNAME(Filter.Key)
		}

		if (Params.SessionId.IsSet())
		{
			CHECK_PARAMS_ID_HANDLE(Params.SessionId.GetValue())
		}

		if (Params.TargetUser.IsSet())
		{
			CHECK_PARAMS_ID_HANDLE(Params.TargetUser.GetValue())
		}
		
		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FFindSessions::Params& Params) const
	{
		if (CurrentSessionSearchPromisesUserMap.Contains(Params.LocalAccountId))
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] Search already in progress", UTF8_TO_TCHAR(__FUNCTION__));
			return Errors::AlreadyPending();
		}

		return TOptional<FOnlineError>();
	}

	// StartMatchmaking

	TOnlineAsyncOpHandle<FStartMatchmaking> FSessionsCommon::StartMatchmaking(FStartMatchmaking::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FStartMatchmaking>(MoveTemp(Params), &FSessionsCommon::StartMatchmakingImpl);
	}

	TFuture<TOnlineResult<FStartMatchmaking>> FSessionsCommon::StartMatchmakingImpl(const FStartMatchmaking::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FStartMatchmaking>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FStartMatchmaking::Params& Params) const
	{
		if (TOptional<FOnlineError> Result = CheckParams(Params.SessionCreationParameters))
		{
			return Result;
		}

		for (const FFindSessionsSearchFilter& Filter : Params.SessionSearchFilters)
		{
			CHECK_PARAMS_FNAME(Filter.Key)
		}

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FStartMatchmaking::Params& Params) const
	{
		if (TOptional<FOnlineError> Result = CheckState(Params.SessionCreationParameters))
		{
			return Result;
		}

		return TOptional<FOnlineError>();
	}

	// JoinSession

	TOnlineAsyncOpHandle<FJoinSession> FSessionsCommon::JoinSession(FJoinSession::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FJoinSession>(MoveTemp(Params), &FSessionsCommon::JoinSessionImpl);
	}

	TFuture<TOnlineResult<FJoinSession>> FSessionsCommon::JoinSessionImpl(const FJoinSession::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FJoinSession>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FJoinSession::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_ID_HANDLE(Params.SessionId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FJoinSession::Params& Params) const
	{
		TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ Params.SessionName });
		if (GetSessionByNameResult.IsOk())
		{
			if (GetSessionByNameResult.GetOkValue().Session->GetSessionId() != Params.SessionId)
			{
				UE_LOGF(LogOnlineServices, Verbose, "[%ls] Another session with the name [%ls] already exists.", UTF8_TO_TCHAR(__FUNCTION__), *Params.SessionName.ToString());
				return TOptional<FOnlineError>(Errors::InvalidState());
			}
		}

		TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
		if (GetSessionByIdResult.IsError())
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] Session [%ls] not found. Please call FindSessions to get an updated list of available sessions. Invites should be cached before joining ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
			return TOptional<FOnlineError>(GetSessionByIdResult.GetErrorValue());
		}

		TSharedRef<const ISession> FoundSession = GetSessionByIdResult.GetOkValue().Session;

		const FSessionSettings& SessionSettings = FoundSession->GetSessionSettings();

		if (FoundSession->GetSessionMembers().Contains(Params.LocalAccountId))
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] User [%ls] already in session [%ls]", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId), *ToLogString(Params.SessionId));
			return TOptional<FOnlineError>(Errors::AccessDenied());
		}

		if (!FoundSession->IsJoinable())
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] Session [%ls] not joinable ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
			return TOptional<FOnlineError>(Errors::AccessDenied());
		}
		
		if (Params.bPresenceEnabled)
		{
			for (const TPair<FName, FOnlineSessionId>& Entry : LocalSessionsByName)
			{
				TOnlineResult<FGetPresenceSession> GetPresenceSessionResult = GetPresenceSession({ Params.LocalAccountId });
				if (GetPresenceSessionResult.IsOk())
				{
					UE_LOGF(LogOnlineServices, Verbose, "[%ls] Could not join session with bPresenceEnabled set to true when another presence session already exists [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *Entry.Key.ToString());
					return TOptional<FOnlineError>(Errors::InvalidState());
				}
			}
		}

		return TOptional<FOnlineError>();
	}

	// AddSessionMember

	TOnlineAsyncOpHandle<FAddSessionMember> FSessionsCommon::AddSessionMember(FAddSessionMember::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FAddSessionMember>(MoveTemp(Params), &FSessionsCommon::AddSessionMemberImpl);
	}

	TFuture<TOnlineResult<FAddSessionMember>> FSessionsCommon::AddSessionMemberImpl(const FAddSessionMember::Params& Params)
	{
		TPromise<TOnlineResult<FAddSessionMember>> Promise;
		TFuture<TOnlineResult<FAddSessionMember>> Future = Promise.GetFuture();

		Promise.EmplaceValue(AddSessionMemberInternal(Params));		

		return Future;
	}

	TOnlineResult<FAddSessionMember> FSessionsCommon::AddSessionMemberInternal(const FAddSessionMember::Params& Params)
	{
		TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.SessionName });
		if (GetMutableSessionByNameResult.IsError())
		{
			return TOnlineResult<FAddSessionMember>(GetMutableSessionByNameResult.GetErrorValue());
		}

		TSharedRef<FSessionCommon> FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

		FSessionSettings& SessionSettings = FoundSession->SessionSettings;

		FoundSession->SessionMembers.Emplace(Params.LocalAccountId);

		return TOnlineResult<FAddSessionMember>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FAddSessionMember::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FAddSessionMember::Params& Params) const
	{
		CHECK_STATE_EXISTING_SESSION_NAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	// RemoveSessionMember

	TOnlineAsyncOpHandle<FRemoveSessionMember> FSessionsCommon::RemoveSessionMember(FRemoveSessionMember::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FRemoveSessionMember>(MoveTemp(Params), &FSessionsCommon::RemoveSessionMemberImpl);
	}

	TFuture<TOnlineResult<FRemoveSessionMember>> FSessionsCommon::RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params)
	{
		TPromise<TOnlineResult<FRemoveSessionMember>> Promise;
		TFuture<TOnlineResult<FRemoveSessionMember>> Future = Promise.GetFuture();

		Promise.EmplaceValue(RemoveSessionMemberInternal(Params));

		return Future;
	}

	TOnlineResult<FRemoveSessionMember> FSessionsCommon::RemoveSessionMemberInternal(const FRemoveSessionMember::Params& Params)
	{
		TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.SessionName });
		if (GetMutableSessionByNameResult.IsError())
		{
			return TOnlineResult<FRemoveSessionMember>(GetMutableSessionByNameResult.GetErrorValue());
		}

		TSharedRef<FSessionCommon> FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;

		FSessionSettings& SessionSettings = FoundSession->SessionSettings;

		FoundSession->SessionMembers.Remove(Params.LocalAccountId);

		return TOnlineResult<FRemoveSessionMember>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FRemoveSessionMember::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FRemoveSessionMember::Params& Params) const
	{
		CHECK_STATE_EXISTING_SESSION_NAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	// SendSessionInvite

	TOnlineAsyncOpHandle<FSendSessionInvite> FSessionsCommon::SendSessionInvite(FSendSessionInvite::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FSendSessionInvite>(MoveTemp(Params), &FSessionsCommon::SendSessionInviteImpl);
	}

	TFuture<TOnlineResult<FSendSessionInvite>> FSessionsCommon::SendSessionInviteImpl(const FSendSessionInvite::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FSendSessionInvite>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FSendSessionInvite::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_FNAME(Params.SessionName)

		if (Params.TargetUsers.IsEmpty())
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] TargetUsers array is empty", UTF8_TO_TCHAR(__FUNCTION__));
			return TOptional<FOnlineError>(Errors::InvalidParams());
		}

		for (const FAccountId& TargetUser : Params.TargetUsers)
		{
			CHECK_PARAMS_ID_HANDLE(TargetUser)
		}

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FSendSessionInvite::Params& Params) const
	{
		CHECK_STATE_EXISTING_SESSION_NAME(Params.SessionName)

		return TOptional<FOnlineError>();
	}

	// RejectSessionInvite

	TOnlineAsyncOpHandle<FRejectSessionInvite> FSessionsCommon::RejectSessionInvite(FRejectSessionInvite::Params&& Params)
	{
		return ExecuteAsyncSessionsMethod<FRejectSessionInvite>(MoveTemp(Params), &FSessionsCommon::RejectSessionInviteImpl);
	}

	TFuture<TOnlineResult<FRejectSessionInvite>> FSessionsCommon::RejectSessionInviteImpl(const FRejectSessionInvite::Params& Params)
	{
		return MakeFulfilledPromise<TOnlineResult<FRejectSessionInvite>>(Errors::NotImplemented()).GetFuture();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckParams(const FRejectSessionInvite::Params& Params) const
	{
		CHECK_PARAMS_ID_HANDLE(Params.LocalAccountId)
		CHECK_PARAMS_ID_HANDLE(Params.SessionInviteId)

		return TOptional<FOnlineError>();
	}

	TOptional<FOnlineError> FSessionsCommon::CheckState(const FRejectSessionInvite::Params& Params) const
	{
		if (const TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>* UserMap = SessionInvitesUserMap.Find(Params.LocalAccountId))
		{
			if (!UserMap->Contains(Params.SessionInviteId))
			{
				UE_LOGF(LogOnlineServices, Verbose, "[%ls] No invite found for invite id [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionInviteId));
				return TOptional<FOnlineError>(Errors::NotFound());
			}
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[%ls] No invite data found for account id [%ls].", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.LocalAccountId));
			return TOptional<FOnlineError>(Errors::InvalidState());
		}

		return TOptional<FOnlineError>();
	}

#undef CHECK_PARAMS_ID_HANDLE
#undef CHECK_PARAMS_FNAME
#undef CHECK_STATE_SESSION_NAME_ALREADY_EXISTS

	// Events
	TOnlineEvent<void(const FSessionCreated&)> FSessionsCommon::OnSessionCreated()
	{
		return SessionEvents.OnSessionCreated;
	}

	TOnlineEvent<void(const FSessionJoined&)> FSessionsCommon::OnSessionJoined()
	{
		return SessionEvents.OnSessionJoined;
	}

	TOnlineEvent<void(const FSessionLeft&)> FSessionsCommon::OnSessionLeft()
	{
		return SessionEvents.OnSessionLeft;
	}

	TOnlineEvent<void(const FSessionUpdated&)> FSessionsCommon::OnSessionUpdated()
	{
		return SessionEvents.OnSessionUpdated;
	}

	TOnlineEvent<void(const FSessionInviteReceived&)> FSessionsCommon::OnSessionInviteReceived()
	{
		return SessionEvents.OnSessionInviteReceived;
	}

	TOnlineEvent<void(const FUISessionJoinRequested&)> FSessionsCommon::OnUISessionJoinRequested()
	{
		return SessionEvents.OnUISessionJoinRequested;
	}

	// Auxiliary methods

	void FSessionsCommon::AddSessionInvite(const TSharedRef<FSessionInviteCommon> SessionInvite, const TSharedRef<FSessionCommon> Session, const FAccountId& LocalAccountId)
	{
		AllSessionsById.Emplace(Session->GetSessionId(), Session);

		TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>& SessionInvitesMap = SessionInvitesUserMap.FindOrAdd(LocalAccountId);
		SessionInvitesMap.Emplace(SessionInvite->InviteId, SessionInvite);
	}

	void FSessionsCommon::AddSearchResult(const TSharedRef<FSessionCommon> Session, const FAccountId& LocalAccountId)
	{
		AllSessionsById.Emplace(Session->GetSessionId(), Session);

		TArray<FOnlineSessionId>& SearchResults = SearchResultsUserMap.FindOrAdd(LocalAccountId);
		SearchResults.Add(Session->GetSessionId());
	}

	void FSessionsCommon::AddSessionWithReferences(const TSharedRef<FSessionCommon> Session, const FName& SessionName, const FAccountId& LocalAccountId, bool bIsPresenceSession)
	{
		AllSessionsById.Emplace(Session->GetSessionId(), Session);

		AddSessionReferences(Session->GetSessionId(), SessionName, LocalAccountId, bIsPresenceSession);
	}

	void FSessionsCommon::AddSessionReferences(const FOnlineSessionId SessionId, const FName& SessionName, const FAccountId& LocalAccountId, bool bIsPresenceSession)
	{
		LocalSessionsByName.Emplace(SessionName, SessionId);

		NamedSessionUserMap.FindOrAdd(LocalAccountId).AddUnique(SessionName);

		if (bIsPresenceSession)
		{
			SetPresenceSessionImpl({ LocalAccountId, SessionId });
		}
	}

	void FSessionsCommon::ClearSessionInvitesForSession(const FAccountId& LocalAccountId, const FOnlineSessionId SessionId)
	{
		if (TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>* UserMap = SessionInvitesUserMap.Find(LocalAccountId))
		{
			TArray<FSessionInviteId> InviteIdsToRemove;
			for (const TPair<FSessionInviteId, TSharedRef<FSessionInviteCommon>>& Entry : *UserMap)
			{
				if (Entry.Value->SessionId == SessionId)
				{
					InviteIdsToRemove.Add(Entry.Key);
				}
			}

			for (const FSessionInviteId& InviteId : InviteIdsToRemove)
			{
				UserMap->Remove(InviteId);
			}
		}
	}

	void FSessionsCommon::ClearSessionReferences(const FOnlineSessionId SessionId, const FName& SessionName, const FAccountId& LocalAccountId)
	{
		NamedSessionUserMap.FindChecked(LocalAccountId).Remove(SessionName);

		TOnlineResult<FIsPresenceSession> IsPresenceSessionResult = IsPresenceSession({ LocalAccountId, SessionId });
		if (IsPresenceSessionResult.IsOk())
		{
			if (IsPresenceSessionResult.GetOkValue().bIsPresenceSession)
			{
				ClearPresenceSessionImpl({ LocalAccountId });
			}
		}

		ClearSessionByName(SessionName);
		ClearSessionById(SessionId);
	}

	void FSessionsCommon::ClearSessionByName(const FName& SessionName)
	{
		for (const TPair<FAccountId, TArray<FName>>& Entry : NamedSessionUserMap)
		{
			if (Entry.Value.Contains(SessionName))
			{
				return;
			}
		}

		// If no references were found, we'll remove the named session entry
		LocalSessionsByName.Remove(SessionName);
	}

	void FSessionsCommon::ClearSessionById(const FOnlineSessionId& SessionId)
	{
		// PresenceSessionsUserMap is not evaluated, since any session there would also be in LocalSessionsByName
		for (const TPair<FName, FOnlineSessionId>& Entry : LocalSessionsByName)
		{
			if (Entry.Value == SessionId)
			{
				return;
			}
		}

		for (const TPair<FAccountId, TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>>& Entry : SessionInvitesUserMap)
		{
			for (const TPair<FSessionInviteId, TSharedRef<FSessionInviteCommon>>& InviteMap : Entry.Value)
			{
				if (InviteMap.Value->SessionId == SessionId)
				{
					return;
				}
			}			
		}

		for (const TPair<FAccountId, TArray<FOnlineSessionId>>& Entry : SearchResultsUserMap)
		{
			if (Entry.Value.Contains(SessionId))
			{
				return;
			}
		}

		// If no references were found, we'll remove the session entry
		AllSessionsById.Remove(SessionId);
	}

#define COPY_TOPTIONAL_VALUE_IF_SET(Value) \
	if (UpdatedValues.Value.IsSet()) \
	{ \
		SessionSettingsChanges.Value = UpdatedValues.Value.GetValue(); \
	} \

	FSessionUpdate FSessionsCommon::BuildSessionUpdate(const TSharedRef<FSessionCommon>& Session, const FSessionSettingsUpdate& UpdatedValues) const
	{
		FSessionUpdate Result;

		FSessionSettingsChanges SessionSettingsChanges;

		COPY_TOPTIONAL_VALUE_IF_SET(SchemaName)
		COPY_TOPTIONAL_VALUE_IF_SET(NumMaxConnections)
		COPY_TOPTIONAL_VALUE_IF_SET(JoinPolicy)
		COPY_TOPTIONAL_VALUE_IF_SET(bAllowNewMembers)

		SessionSettingsChanges.RemovedCustomSettings.Append(UpdatedValues.RemovedCustomSettings);

		for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& Entry : UpdatedValues.UpdatedCustomSettings)
		{
			if (const FCustomSessionSetting* CustomSetting = Session->GetSessionSettings().CustomSettings.Find(Entry.Key))
			{
				FCustomSessionSettingUpdate SettingUpdate = { *CustomSetting, Entry.Value };

				SessionSettingsChanges.ChangedCustomSettings.Emplace(Entry.Key, SettingUpdate);
			}
			else
			{
				SessionSettingsChanges.AddedCustomSettings.Add(Entry);
			}
		}

		Result.SessionSettingsChanges = SessionSettingsChanges;

		return Result;
	}

#undef COPY_TOPTIONAL_VALUE_IF_SET

/* UE::Online */ }
