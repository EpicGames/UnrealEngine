// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SocialCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FSocialCommon::FSocialCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Social"), InServices)
{
}

void FSocialCommon::RegisterCommands()
{
	RegisterCommand(&FSocialCommon::QueryFriends);
	RegisterCommand(&FSocialCommon::GetFriends);
	RegisterCommand(&FSocialCommon::SendFriendInvite);
	RegisterCommand(&FSocialCommon::AcceptFriendInvite);
	RegisterCommand(&FSocialCommon::RejectFriendInvite);
	RegisterCommand(&FSocialCommon::QueryBlockedUsers);
	RegisterCommand(&FSocialCommon::GetBlockedUsers);
	RegisterCommand(&FSocialCommon::BlockUser);
}

TOnlineAsyncOpHandle<FQueryFriends> FSocialCommon::QueryFriends(FQueryFriends::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryFriends> Operation = GetOp<FQueryFriends>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetFriends> FSocialCommon::GetFriends(FGetFriends::Params&& Params)
{
	return TOnlineResult<FGetFriends>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FSendFriendInvite> FSocialCommon::SendFriendInvite(FSendFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FSendFriendInvite> Operation = GetOp<FSendFriendInvite>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAcceptFriendInvite> FSocialCommon::AcceptFriendInvite(FAcceptFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FAcceptFriendInvite> Operation = GetOp<FAcceptFriendInvite>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FRejectFriendInvite> FSocialCommon::RejectFriendInvite(FRejectFriendInvite::Params&& Params)
{
	TOnlineAsyncOpRef<FRejectFriendInvite> Operation = GetOp<FRejectFriendInvite>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineEvent<void(const FRelationshipUpdated&)> FSocialCommon::OnRelationshipUpdated()
{
	return OnRelationshipUpdatedEvent;
}

TOnlineAsyncOpHandle<FQueryBlockedUsers> FSocialCommon::QueryBlockedUsers(FQueryBlockedUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryBlockedUsers> Operation = GetOp<FQueryBlockedUsers>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetBlockedUsers> FSocialCommon::GetBlockedUsers(FGetBlockedUsers::Params&& Params)
{
	return TOnlineResult<FGetBlockedUsers>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FBlockUser> FSocialCommon::BlockUser(FBlockUser::Params&& Params)
{
	TOnlineAsyncOpRef<FBlockUser> Operation = GetOp<FBlockUser>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

void FSocialCommon::BroadcastRelationshipUpdated(FAccountId LocalAccountId, FAccountId RemoteAccountId, ERelationship OldRelationship, ERelationship NewRelationship)
{
	FRelationshipUpdated Event;
	Event.LocalAccountId = LocalAccountId;
	Event.RemoteAccountId = RemoteAccountId;
	Event.OldRelationship = OldRelationship;
	Event.NewRelationship = NewRelationship;

	OnRelationshipUpdatedEvent.Broadcast(Event);
}


/* UE::Online */ }
