// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/StatsCommon.h"

#include "Online/AuthCommon.h"

namespace UE::Online {

const TCHAR* LexToString(EStatModifyMethod Value)
{
	switch (Value)
	{
	case EStatModifyMethod::Sum:
		return TEXT("Sum");
	case EStatModifyMethod::Largest:
		return TEXT("Largest");
	case EStatModifyMethod::Smallest:
		return TEXT("Smallest");
	default: checkNoEntry(); // Intentional fallthrough
	case EStatModifyMethod::Set:
		return TEXT("Set");
	}
}

void LexFromString(EStatModifyMethod& OutValue, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Sum")) == 0)
	{
		OutValue = EStatModifyMethod::Sum;
	}
	else if (FCString::Stricmp(InStr, TEXT("Set")) == 0)
	{
		OutValue = EStatModifyMethod::Set;
	}
	else if (FCString::Stricmp(InStr, TEXT("Largest")) == 0)
	{
		OutValue = EStatModifyMethod::Largest;
	}
	else if (FCString::Stricmp(InStr, TEXT("Smallest")) == 0)
	{
		OutValue = EStatModifyMethod::Smallest;
	}
	else
	{
		ensureMsgf(false, TEXT("Can't convert %s to EStatModifyMethod"), InStr);
		OutValue = EStatModifyMethod::Set;
	}
}

FStatsCommon::FStatsCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("Stats"), InServices)
{
}

void FStatsCommon::UpdateConfig()
{
	FStatsCommonConfig Config;
	TOnlineComponent::LoadConfig(Config);

	for (FStatDefinition& StatDefinition : Config.StatDefinitions)
	{
		FString StatName = StatDefinition.Name;
		StatDefinitions.Emplace(MoveTemp(StatName), MoveTemp(StatDefinition));
	}
}

void FStatsCommon::RegisterCommands()
{
	TOnlineComponent<IStats>::RegisterCommands();

	RegisterCommand(&FStatsCommon::UpdateStats);
	RegisterCommand(&FStatsCommon::QueryStats);
	RegisterCommand(&FStatsCommon::BatchQueryStats);
	RegisterCommand(&FStatsCommon::TriggerEvent);
#if !UE_BUILD_SHIPPING
	RegisterCommand(&FStatsCommon::ResetStats);
#endif // !UE_BUILD_SHIPPING
}

TOnlineAsyncOpHandle<FUpdateStats> FStatsCommon::UpdateStats(FUpdateStats::Params&& Params)
{
	return UpdateStats_Implementation(MoveTemp(Params))->GetHandle(); 
}

TOnlineAsyncOpHandle<FQueryStats> FStatsCommon::QueryStats(FQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryStats> Op = GetJoinableOp<FQueryStats>(MoveTemp(Params));
	FAuthCommon* AuthPtr = Services.Get<FAuthCommon>();
	if (AuthPtr == nullptr || !AuthPtr->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryStats>& InAsyncOp)
	{
		FQueryStats::Result Result;

		FUserStats* ExistingUserStats = CachedUsersStats.FindByPredicate(FFindUserStatsByAccountId(InAsyncOp.GetParams().TargetAccountId));
		ReadStatsFromCache(ExistingUserStats, InAsyncOp.GetParams().StatNames, Result.Stats);

		InAsyncOp.SetResult(MoveTemp(Result));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FTriggerEvent> FStatsCommon::TriggerEvent(FTriggerEvent::Params&& Params)
{
	FUserStats UserStats;
	UserStats.AccountId = Params.LocalAccountId;
	for (const TPair<FString, FSchemaVariant>& EventPair : Params.EventParams)
	{
		// generic mapping: convert the event parameter value into a Stat value
		UserStats.Stats.Emplace(EventPair.Key, FStatValue(EventPair.Value));
	}

	return UpdateStats_Implementation({ Params.LocalAccountId, { MoveTemp(UserStats) } })->GetWrappedHandle<FTriggerEvent>(
	[this](TOnlineResult<FUpdateStats> UpdateResult) -> TOnlineResult<FTriggerEvent>
	{
		if (UpdateResult.IsOk())
		{
			// TODO: additional processing here
			return TOnlineResult<FTriggerEvent>();
		}

		return TOnlineResult<FTriggerEvent>(MoveTemp(UpdateResult.GetErrorValue()));
	});
}

TOnlineAsyncOpHandle<FBatchQueryStats> FStatsCommon::BatchQueryStats(FBatchQueryStats::Params&& Params)
{
	TOnlineAsyncOpRef<FBatchQueryStats> Op = GetOp<FBatchQueryStats>(MoveTemp(Params));
	FAuthCommon* AuthPtr = Services.Get<FAuthCommon>();
	if (AuthPtr == nullptr || !AuthPtr->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FBatchQueryStats>& InAsyncOp)
	{
		FBatchQueryStats::Result Result;

		for (const FAccountId& TargetAccountId : InAsyncOp.GetParams().TargetAccountIds)
		{
			FUserStats& UserStats = Result.UsersStats.Emplace_GetRef();
			UserStats.AccountId = TargetAccountId;
			FUserStats* ExistingUserStats = CachedUsersStats.FindByPredicate(FFindUserStatsByAccountId(TargetAccountId));
			ReadStatsFromCache(ExistingUserStats, InAsyncOp.GetParams().StatNames, UserStats.Stats);
		}

		InAsyncOp.SetResult(MoveTemp(Result));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

#if !UE_BUILD_SHIPPING
TOnlineAsyncOpHandle<FResetStats> FStatsCommon::ResetStats(FResetStats::Params&& Params)
{
	TOnlineAsyncOpRef<FResetStats> Op = GetOp<FResetStats>(MoveTemp(Params));
	FAuthCommon* AuthPtr = Services.Get<FAuthCommon>();
	if (AuthPtr == nullptr || !AuthPtr->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FResetStats>& InAsyncOp)
	{
		const uint32 Index = CachedUsersStats.IndexOfByPredicate(FFindUserStatsByAccountId(InAsyncOp.GetParams().LocalAccountId));
		if (Index != INDEX_NONE)
		{
			CachedUsersStats.RemoveAt(Index);
		}

		FFindUserStatsByAccountId FindUserStatsByAccountId(InAsyncOp.GetParams().LocalAccountId);
		CachedUsersStats.RemoveAll(FindUserStatsByAccountId);

		InAsyncOp.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}
#endif // !UE_BUILD_SHIPPING

TOnlineResult<FGetCachedStats> FStatsCommon::GetCachedStats(FGetCachedStats::Params&& Params) const
{
	return TOnlineResult<FGetCachedStats>({ CachedUsersStats });
}

void FStatsCommon::CacheUserStats(const FUserStats& UserStats)
{
	if (FUserStats* ExistingUserStats = CachedUsersStats.FindByPredicate(FFindUserStatsByAccountId(UserStats.AccountId)))
	{
		for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
		{
			if (FStatValue* StatValue = ExistingUserStats->Stats.Find(StatPair.Key))
			{
				*StatValue = StatPair.Value;
			}
			else
			{
				ExistingUserStats->Stats.Add(StatPair);
			}
		}
	}
	else
	{
		CachedUsersStats.Emplace(UserStats);
	}
}

TOnlineEvent<void(const FStatsUpdated&)> FStatsCommon::OnStatsUpdated() 
{ 
	return OnStatsUpdatedEvent; 
}

const FStatDefinition* FStatsCommon::GetStatDefinition(const FString& StatName) const 
{ 
	return StatDefinitions.Find(StatName); 
}

TOnlineAsyncOpRef<FUpdateStats> FStatsCommon::UpdateStats_Implementation(FUpdateStats::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdateStats> Op = GetOp<FUpdateStats>(MoveTemp(Params));
	FAuthCommon* AuthPtr = Services.Get<FAuthCommon>();
	if (AuthPtr == nullptr || !AuthPtr->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op;
	}

	Op->Then([this](TOnlineAsyncOp<FUpdateStats>& InAsyncOp) mutable
	{
		for (const FUserStats& UpdateUserStats : InAsyncOp.GetParams().UpdateUsersStats)
		{
			FUserStats* ExistingUserStats = CachedUsersStats.FindByPredicate(FFindUserStatsByAccountId(UpdateUserStats.AccountId));
			if (!ExistingUserStats)
			{
				ExistingUserStats = &CachedUsersStats.Emplace_GetRef();
				ExistingUserStats->AccountId = UpdateUserStats.AccountId;
			}

			TMap<FString, FStatValue>& UserStats = ExistingUserStats->Stats;
			for (const TPair<FString, FSchemaVariant>& UpdateUserStatPair : UpdateUserStats.Stats)
			{
				if (const FStatDefinition* StatDefinition = GetStatDefinition(UpdateUserStatPair.Key))
				{
					if (FStatValue* StatValue = UserStats.Find(UpdateUserStatPair.Key))
					{
						switch (StatDefinition->ModifyMethod)
						{
						case EStatModifyMethod::Set:
							*StatValue = UpdateUserStatPair.Value;
							break;
						case EStatModifyMethod::Sum:
							if (StatValue->VariantData.IsType<double>())
							{
								StatValue->Set(StatValue->GetDouble() + UpdateUserStatPair.Value.GetDouble());
							}
							else if (StatValue->VariantData.IsType<int64>())
							{
								StatValue->Set(StatValue->GetInt64() + UpdateUserStatPair.Value.GetInt64());
							}
							break;
						case EStatModifyMethod::Largest:
							if (StatValue->VariantData.IsType<double>())
							{
								StatValue->Set(FMath::Max(StatValue->GetDouble(), UpdateUserStatPair.Value.GetDouble()));
							}
							else if (StatValue->VariantData.IsType<int64>())
							{
								StatValue->Set(FMath::Max(StatValue->GetInt64(), UpdateUserStatPair.Value.GetInt64()));
							}
							break;
						case EStatModifyMethod::Smallest:
							if (StatValue->VariantData.IsType<double>())
							{
								StatValue->Set(FMath::Min(StatValue->GetDouble(), UpdateUserStatPair.Value.GetDouble()));
							}
							else if (StatValue->VariantData.IsType<int64>())
							{
								StatValue->Set(FMath::Min(StatValue->GetInt64(), UpdateUserStatPair.Value.GetInt64()));
							}
							break;
						}
					}
					else
					{
						UserStats.Emplace(UpdateUserStatPair.Key, UpdateUserStatPair.Value);
					}
				}
			}

			CacheUserStats(*ExistingUserStats);
		}

		InAsyncOp.SetResult({});
		OnStatsUpdatedEvent.Broadcast(InAsyncOp.GetParams());
	})
	.Enqueue(GetSerialQueue());

	return Op;
}

void FStatsCommon::ReadStatsFromCache(const FUserStats* ExistingUserStats, const TArray<FString>& StatNames, TMap<FString, FStatValue>& OutStats) const
{
	for (const FString& StatName : StatNames)
	{
		if (ExistingUserStats)
		{
			if (const FStatValue* StatValue = ExistingUserStats->Stats.Find(StatName))
			{
				OutStats.Add(StatName, *StatValue);
				continue;
			}
		}

		if (const FStatDefinition* StatDefinition = StatDefinitions.Find(StatName))
		{
			const FStatValue& StatValue = OutStats.FindOrAdd(StatName, StatDefinition->DefaultValue);
			if (StatValue.GetType() == ESchemaAttributeType::None)
			{
				UE_LOGF(LogOnlineServices, Warning, "Stat Value in config definitions %ls has no valid type", *StatName);
			}
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "Make sure to add the Config.StatDefinitions array");
		}
	}
}

/* UE::Online */ }
