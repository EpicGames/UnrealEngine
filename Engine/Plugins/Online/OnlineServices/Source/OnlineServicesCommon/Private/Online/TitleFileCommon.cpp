// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/TitleFileCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FTitleFileCommon::FTitleFileCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("TitleFile"), InServices)
{
}

void FTitleFileCommon::RegisterCommands()
{
	RegisterCommand(&FTitleFileCommon::EnumerateFiles);
	RegisterCommand(&FTitleFileCommon::GetEnumeratedFiles);
	RegisterCommand(&FTitleFileCommon::ReadFile);
}

TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> FTitleFileCommon::EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params)
{
	TOnlineAsyncOpRef<FTitleFileEnumerateFiles> Operation = GetOp<FTitleFileEnumerateFiles>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FTitleFileGetEnumeratedFiles> FTitleFileCommon::GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params)
{
	return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FTitleFileReadFile> FTitleFileCommon::ReadFile(FTitleFileReadFile::Params&& Params)
{
	TOnlineAsyncOpRef<FTitleFileReadFile> Operation = GetOp<FTitleFileReadFile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

/* UE::Online */}
