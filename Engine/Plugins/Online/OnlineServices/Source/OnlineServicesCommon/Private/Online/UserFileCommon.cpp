// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserFileCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FUserFileCommon::FUserFileCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("UserFile"), InServices)
{
}

void FUserFileCommon::RegisterCommands()
{
	RegisterCommand(&FUserFileCommon::EnumerateFiles);
	RegisterCommand(&FUserFileCommon::GetEnumeratedFiles);
	RegisterCommand(&FUserFileCommon::ReadFile);
	RegisterCommand(&FUserFileCommon::WriteFile);
	RegisterCommand(&FUserFileCommon::CopyFile);
	RegisterCommand(&FUserFileCommon::DeleteFile);
}

TOnlineAsyncOpHandle<FUserFileEnumerateFiles> FUserFileCommon::EnumerateFiles(FUserFileEnumerateFiles::Params&& Params)
{
	TOnlineAsyncOpRef<FUserFileEnumerateFiles> Operation = GetOp<FUserFileEnumerateFiles>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FUserFileGetEnumeratedFiles> FUserFileCommon::GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params)
{
	return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FUserFileReadFile> FUserFileCommon::ReadFile(FUserFileReadFile::Params&& Params)
{
	TOnlineAsyncOpRef<FUserFileReadFile> Operation = GetOp<FUserFileReadFile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileWriteFile> FUserFileCommon::WriteFile(FUserFileWriteFile::Params&& Params)
{
	TOnlineAsyncOpRef<FUserFileWriteFile> Operation = GetOp<FUserFileWriteFile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileCopyFile> FUserFileCommon::CopyFile(FUserFileCopyFile::Params&& Params)
{
	TOnlineAsyncOpRef<FUserFileCopyFile> Operation = GetOp<FUserFileCopyFile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileDeleteFile> FUserFileCommon::DeleteFile(FUserFileDeleteFile::Params&& Params)
{
	TOnlineAsyncOpRef<FUserFileDeleteFile> Operation = GetOp<FUserFileDeleteFile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

/* UE::Online */}
