// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserInfoCommon.h"

#include "Online/OnlineUtils.h"

namespace UE::Online {

FUserInfoCommon::FUserInfoCommon(FOnlineServicesCommon& InServices)
	: TOnlineComponent(TEXT("UserInfo"), InServices)
{
}

void FUserInfoCommon::RegisterCommands()
{
	RegisterCommand(&FUserInfoCommon::QueryUserInfo);
	RegisterCommand(&FUserInfoCommon::GetUserInfo);
	RegisterCommand(&FUserInfoCommon::QueryUserAvatar);
	RegisterCommand(&FUserInfoCommon::GetUserAvatar);
	RegisterCommand(&FUserInfoCommon::ShowUserProfile);
}

TOnlineAsyncOpHandle<FQueryUserInfo> FUserInfoCommon::QueryUserInfo(FQueryUserInfo::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryUserInfo> Operation = GetOp<FQueryUserInfo>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetUserInfo> FUserInfoCommon::GetUserInfo(FGetUserInfo::Params&& Params)
{
	return TOnlineResult<FGetUserInfo>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FQueryUserAvatar> FUserInfoCommon::QueryUserAvatar(FQueryUserAvatar::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryUserAvatar> Operation = GetOp<FQueryUserAvatar>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

TOnlineResult<FGetUserAvatar> FUserInfoCommon::GetUserAvatar(FGetUserAvatar::Params&& Params)
{
	return TOnlineResult<FGetUserAvatar>(Errors::NotImplemented());
}

TOnlineAsyncOpHandle<FShowUserProfile> FUserInfoCommon::ShowUserProfile(FShowUserProfile::Params&& Params)
{
	TOnlineAsyncOpRef<FShowUserProfile> Operation = GetOp<FShowUserProfile>(MoveTemp(Params));
	Operation->SetError(Errors::NotImplemented());
	return Operation->GetHandle();
}

/* UE::Online */ }
