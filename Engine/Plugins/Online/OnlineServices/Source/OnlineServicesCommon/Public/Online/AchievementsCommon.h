// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Achievements.h"
#include "Online/OnlineComponent.h"
#include "Online/Stats.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

struct FAchievementUnlockCondition
{
	FString StatName;
	FStatValue UnlockThreshold; // The unlock rule depends on Stat modification type
};

struct FAchievementUnlockRule
{
	FString AchievementId;
	TArray<FAchievementUnlockCondition> Conditions;

	bool ContainsStat(const FString& StatName) const;
};

struct FAchievementsCommonConfig
{
	bool bIsTitleManaged = false;
	TArray<FAchievementUnlockRule> UnlockRules;
};

namespace Meta
{

BEGIN_ONLINE_STRUCT_META(FAchievementUnlockCondition)
	ONLINE_STRUCT_FIELD(FAchievementUnlockCondition, StatName),
	ONLINE_STRUCT_FIELD(FAchievementUnlockCondition, UnlockThreshold)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementUnlockRule)
	ONLINE_STRUCT_FIELD(FAchievementUnlockRule, AchievementId),
	ONLINE_STRUCT_FIELD(FAchievementUnlockRule, Conditions)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementsCommonConfig)
	ONLINE_STRUCT_FIELD(FAchievementsCommonConfig, bIsTitleManaged),
	ONLINE_STRUCT_FIELD(FAchievementsCommonConfig, UnlockRules)
END_ONLINE_STRUCT_META()

/* Meta */ }

class FAchievementsCommon : public TOnlineComponent<IAchievements>
{
public:
	using Super = IAchievements;

	UE_API FAchievementsCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void Initialize() override;
	UE_API virtual void Shutdown() override;
	UE_API virtual void UpdateConfig() override;
	UE_API virtual void RegisterCommands() override;

	// IAchievements
	UE_API virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const override;
	UE_API virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) override;
	UE_API virtual TOnlineResult<FDisplayAchievementUI> DisplayAchievementUI(FDisplayAchievementUI::Params&& Params) override;
	UE_API virtual TOnlineEvent<void(const FAchievementStateUpdated&)> OnAchievementStateUpdated() override;

protected:
	TOnlineEventCallable<void(const FAchievementStateUpdated&)> OnAchievementStateUpdatedEvent;

	UE_API void OnAchievementStatesQueried(const FAccountId& AccountId);

	UE_API void UnlockAchievementsByStats(const FStatsUpdated& StatsUpdated);
	UE_API void ExecuteUnlockRulesRelatedToStat(const FAccountId& AccountId, const FString& StatName, const TMap<FString, FStatValue>& Stats, TArray<FString>& OutAchievementsToUnlock);
	UE_API bool MeetUnlockCondition(const FAchievementUnlockRule& AchievementUnlockRule, const TMap<FString, FStatValue>& Stats);
	UE_API bool IsUnlocked(const FAccountId& AccountId, const FString& AchievementName) const;

	FOnlineEventDelegateHandle StatEventHandle;

	FAchievementsCommonConfig Config;

	using FAchievementStateMap = TMap<FString, FAchievementState>;
	TMap<FAccountId, FAchievementStateMap> AchievementStates;
};

/* UE::Online */ }

#undef UE_API
