// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FAccountInfoRegistry
{
public:
	virtual ~FAccountInfoRegistry() = default;

	UE_API TSharedPtr<FAccountInfo> Find(FPlatformUserId PlatformUserId) const;
	UE_API TSharedPtr<FAccountInfo> Find(FAccountId AccountId) const;

	UE_API TArray<TSharedRef<FAccountInfo>> GetAllAccountInfo(TFunction<bool(const TSharedRef<FAccountInfo>&)> Predicate) const;

protected:
	UE_API TSharedPtr<FAccountInfo> FindNoLock(FPlatformUserId PlatformUserId) const;
	UE_API TSharedPtr<FAccountInfo> FindNoLock(FAccountId AccountId) const;

	UE_API virtual void DoRegister(const TSharedRef<FAccountInfo>& AccountInfo);
	UE_API virtual void DoUnregister(const TSharedRef<FAccountInfo>& AccountInfo);

	mutable FRWLock IndexLock;

private:
	TMap<FPlatformUserId, TSharedRef<FAccountInfo>> AuthDataByPlatformUserId;
	TMap<FAccountId, TSharedRef<FAccountInfo>> AuthDataByOnlineAccountIdHandle;
};

class FAuthCommon : public TOnlineComponent<IAuth>
{
public:
	using Super = IAuth;

	UE_API FAuthCommon(FOnlineServicesCommon& InServices);

	// IOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// IAuth
	UE_API virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthLogout> Logout(FAuthLogout::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthCreateAccount> CreateAccount(FAuthCreateAccount::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthLinkAccount> LinkAccount(FAuthLinkAccount::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthModifyAccountAttributes> ModifyAccountAttributes(FAuthModifyAccountAttributes::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthQueryVerifiedAuthTicket> QueryVerifiedAuthTicket(FAuthQueryVerifiedAuthTicket::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthCancelVerifiedAuthTicket> CancelVerifiedAuthTicket(FAuthCancelVerifiedAuthTicket::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthBeginVerifiedAuthSession> BeginVerifiedAuthSession(FAuthBeginVerifiedAuthSession::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAuthEndVerifiedAuthSession> EndVerifiedAuthSession(FAuthEndVerifiedAuthSession::Params&& Params) override;
	UE_API virtual TOnlineResult<FAuthGetLocalOnlineUserByOnlineAccountId> GetLocalOnlineUserByOnlineAccountId(FAuthGetLocalOnlineUserByOnlineAccountId::Params&& Params) const override;
	UE_API virtual TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> GetLocalOnlineUserByPlatformUserId(FAuthGetLocalOnlineUserByPlatformUserId::Params&& Params) const override;
	UE_API virtual TOnlineResult<FAuthGetAllLocalOnlineUsers> GetAllLocalOnlineUsers(FAuthGetAllLocalOnlineUsers::Params&& Params) const override;
	UE_API virtual TOnlineResult<FAuthGetLinkAccountContinuationId> GetLinkAccountContinuationId(FAuthGetLinkAccountContinuationId::Params&& Params) const override;
	UE_API virtual TOnlineResult<FAuthGetRelyingParty> GetRelyingParty(FAuthGetRelyingParty::Params&& Params) const override;
	UE_API virtual TOnlineEvent<void(const FAuthLoginStatusChanged&)> OnLoginStatusChanged() override;
	UE_API virtual TOnlineEvent<void(const FAuthPendingAuthExpiration&)> OnPendingAuthExpiration() override;
	UE_API virtual TOnlineEvent<void(const FAuthAccountAttributesChanged&)> OnAccountAttributesChanged() override;
	UE_API virtual bool IsLoggedIn(const FAccountId& AccountId) const override;
	UE_API virtual bool IsLoggedIn(const FPlatformUserId& PlatformUserId) const override;

protected:
	virtual const FAccountInfoRegistry& GetAccountInfoRegistry() const = 0;

	TOnlineEventCallable<void(const FAuthLoginStatusChanged&)> OnAuthLoginStatusChangedEvent;
	TOnlineEventCallable<void(const FAuthPendingAuthExpiration&)> OnAuthPendingAuthExpirationEvent;
	TOnlineEventCallable<void(const FAuthAccountAttributesChanged&)> OnAuthAccountAttributesChangedEvent;
};

/* UE::Online */ }

#undef UE_API
