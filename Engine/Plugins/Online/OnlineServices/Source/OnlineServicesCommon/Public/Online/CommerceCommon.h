// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Commerce.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FCommerceCommon : public TOnlineComponent<ICommerce>
{
public:
	using Super = ICommerce;

	UE_API FCommerceCommon(FOnlineServicesCommon& InServices);

	// IOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// ICommerce
	UE_API virtual TOnlineAsyncOpHandle<FCommerceQueryOffers> QueryOffers(FCommerceQueryOffers::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceQueryOffersById> QueryOffersById(FCommerceQueryOffersById::Params&& Params) override;
	UE_API virtual TOnlineResult<FCommerceGetOffers> GetOffers(FCommerceGetOffers::Params&& Params) override;
	UE_API virtual TOnlineResult<FCommerceGetOffersById> GetOffersById(FCommerceGetOffersById::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceShowStoreUI> ShowStoreUI(FCommerceShowStoreUI::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceCheckout> Checkout(FCommerceCheckout::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceQueryEntitlements> QueryEntitlements(FCommerceQueryEntitlements::Params&& Params) override;
	UE_API virtual TOnlineResult<FCommerceGetEntitlements> GetEntitlements(FCommerceGetEntitlements::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params) override;
	UE_API virtual TOnlineEvent<void(const FCommerceOnPurchaseComplete&)> OnPurchaseCompleted() override;

	//CommerceCommon
	UE_API FText GetFormattedPrice(uint64 Price, int32 DecimalPoint, FString CurrencyCode);

protected:
	TOnlineEventCallable<void(const FCommerceOnPurchaseComplete&)> OnPurchaseCompletedEvent;
};

/* UE::Online */
}

#undef UE_API
