// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Connectivity.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FConnectivityCommon : public TOnlineComponent<IConnectivity>
{
public:
	using Super = IConnectivity;

	UE_API FConnectivityCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// IConnectivity
	UE_API virtual TOnlineResult<FGetConnectionStatus> GetConnectionStatus(FGetConnectionStatus::Params&& Params) override;
	UE_API virtual TOnlineEvent<void(const FConnectionStatusChanged&)> OnConnectionStatusChanged() override;

protected:
	EOnlineServicesConnectionStatus ConnectionStatus = EOnlineServicesConnectionStatus::NotConnected;

	TOnlineEventCallable<void(const FConnectionStatusChanged&)> OnConnectionStatusChangedEvent;
};

/* UE::Online */}

#undef UE_API
