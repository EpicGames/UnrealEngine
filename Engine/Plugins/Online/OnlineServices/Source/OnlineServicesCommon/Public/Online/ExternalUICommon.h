// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/ExternalUI.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FExternalUICommon : public TOnlineComponent<IExternalUI>
{
public:
	using Super = IExternalUI;

	UE_API FExternalUICommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// IExternalUI
	UE_API virtual TOnlineAsyncOpHandle<FExternalUIShowLoginUI> ShowLoginUI(FExternalUIShowLoginUI::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params) override;

	UE_API virtual TOnlineEvent<void(const FExternalUIStatusChanged&)> OnExternalUIStatusChanged() override;
protected:
	TOnlineEventCallable<void(const FExternalUIStatusChanged&)> OnExternalUIStatusChangedEvent;
};

/* UE::Online */ }

#undef UE_API
