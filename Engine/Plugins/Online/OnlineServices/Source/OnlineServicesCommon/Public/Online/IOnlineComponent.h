// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


namespace UE::Online {

class IOnlineComponent
{
public:
	virtual ~IOnlineComponent() = default;
	// Called after component has been constructed. It is not safe to reference other components at this time
	virtual void Initialize() = 0;
	// Called after all components have been initialized
	virtual void PostInitialize() = 0;
	// Called whenever we need to reload data from config
	virtual void UpdateConfig() = 0;
	// Called every Tick
	virtual void Tick(float DeltaSeconds) = 0;
	// Called before any component has been shutdown
	virtual void PreShutdown() = 0;
	// Called right before the component is destroyed. It is not safe to reference any other components at this time
	virtual void Shutdown() = 0;
};

/* UE::Online */ }
