// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Leaderboards.h"
#include "Online/OnlineComponent.h"
#include "Online/Stats.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

enum class ELeaderboardUpdateMethod : uint8
{
	/** If current leaderboard score is better than the uploaded one, keep the current one */
	KeepBest,
	/** Leaderboard score is always replaced with uploaded value */
	Force
};

const TCHAR* LexToString(ELeaderboardUpdateMethod Value);
void LexFromString(ELeaderboardUpdateMethod& OutValue, const TCHAR* InStr);

enum ELeaderboardOrderMethod : uint8
{
	Ascending,
	Descending
};

const TCHAR* LexToString(ELeaderboardOrderMethod Value);
void LexFromString(ELeaderboardOrderMethod& OutValue, const TCHAR* InStr);

struct FLeaderboardDefinition
{
	/* The name of the leaderboard */
	FString Name;
	/* Corresponding leaderboard id on the platform if needed */
	int32 Id = 0;
	/* How the leaderboard score will be updated */
	ELeaderboardUpdateMethod UpdateMethod = ELeaderboardUpdateMethod::Force;
	/* How the leaderboard score will be ordered */
	ELeaderboardOrderMethod OrderMethod = ELeaderboardOrderMethod::Descending;
};

struct FLeaderboardsCommonConfig
{
	bool bIsTitleManaged = false;
	TArray<FLeaderboardDefinition> LeaderboardDefinitions;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FLeaderboardDefinition)
	ONLINE_STRUCT_FIELD(FLeaderboardDefinition, Name),
	ONLINE_STRUCT_FIELD(FLeaderboardDefinition, Id),
	ONLINE_STRUCT_FIELD(FLeaderboardDefinition, UpdateMethod),
	ONLINE_STRUCT_FIELD(FLeaderboardDefinition, OrderMethod)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLeaderboardsCommonConfig)
	ONLINE_STRUCT_FIELD(FLeaderboardsCommonConfig, bIsTitleManaged),
	ONLINE_STRUCT_FIELD(FLeaderboardsCommonConfig, LeaderboardDefinitions)
END_ONLINE_STRUCT_META()


/* Meta */ }

struct FWriteLeaderboardScores
{
	static constexpr TCHAR Name[] = TEXT("WriteLeaderboardScores");

	struct Params
	{
		FAccountId LocalAccountId;
		FString BoardName;
		uint64 Score;
	};

	struct Result
	{
	};
};

class FLeaderboardsCommon : public TOnlineComponent<ILeaderboards>
{
public:
	using Super = TOnlineComponent<ILeaderboards>;

	UE_API FLeaderboardsCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void Initialize() override;
	UE_API virtual void UpdateConfig() override;
	UE_API virtual void RegisterCommands() override;

	// ILeaderboards
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesForUsers> ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundRank> ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundUser> ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params) override;

	// Internal use only, for stats common implementation
	UE_API virtual TOnlineAsyncOpHandle<FWriteLeaderboardScores> WriteLeaderboardScores(FWriteLeaderboardScores::Params&& Params);

protected:
	UE_API void WriteLeaderboardsByStats(const FStatsUpdated& StatsUpdated);

	TMap<FString, FLeaderboardDefinition> LeaderboardDefinitions;
	FOnlineEventDelegateHandle StatEventHandle;
	bool bIsTitleManaged = false;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FWriteLeaderboardScores::Params)
	ONLINE_STRUCT_FIELD(FWriteLeaderboardScores::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FWriteLeaderboardScores::Params, BoardName),
	ONLINE_STRUCT_FIELD(FWriteLeaderboardScores::Params, Score)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FWriteLeaderboardScores::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef UE_API
