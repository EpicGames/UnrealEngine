// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineComponent.h"
#include "Online/LobbiesCommonTypes.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FLobbiesCommon : public TOnlineComponent<ILobbies>
{
public:
	using Super = ILobbies;

	UE_API FLobbiesCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void Initialize() override;
	UE_API virtual void RegisterCommands() override;

	// ILobbies
	UE_API virtual TOnlineAsyncOpHandle<FCreateLobby> CreateLobby(FCreateLobby::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FFindLobbies> FindLobbies(FFindLobbies::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FRestoreLobbies> RestoreLobbies(FRestoreLobbies::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FJoinLobby> JoinLobby(FJoinLobby::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FLeaveLobby> LeaveLobby(FLeaveLobby::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FInviteLobbyMember> InviteLobbyMember(FInviteLobbyMember::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FDeclineLobbyInvitation> DeclineLobbyInvitation(FDeclineLobbyInvitation::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FKickLobbyMember> KickLobbyMember(FKickLobbyMember::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FPromoteLobbyMember> PromoteLobbyMember(FPromoteLobbyMember::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FModifyLobbySchema> ModifyLobbySchema(FModifyLobbySchema::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FModifyLobbyJoinPolicy> ModifyLobbyJoinPolicy(FModifyLobbyJoinPolicy::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FModifyLobbyAttributes> ModifyLobbyAttributes(FModifyLobbyAttributes::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FModifyLobbyMemberAttributes> ModifyLobbyMemberAttributes(FModifyLobbyMemberAttributes::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetPresenceLobby> GetPresenceLobby(FGetPresenceLobby::Params&& Params) override;
	UE_API virtual TOnlineResult<FIsPresenceLobby> IsPresenceLobby(FIsPresenceLobby::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetJoinedLobbies> GetJoinedLobbies(FGetJoinedLobbies::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetReceivedInvitations> GetReceivedInvitations(FGetReceivedInvitations::Params&& Params) override;

	UE_API virtual TOnlineEvent<void(const FLobbyJoined&)> OnLobbyJoined() override;
	UE_API virtual TOnlineEvent<void(const FLobbyLeft&)> OnLobbyLeft() override;
	UE_API virtual TOnlineEvent<void(const FLobbyMemberJoined&)> OnLobbyMemberJoined() override;
	UE_API virtual TOnlineEvent<void(const FLobbyMemberLeft&)> OnLobbyMemberLeft() override;
	UE_API virtual TOnlineEvent<void(const FLobbyLeaderChanged&)> OnLobbyLeaderChanged() override;
	UE_API virtual TOnlineEvent<void(const FLobbySchemaChanged&)> OnLobbySchemaChanged() override;
	UE_API virtual TOnlineEvent<void(const FLobbyAttributesChanged&)> OnLobbyAttributesChanged() override;
	UE_API virtual TOnlineEvent<void(const FLobbyMemberAttributesChanged&)> OnLobbyMemberAttributesChanged() override;
	UE_API virtual TOnlineEvent<void(const FLobbyInvitationAdded&)> OnLobbyInvitationAdded() override;
	UE_API virtual TOnlineEvent<void(const FLobbyInvitationRemoved&)> OnLobbyInvitationRemoved() override;
	UE_API virtual TOnlineEvent<void(const FUILobbyJoinRequested&)> OnUILobbyJoinRequested() override;

protected:
#if LOBBIES_FUNCTIONAL_TEST_ENABLED
	UE_API TOnlineAsyncOpHandle<FFunctionalTestLobbies> FunctionalTest(FFunctionalTestLobbies::Params&& Params);
#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

	TMap<FAccountId, FLobbyId> PresenceLobbiesUserMap;
	TSharedRef<FSchemaRegistry> SchemaRegistry;
	FLobbyEvents LobbyEvents;
};

/* UE::Online */ }

#undef UE_API
