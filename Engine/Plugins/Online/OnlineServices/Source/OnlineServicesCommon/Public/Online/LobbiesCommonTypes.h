// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Online/Lobbies.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "Online/Schema.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

#define LOBBIES_FUNCTIONAL_TEST_ENABLED !UE_BUILD_SHIPPING

ONLINESERVICESCOMMON_API extern FSchemaId LobbyBaseSchemaId;
ONLINESERVICESCOMMON_API extern FSchemaCategoryId LobbySchemaCategoryId;
ONLINESERVICESCOMMON_API extern FSchemaCategoryId LobbyMemberSchemaCategoryId;

class FOnlineServicesCommon;

struct FLobbyEvents final
{
	TOnlineEventCallable<void(const FLobbyJoined&)> OnLobbyJoined;
	TOnlineEventCallable<void(const FLobbyLeft&)> OnLobbyLeft;
	TOnlineEventCallable<void(const FLobbyMemberJoined&)> OnLobbyMemberJoined;
	TOnlineEventCallable<void(const FLobbyMemberLeft&)> OnLobbyMemberLeft;
	TOnlineEventCallable<void(const FLobbyLeaderChanged&)> OnLobbyLeaderChanged;
	TOnlineEventCallable<void(const FLobbySchemaChanged&)> OnLobbySchemaChanged;
	TOnlineEventCallable<void(const FLobbyAttributesChanged&)> OnLobbyAttributesChanged;
	TOnlineEventCallable<void(const FLobbyMemberAttributesChanged&)> OnLobbyMemberAttributesChanged;
	TOnlineEventCallable<void(const FLobbyInvitationAdded&)> OnLobbyInvitationAdded;
	TOnlineEventCallable<void(const FLobbyInvitationRemoved&)> OnLobbyInvitationRemoved;
	TOnlineEventCallable<void(const FUILobbyJoinRequested&)> OnUILobbyJoinRequested;
};

// Todo: put this somewhere else
template <typename AwaitedType>
TFuture<TArray<AwaitedType>> WhenAll(TArray<TFuture<AwaitedType>>&& Futures)
{
	struct FWhenAllState
	{
		TArray<TFuture<AwaitedType>> Futures;
		TArray<AwaitedType> Results;
		TPromise<TArray<AwaitedType>> FinalPromise;
	};

	if (Futures.IsEmpty())
	{
		return MakeFulfilledPromise<TArray<AwaitedType>>().GetFuture();
	}
	else
	{
		TSharedRef<FWhenAllState> WhenAllState = MakeShared<FWhenAllState>();
		WhenAllState->Futures = MoveTemp(Futures);

		for (TFuture<AwaitedType>& Future : WhenAllState->Futures)
		{
			Future.Then([WhenAllState](TFuture<AwaitedType>&& AwaitedResult)
			{
				WhenAllState->Results.Emplace(MoveTempIfPossible(AwaitedResult.Get()));

				if (WhenAllState->Futures.Num() == WhenAllState->Results.Num())
				{
					WhenAllState->FinalPromise.EmplaceValue(MoveTemp(WhenAllState->Results));
				}
			});
		}

		return WhenAllState->FinalPromise.GetFuture();
	}
}

/**
* Accessor for using the public lobby attributes directly when committing schema changes.
*/
class FSchemaCategoryInstanceLobbySnapshotAccessor
{
public:
	FSchemaCategoryInstanceLobbySnapshotAccessor(TMap<FSchemaAttributeId, FSchemaVariant>& InClientSnapshot)
		: ClientSnapshot(InClientSnapshot)
	{
	}

	TMap<FSchemaAttributeId, FSchemaVariant>& GetMutableClientSnapshot()
	{
		return ClientSnapshot;
	}
private:

	TMap<FSchemaAttributeId, FSchemaVariant>& ClientSnapshot;
};

/**
* Alias for schema category instance used by lobby and lobby member data.
*/
using FLobbiesSchemaCategoryInstance = TSchemaCategoryInstance<FSchemaCategoryInstanceLobbySnapshotAccessor>;

/**
* Lobby data structure as seen by the client with additional fields for internal use.
*/
struct FLobbyInternal : public FLobby
{
	FLobbyInternal(const TSharedRef<const FSchemaRegistry>& SchemaRegistry)
		: FLobby()
		, SchemaData(FSchemaId(), LobbyBaseSchemaId, LobbySchemaCategoryId, SchemaRegistry, Attributes)
	{
	}

	FLobbiesSchemaCategoryInstance SchemaData;
};

/**
* Lobby member data structure as seen by the client with additional fields for internal use.
*/
struct FLobbyMemberInternal : public FLobbyMember
{
	FLobbyMemberInternal(const TSharedRef<const FSchemaRegistry>& SchemaRegistry, const FSchemaId& DerivedSchemaId = FSchemaId())
		: FLobbyMember()
		, SchemaData(DerivedSchemaId, LobbyBaseSchemaId, LobbyMemberSchemaCategoryId, SchemaRegistry, Attributes)
	{
	}

	FLobbiesSchemaCategoryInstance SchemaData;
};

struct FLobbyClientAttributeChanges
{
	/** New or changed attributes. */
	TMap<FSchemaAttributeId, FSchemaVariant> UpdatedAttributes;
	/** Attributes to be removed. */
	TSet<FSchemaAttributeId> RemovedAttributes;
};

struct FLobbyClientMemberAttributeChanges
{
	/** New or changed attributes. */
	TMap<FSchemaAttributeId, FSchemaVariant> UpdatedAttributes;
	/** Attributes to be removed. */
	TSet<FSchemaAttributeId> RemovedAttributes;
};

/**
 * Local changes to a lobby to be applied to the service.
 */
struct FLobbyClientChanges
{
	/** Local name for the lobby. */
	TOptional<FName> LocalName;
	/** Setting for new join policy. */
	TOptional<ELobbyJoinPolicy> JoinPolicy;
	/** Setting for lobby ownership change. */
	TOptional<FAccountId> OwnerAccountId;
	/** Setting for lobby schema change. */
	TOptional<FSchemaId> LobbySchema;
	/** Setting for changing lobby attributes. */
	TOptional<FLobbyClientAttributeChanges> Attributes;
	/** Setting for adding or updating the local user. */
	TOptional<FLobbyClientMemberAttributeChanges> MemberAttributes;
	/** Setting for removing the local user. */
	TOptional<ELobbyMemberLeaveReason> LocalUserLeaveReason;
	/** Setting for kicking the target lobby member. */
	TOptional<FAccountId> KickedTargetMember;
};

/**
 * Local changes to a lobby translated to the changes which will be applied to the service.
 */
struct FLobbyClientServiceChanges
{
	/** Setting for new join policy. */
	TOptional<ELobbyJoinPolicy> JoinPolicy;
	/** Setting for lobby ownership change. */
	TOptional<FAccountId> OwnerAccountId;
	/** Target member to be kicked. */
	TOptional<FAccountId> KickedTargetMember;
	/** Added or changed lobby attributes and their values. */
	TMap<FSchemaServiceAttributeId, FSchemaServiceAttributeData> UpdatedAttributes;
	/** Removed lobby attribute ids. */
	TSet<FSchemaServiceAttributeId> RemovedAttributes;
	/** Added or changed lobby member attributes and their values. */
	TMap<FSchemaServiceAttributeId, FSchemaServiceAttributeData> UpdatedMemberAttributes;
	/** Removed lobby member attribute ids. */
	TSet<FSchemaServiceAttributeId> RemovedMemberAttributes;
};

/**
* Full snapshot of lobby data excluding the individual member attributes
*/
struct FLobbyServiceSnapshot
{
	FAccountId OwnerAccountId;
	int32 MaxMembers;
	ELobbyJoinPolicy JoinPolicy;
	FSchemaServiceSnapshot SchemaServiceSnapshot;
	TSet<FAccountId> Members;
};

/**
* Full snapshot of a lobby member.
*/
struct FLobbyMemberServiceSnapshot
{
	FAccountId AccountId;
	FSchemaServiceSnapshot SchemaServiceSnapshot;
};

struct FLobbyClientDataPrepareServiceSnapshot
{
	struct Params
	{
		/** Full snapshot from the service. */
		FLobbyServiceSnapshot LobbySnapshot;
		/** Full snapshot for each member from the service. */
		TMap<FAccountId, FLobbyMemberServiceSnapshot> LobbyMemberSnapshots;
		/** Any leaving members notified by the service. */
		TMap<FAccountId, ELobbyMemberLeaveReason> LeaveReasons;
	};

	struct Result
	{
	};
};

struct FLobbyClientDataCommitServiceSnapshot
{
	struct Params
	{
		/** Notifier for signaling client events on data change. */
		FLobbyEvents* LobbyEvents = nullptr;
	};

	struct Result
	{
		/** List of local members leaving the lobby. */
		TArray<FAccountId> LeavingLocalMembers;
	};
};

struct FLobbyClientDataPrepareClientChanges
{
	struct Params
	{
		/** The local user who is making the changes. */
		FAccountId LocalAccountId;
		/** Changes to be applied to service data. */
		FLobbyClientChanges ClientChanges;
	};

	struct Result
	{
		/** Translated changes to be applied to the service. */
		FLobbyClientServiceChanges ServiceChanges;
	};
};

struct FLobbyClientDataCommitClientChanges
{
	struct Params
	{
		/** Notifier for signaling client events on data change. */
		FLobbyEvents* LobbyEvents = nullptr;
	};

	struct Result
	{
		/** List of local members leaving the lobby. */
		TArray<FAccountId> LeavingLocalMembers;
	};
};

/** Lobby data as seen by the client. */
struct FLobbyClientData final
{
public:
	UE_API FLobbyClientData(FLobbyId LobbyId, const TSharedRef<const FSchemaRegistry>& InSchemaRegistry);

	TSharedRef<const FLobby> GetPublicDataPtr() const { return InternalPublicData; }
	const FLobby& GetPublicData() const { return *InternalPublicData; }

	UE_API TSharedPtr<const FLobbyMemberInternal> GetMemberData(FAccountId MemberAccountId) const;

	/**
	 * Apply updated lobby data and generate changes.
	 * The LeaveReason provides context for members who have left since the most recent snapshot.
	 * Changing a lobby generates events for the local client.
	 */
	UE_API TOnlineResult<FLobbyClientDataPrepareServiceSnapshot> PrepareServiceSnapshot(FLobbyClientDataPrepareServiceSnapshot::Params&& Params) const;
	UE_API FLobbyClientDataCommitServiceSnapshot::Result CommitServiceSnapshot(FLobbyClientDataCommitServiceSnapshot::Params&& Params);

	UE_API TOnlineResult<FLobbyClientDataPrepareClientChanges> PrepareClientChanges(FLobbyClientDataPrepareClientChanges::Params&& Params) const;
	UE_API FLobbyClientDataCommitClientChanges::Result CommitClientChanges(FLobbyClientDataCommitClientChanges::Params&& Params);

private:
	UE_API void ResetPreparedChanges() const;
	UE_API void AddMember(const TSharedRef<FLobbyMemberInternal>& Member);
	UE_API void RemoveMember(const TSharedRef<FLobbyMemberInternal>& Member);

	struct FPreparedServiceChanges
	{
		/** Set when there is a change in lobby ownership. */
		TOptional<FAccountId> OwnerAccountId;
		/** Set when there is a change in max members. */
		TOptional<int32> MaxMembers;
		/** Set when there is a change in join policy. */
		TOptional<ELobbyJoinPolicy> JoinPolicy;
		/** Members who did not already have a local representation. */
		TArray<TSharedRef<FLobbyMemberInternal>> NewMemberDataStorage;
		/** Members who already exist in the lobby. */
		TArray<TSharedRef<FLobbyMemberInternal>> UpdatedMemberDataStorage;
		/** Any leaving members and their reason. */
		TArray<TPair<TSharedRef<FLobbyMemberInternal>, ELobbyMemberLeaveReason>> LeavingMembers;
	};

	struct FPreparedClientChanges
	{
		/** The local user associated with the change. */
		FAccountId LocalAccountId;
		/** Local name for the lobby. */
		TOptional<FName> LocalName;
		/** Setting for new join policy. */
		TOptional<ELobbyJoinPolicy> JoinPolicy;
		/** Setting for lobby ownership change. */
		TOptional<FAccountId> OwnerAccountId;
		/** Target member to be kicked. */
		TOptional<FAccountId> KickedTargetMember;
		/** Access to local member when joining and assigning initial attributes. */
		TSharedPtr<FLobbyMemberInternal> JoiningLocalMember;
		/** Access to local member when updating attributes. */
		TSharedPtr<FLobbyMemberInternal> UpdatedLocalMember;
		/** Reason to be set when the local member is leaving. */
		TOptional<ELobbyMemberLeaveReason> LocalUserLeaveReason;
		/** Whether there are waiting lobby attribute changes to be committed. */
		bool bLobbyAttributesChanged = false;
	};

	/**
	 * The schema registry is needed to initialize new members.
	 */
	TSharedRef<const FSchemaRegistry> SchemaRegistry;

	/**
	 * The shared pointer given back to user code with lobby operation results and notifications.
	 * Any changes to this data are immediately available to users.
	 */
	TSharedRef<FLobbyInternal> InternalPublicData;

	/** Mutable lobby member data storage. */
	TMap<FAccountId, TSharedRef<FLobbyMemberInternal>> MemberDataStorage;

	/**
	 * Keep track of which members are local to the client.
	 * When all local members have been removed all members will be removed.
	 */
	TSet<FAccountId> LocalMembers;

	/** Updates which have been applied but not yet committed. */
	mutable TOptional<FPreparedServiceChanges> PreparedServiceChanges;
	mutable TOptional<FPreparedClientChanges> PreparedClientChanges;
};

#if LOBBIES_FUNCTIONAL_TEST_ENABLED
struct FFunctionalTestLobbies
{
	static constexpr TCHAR Name[] = TEXT("FunctionalTest");

	struct Params
	{
	};

	struct Result
	{
	};
};
#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

namespace Meta {

#if LOBBIES_FUNCTIONAL_TEST_ENABLED
BEGIN_ONLINE_STRUCT_META(FFunctionalTestLobbies::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFunctionalTestLobbies::Result)
END_ONLINE_STRUCT_META()
#endif // LOBBIES_FUNCTIONAL_TEST_ENABLED

/* Meta */ }

/* UE::Online */ }

#undef UE_API
