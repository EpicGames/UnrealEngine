// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Online/Sessions.h"

class FNboSerializeFromBuffer;
class FNboSerializeToBuffer;
namespace UE::Online { class FSchemaVariant; }
namespace UE::Online { class FSessionCommon; }

/**
 * Serializes data in network byte order form into a buffer
 */
namespace UE::Online {

namespace NboSerializerCommonSvc {

/** SerializeToBuffer methods */

ONLINESERVICESCOMMON_API void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSchemaVariant& Data);
ONLINESERVICESCOMMON_API void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FCustomSessionSetting& CustomSessionSetting);
ONLINESERVICESCOMMON_API void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FCustomSessionSettingsMap& CustomSessionSettingsMap);
ONLINESERVICESCOMMON_API void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionSettings& SessionSettings);
ONLINESERVICESCOMMON_API void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionCommon& Session);

/** SerializeFromBuffer methods */

ONLINESERVICESCOMMON_API void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSchemaVariant& Data);
ONLINESERVICESCOMMON_API void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FCustomSessionSetting& CustomSessionSetting);
ONLINESERVICESCOMMON_API void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FCustomSessionSettingsMap& CustomSessionSettingsMap);
ONLINESERVICESCOMMON_API void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionSettings& SessionSettings);
ONLINESERVICESCOMMON_API void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionCommon& Session);

/* NboSerializerCommonSvc */ }

/* UE::Online */ }
