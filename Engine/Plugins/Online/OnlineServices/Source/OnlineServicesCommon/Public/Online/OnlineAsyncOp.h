// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Async.h"
#include "Containers/Ticker.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineResult.h"
#include "Online/OnlineServicesCommonDelegates.h"
#include "Online/OnlineServicesDelegates.h"
#include "Online/OnlineServicesLog.h"
#include "Online/OnlineServices.h"
#include "Online/OnlineTypeInfo.h"
#include "Traits/ElementType.h"
#include "Traits/IsContiguousContainer.h"

namespace UE::Online {

class FOnlineServicesCommon;

class FOnlineAsyncOp;
template <typename OpType> class TOnlineAsyncOp;
template <typename OpType, typename T> class TOnlineChainableAsyncOp;

enum class EOnlineAsyncExecutionPolicy : uint8
{
	RunOnGameThread,	// Run on the game thread, will execute immediately if we are already on the game thread
	RunOnNextTick,		// Run on the game thread next time we tick
	RunOnThreadPool,	// Run on a specified thread pool
	RunOnTaskGraph,		// Run on the task graph
	RunImmediately		// Call immediately, in the current thread
};

class FOnlineAsyncExecutionPolicy
{
public:
	FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy InExecutionPolicy)
		: ExecutionPolicy(InExecutionPolicy)
	{
	}

	static FOnlineAsyncExecutionPolicy RunOnGameThread() { return FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy::RunOnGameThread); }
	static FOnlineAsyncExecutionPolicy RunOnNextTick() { return FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy::RunOnNextTick); }
	static FOnlineAsyncExecutionPolicy RunOnThreadPool() { return FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy::RunOnThreadPool); } // TODO: allow thread pool to be specified
	static FOnlineAsyncExecutionPolicy RunOnTaskGraph() { return FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy::RunOnTaskGraph); }
	static FOnlineAsyncExecutionPolicy RunImmediately() { return FOnlineAsyncExecutionPolicy(EOnlineAsyncExecutionPolicy::RunImmediately); }

	const EOnlineAsyncExecutionPolicy& GetExecutionPolicy() const { return ExecutionPolicy; }

private:
	EOnlineAsyncExecutionPolicy ExecutionPolicy;
};

/* For use when we need to put a TOnlineResult in an object that needs to be default constructed such as a TPromise */
template <typename Result>
class TDefaultErrorResultInternal : public TResult<Result, FOnlineError>
{
public:
	using TResult<Result, FOnlineError>::TResult;

	TDefaultErrorResultInternal()
		: TResult<Result, FOnlineError>(Errors::Unknown())
	{
	}
};

template <typename OpType>
class TDefaultErrorResult : public TOnlineResult<OpType>
{
public:
	using TOnlineResult<OpType>::TOnlineResult;

	TDefaultErrorResult()
		: TOnlineResult<OpType>(Errors::Unknown())
	{
	}
};

template <typename TResultType>
struct TContinuationResult
{
public:
	static TContinuationResult Repeat() { return TContinuationResult<TResultType>(); }
	static TContinuationResult Complete(TResultType&& InResult) { return TContinuationResult(MoveTempIfPossible(InResult)); }
	bool IsComplete() const { return Result.IsSet(); }
	TResultType& GetResult() { return Result.GetValue(); }
	const TResultType& GetResult() const { return Result.GetValue(); }
private:

	TContinuationResult()
	{
	}
	TContinuationResult(TResultType&& InResult) : Result(InResult)
	{
	}
	TOptional<TResultType> Result;
};

template <>
struct TContinuationResult<void>
{
public:
	static TContinuationResult Repeat() { return TContinuationResult<void>(); }
	static TContinuationResult Complete() { return TContinuationResult(FIsCompleteTag()); }
	bool IsComplete() const { return bIsComplete; }
private:
	struct FIsCompleteTag {};

	TContinuationResult()
	{
	}
	TContinuationResult(const FIsCompleteTag&) : bIsComplete(true)
	{
	}
	bool bIsComplete = false;
};

template <typename T>
struct TContinuationResultType
{
	using ResultType = T;
};

template <typename T>
struct TContinuationResultType<TContinuationResult<T>>
{
	using ResultType = T;
};

template <typename T>
using TContinuationResultType_T = typename TContinuationResultType<T>::ResultType;

template <typename T>
FString ToLogString(const TDefaultErrorResultInternal<T>& Result)
{
	if (Result.IsOk())
	{
		return ToLogString(Result.GetOkValue());
	}
	else
	{
		return ToLogString(Result.GetErrorValue());
	}
}

namespace Private
{

class FOnlineOperationData // Map of (TypeName,Key)->(data of any Type)
{
public:
	template <typename T>
	void Set(const FString& Key, T&& InData)
	{
		Data.Add(FOperationDataKey{ TOnlineTypeInfo<T>::GetTypeName(), Key }, MakeUnique<TData<T>>(MoveTemp(InData)));
	}

	template <typename T>
	void Set(const FString& Key, const T& InData)
	{
		Data.Add(FOperationDataKey{ TOnlineTypeInfo<T>::GetTypeName(), Key }, MakeUnique<TData<T>>(InData));
	}

	template <typename T>
	const T* Get(const FString& Key) const
	{
		if (auto Value = Data.Find(FOperationDataKey{ TOnlineTypeInfo<T>::GetTypeName(), Key }))
		{
			return static_cast<const T*>((*Value)->GetData());
		}

		return nullptr;
	}

	template <typename T>
	T* Get(const FString& Key)
	{
		if (auto Value = Data.Find(FOperationDataKey{ TOnlineTypeInfo<T>::GetTypeName(), Key }))
		{
			return static_cast<T*>((*Value)->GetData());
		}

		return nullptr;
	}

	struct FOperationDataKey
	{
		FOnlineTypeName TypeName;
		FString Key;

		bool operator==(const FOperationDataKey& Other) const
		{
			return TypeName == Other.TypeName && Key == Other.Key;
		}
	};

private:
	class IData
	{
	public:
		virtual ~IData() = default;
		virtual FOnlineTypeName GetTypeName() = 0;
		virtual void* GetData() = 0;

		template <typename T>
		const T* Get()
		{
			if (GetTypeName() == TOnlineTypeInfo<T>::GetTypeName())
			{
				return static_cast<T*>(GetData());
			}

			return nullptr;
		}
	};

	template <typename T>
	class TData : public IData
	{
	public:
		TData(const T& InData)
			: Data(InData)
		{
		}

		TData(T&& InData)
			: Data(MoveTemp(InData))
		{
		}

		virtual FOnlineTypeName GetTypeName() override
		{
			return  TOnlineTypeInfo<T>::GetTypeName();
		}

		virtual void* GetData() override
		{
			return &Data;
		}

	private:
		T Data;
	};

	friend uint32 GetTypeHash(const FOperationDataKey& Key);

	TMap<FOperationDataKey, TUniquePtr<IData>> Data;
};

inline uint32 GetTypeHash(const FOnlineOperationData::FOperationDataKey& Key)
{
	using ::GetTypeHash;
	return HashCombine(GetTypeHash(Key.TypeName), GetTypeHash(Key.Key));
}

template <typename... Params>
struct TOnlineAsyncOpCallableTraitsHelper2
{
};

template <typename TResultType, typename OpType>
struct TOnlineAsyncOpCallableTraitsHelper2<TResultType, OpType&>
{
	using ParamType = void;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = false;
	static constexpr bool bRequiresPromise = false;
};

template <typename TResultType, typename OpType, typename TParamType>
struct TOnlineAsyncOpCallableTraitsHelper2<TResultType, OpType&, TParamType>
{
	using ParamType = TParamType;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = false;
	static constexpr bool bRequiresPromise = false;
};

template <typename TResultType, typename OpType>
struct TOnlineAsyncOpCallableTraitsHelper2<TFuture<TResultType>, OpType&>
{
	using ParamType = void;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = true;
	static constexpr bool bRequiresPromise = false;
};

template <typename TResultType, typename OpType, typename TParamType>
struct TOnlineAsyncOpCallableTraitsHelper2<TFuture<TResultType>, OpType&, TParamType>
{
	using ParamType = TParamType;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = true;
	static constexpr bool bRequiresPromise = false;
};

template <typename TResultType, typename OpType>
struct TOnlineAsyncOpCallableTraitsHelper2<void, OpType&, TPromise<TResultType>&&>
{
	using ParamType = void;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = true;
	static constexpr bool bRequiresPromise = true;
};

template <typename TResultType, typename OpType, typename TParamType>
struct TOnlineAsyncOpCallableTraitsHelper2<void, OpType&, TParamType, TPromise<TResultType>&&>
{
	using ParamType = TParamType;
	using ResultType = TResultType;
	static constexpr bool bAsyncResult = true;
	static constexpr bool bRequiresPromise = true;
};

template <typename CallableType>
struct TOnlineAsyncOpCallableTraitsHelper
{
};

template <typename ReturnType, typename... ParamTypes>
struct TOnlineAsyncOpCallableTraitsHelper<ReturnType(ParamTypes...)>
	: public TOnlineAsyncOpCallableTraitsHelper2<ReturnType, ParamTypes...>
{
};

template <typename ReturnType, typename ObjectType, typename... ParamTypes>
struct TOnlineAsyncOpCallableTraitsHelper<ReturnType(ObjectType::*)(ParamTypes...)>
	: public TOnlineAsyncOpCallableTraitsHelper2<ReturnType, ParamTypes...>
{
};

template <typename ReturnType, typename ObjectType, typename... ParamTypes>
struct TOnlineAsyncOpCallableTraitsHelper<ReturnType(ObjectType::*)(ParamTypes...) const>
	: public TOnlineAsyncOpCallableTraitsHelper2<ReturnType, ParamTypes...>
{
};

template <typename CallableType, typename = void>
struct TOnlineAsyncOpCallableTraits
{
};

// function pointers
template <typename CallableFunction>
struct TOnlineAsyncOpCallableTraits<CallableFunction, std::enable_if_t<std::is_function_v<std::remove_pointer_t<CallableFunction>>, void>>
	: public TOnlineAsyncOpCallableTraitsHelper<CallableFunction>
{
};

// lambdas, TFunction, functor objects (anything with operator())
template <typename CallableObject>
struct TOnlineAsyncOpCallableTraits<CallableObject, std::enable_if_t<!std::is_function_v<std::remove_pointer_t<CallableObject>>, void>>
	: public TOnlineAsyncOpCallableTraitsHelper<decltype(&std::remove_reference_t<CallableObject>::operator())>
{
};


class IStep
{
public:
	virtual ~IStep() = default;
	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const = 0;
	virtual void Execute() = 0;
};

template <typename ResultType>
class TStep : public IStep
{
public:
	TStep(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: ExecutionPolicy(MoveTemp(InExecutionPolicy))
	{
	}

	~TStep()
	{
		if (bResultSet)
		{
			DestructItem(Result.GetTypedPtr());
		}
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
			if (PinnedOperation)
			{
				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
				{
					TPromise<ResultType> Promise;
					// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
					Promise.GetFuture()
					.Next([this, WeakOperation](const ResultType& Value)
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							EmplaceResult(Value);
							PinnedOperation2->ExecuteNextStep();
						}
					});

					Callable(*PinnedOperation, MoveTempIfPossible(LastResult), MoveTemp(Promise));
				}
				else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					Callable(*PinnedOperation, MoveTempIfPossible(LastResult))
					.Next([this, WeakOperation](const ResultType& Value)
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							EmplaceResult(Value);
							PinnedOperation2->ExecuteNextStep();
						}
					});
				}
				else
				{
					EmplaceResult(Callable(*PinnedOperation, MoveTempIfPossible(LastResult)));
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}

	template <typename OpType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), Callable = MoveTemp(InCallable)]() mutable
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
			if (PinnedOperation)
			{
				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
				{
					TPromise<ResultType> Promise;
					// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
					Promise.GetFuture()
					.Next([this, WeakOperation](ResultType&& Value)
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							EmplaceResult(Forward<ResultType>(Value));
							PinnedOperation2->ExecuteNextStep();
						}
					});

					Callable(*PinnedOperation, MoveTemp(Promise));
				}
				else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					Callable(*PinnedOperation)
					.Next([this, WeakOperation](ResultType&& Value)
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							EmplaceResult(Forward<ResultType>(Value));
							PinnedOperation2->ExecuteNextStep();
						}
					});
				}
				else
				{
					EmplaceResult(Callable(*PinnedOperation));
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}

	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const override
	{
		return ExecutionPolicy;
	}

	virtual void Execute() override
	{
		check(ExecFunction)
		ExecFunction();
	}


	ResultType& GetResultRef()
	{
		return *Result.GetTypedPtr();
	}

private:
	template<typename... ArgTypes>
	void EmplaceResult(ArgTypes&&... Args)
	{
		check(!bResultSet)
		new(Result.GetTypedPtr()) ResultType(Forward<ArgTypes>(Args)...);
		bResultSet = true;
	}

	FOnlineAsyncExecutionPolicy ExecutionPolicy;
	TUniqueFunction<void()> ExecFunction;
	TTypeCompatibleBytes<ResultType> Result;
	bool bResultSet = false;
};


template <>
class TStep<void> : public IStep
{
public:
	TStep(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: ExecutionPolicy(MoveTemp(InExecutionPolicy))
	{
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
			if (PinnedOperation)
			{
				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					Callable(*PinnedOperation, MoveTempIfPossible(LastResult))
					.Next([this, WeakOperation]()
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							PinnedOperation2->ExecuteNextStep();
						}
					});
				}
				else
				{
					Callable(*PinnedOperation, MoveTempIfPossible(LastResult));
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}

	template <typename OpType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), Callable = MoveTemp(InCallable)]() mutable
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
			if (PinnedOperation)
			{
				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					Callable(*PinnedOperation)
					.Next([this, WeakOperation]()
					{
						if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
						{
							PinnedOperation2->ExecuteNextStep();
						}
					});
				}
				else
				{
					Callable(*PinnedOperation);
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}

	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const override
	{
		return ExecutionPolicy;
	}

	virtual void Execute() override
	{
		check(ExecFunction)
		ExecFunction();
	}

private:
	FOnlineAsyncExecutionPolicy ExecutionPolicy;
	TUniqueFunction<void()> ExecFunction;
};

template <typename ResultType>
class TStep<TContinuationResult<ResultType>> : public IStep
{
	public:
	TStep(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: ExecutionPolicy(MoveTemp(InExecutionPolicy))
	{
	}

	~TStep()
	{
		if (bResultSet)
		{
			DestructItem(Result.GetTypedPtr());
		}
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
			{
				TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
				if (PinnedOperation)
				{
					if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
					{
						TPromise<TContinuationResult<ResultType>> Promise;
						// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
						Promise.GetFuture()
						.Next([this, WeakOperation](const TContinuationResult<ResultType>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									EmplaceResult(Value);
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
						Callable(*PinnedOperation, MoveTempIfPossible(LastResult), MoveTemp(Promise));
					}
					else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
					{
						Callable(*PinnedOperation, LastResult)
						.Next([this, WeakOperation](const TContinuationResult<ResultType>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									EmplaceResult(Value.GetResult());
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
					}
					else
					{
						const TContinuationResult<ResultType>& Value = Callable(*PinnedOperation, LastResult);
						if (!Value.IsComplete())
						{
							PinnedOperation->ExecuteRepeatStep();
						}
						else
						{
							EmplaceResult(Value.GetResult());
							PinnedOperation->ExecuteNextStep();
						}
					}
				}
			};
	}

	template <typename OpType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), Callable = MoveTemp(InCallable)]() mutable
			{
				TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
				if (PinnedOperation)
				{
					if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
					{
						TPromise<TContinuationResult<ResultType>> Promise;
						// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
						Promise.GetFuture()
						.Next([this, WeakOperation](const TContinuationResult<ResultType>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									EmplaceResult(Value);
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
						Callable(*PinnedOperation, MoveTemp(Promise));
					}
					else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
					{
						Callable(*PinnedOperation)
						.Next([this, WeakOperation](const TContinuationResult<ResultType>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									EmplaceResult(Value);
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
					}
					else
					{
						const TContinuationResult<ResultType>& Value = Callable(*PinnedOperation);
						if (!Value.IsComplete())
						{
							PinnedOperation->ExecuteRepeatStep();
						}
						else
						{
							EmplaceResult(Value.GetResult());
							PinnedOperation->ExecuteNextStep();
						}
					}
				}
			};
	}

	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const override
	{
		return ExecutionPolicy;
	}

	virtual void Execute() override
	{
		// Repeating continuations will loop until finished. Not safe to run on game thread.
		check(ExecutionPolicy.GetExecutionPolicy() != EOnlineAsyncExecutionPolicy::RunOnGameThread)
			check(ExecFunction)
			ExecFunction();
	}


	ResultType& GetResultRef()
	{
		return *Result.GetTypedPtr();
	}

private:
	template<typename... ArgTypes>
	void EmplaceResult(ArgTypes&&... Args)
	{
		check(!bResultSet)
			new(Result.GetTypedPtr()) ResultType(Forward<ArgTypes>(Args)...);
		bResultSet = true;
	}

	FOnlineAsyncExecutionPolicy ExecutionPolicy;
	TUniqueFunction<void()> ExecFunction;
	TTypeCompatibleBytes<ResultType> Result;
	bool bResultSet = false;
};

template <>
class TStep<TContinuationResult<void>> : public IStep
{
public:
	TStep(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: ExecutionPolicy(MoveTemp(InExecutionPolicy))
	{
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
			{
				TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
				if (PinnedOperation)
				{
					if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
					{
						Callable(*PinnedOperation, LastResult)
						.Next([this, WeakOperation](const TContinuationResult<void>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
					}
					else
					{
						const TContinuationResult<void>& Value = Callable(*PinnedOperation, LastResult);
						if (!Value.IsComplete())
						{
							PinnedOperation->ExecuteRepeatStep();
						}
						else
						{
							PinnedOperation->ExecuteNextStep();
						}
					}
				}
			};
	}

	template <typename OpType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, CallableType&& InCallable)
	{
		ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), Callable = MoveTemp(InCallable)]() mutable
			{
				TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
				if (PinnedOperation)
				{
					if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
					{
						Callable(*PinnedOperation)
						.Next([this, WeakOperation](const TContinuationResult<void>& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if (!Value.IsComplete())
								{
									PinnedOperation2->ExecuteRepeatStep();
								}
								else
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}

						});
					}
					else
					{
						const TContinuationResult<void>& Value = Callable(*PinnedOperation);
						if (!Value.IsComplete())
						{
							PinnedOperation->ExecuteRepeatStep();
						}
						else
						{
							PinnedOperation->ExecuteNextStep();
						}
					}
				}
			};
	}

	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const override
	{
		return ExecutionPolicy;
	}

	virtual void Execute() override
	{
		// Repeating continuations will loop until finished. Not safe to run on game thread.
		check(ExecutionPolicy.GetExecutionPolicy() != EOnlineAsyncExecutionPolicy::RunOnGameThread)
		check(ExecFunction)
		ExecFunction();
	}

private:
	FOnlineAsyncExecutionPolicy ExecutionPolicy;
	TUniqueFunction<void()> ExecFunction;
};



template <typename ResultElementType>
class TForEachStepBase : public IStep
{
public:
	using ResultType = TArray<ResultElementType>;

	TForEachStepBase(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: ExecutionPolicy(MoveTemp(InExecutionPolicy))
	{
	}

	~TForEachStepBase()
	{
		if (bResultConstructed)
		{
			DestructItem(Result.GetTypedPtr());
		}
	}

	virtual const FOnlineAsyncExecutionPolicy& GetExecutionPolicy() const override
	{
		return ExecutionPolicy;
	}

	virtual void Execute() override
	{
		check(ExecFunction)
		ExecFunction();
	}

	ResultType& GetResultRef()
	{
		return *Result.GetTypedPtr();
	}

protected:
	FOnlineAsyncExecutionPolicy ExecutionPolicy;
	TUniqueFunction<void()> ExecFunction;
	TTypeCompatibleBytes<ResultType> Result;
	std::atomic<int32> NumResultsSet = 0;
	bool bResultConstructed = false;
};

template <typename ResultElementType>
class TForEachStep : public TForEachStepBase<ResultElementType>
{
public:
	using typename TForEachStepBase<ResultElementType>::ResultType;

	TForEachStep(FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: TForEachStepBase<ResultElementType>(MoveTemp(InExecutionPolicy))
	{
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		this->ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin();
			if (PinnedOperation)
			{
				int32 ResultIndex = 0;
				new(this->Result.GetTypedPtr()) ResultType();
				this->GetResultRef().SetNum(LastResult.Num());
				this->bResultConstructed = true;

				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
				{
					for (auto&& LastResultElement : LastResult)
					{
						TPromise<ResultElementType> Promise;
						// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
						Promise.GetFuture()
						.Next([this, WeakOperation, ThisResultIndex = ResultIndex++](const ResultElementType& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								this->GetResultRef()[ThisResultIndex] = Value;
								if (++this->NumResultsSet == this->GetResultRef().Num())
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});

						Callable(*PinnedOperation, MoveTempIfPossible(LastResultElement), MoveTemp(Promise));
					}
				}
				else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					for (auto&& LastResultElement : LastResult)
					{
						Callable(*PinnedOperation, MoveTempIfPossible(LastResultElement))
						.Next([this, WeakOperation, ThisResultIndex = ResultIndex++](const ResultElementType& Value)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								this->GetResultRef()[ThisResultIndex] = Value;
								if (++this->NumResultsSet == this->GetResultRef().Num())
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
					}
				}
				else
				{
					for (auto&& LastResultElement : LastResult)
					{
						this->GetResultRef()[ResultIndex++] = Callable(*PinnedOperation, MoveTempIfPossible(LastResultElement));
					}
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}
};

namespace Private
{
// Primary template with a dependent static_assert that only fails when instantiated
template <typename ResultType, typename = void>
struct TForEachNResultElementType
{
	// Make the static_assert dependent on the template parameter to delay evaluation
	static_assert(sizeof(ResultType) == 0, "Unsupported result type for TForEachNResultElementType");
};

template <typename ResultType>
struct TForEachNResultElementType<ResultType, std::enable_if_t<TIsContiguousContainer<ResultType>::Value, void >>
{
	using ResultElementType = TElementType_T<ResultType>;
};

template <typename ResultType>
struct TForEachNResultElementType<ResultType, std::enable_if_t<!TIsContiguousContainer<ResultType>::Value, void>>
{
	using ResultElementType = ResultType;
};

template <typename ResultType>
using TForEachNResultElementType_T = typename TForEachNResultElementType<ResultType>::ResultElementType;

} // Private

template <typename CallableResultType, typename ResultElementType = Private::TForEachNResultElementType_T<CallableResultType>>
class TForEachNStep : public TForEachStepBase<ResultElementType>
{
public:
	using typename TForEachStepBase<ResultElementType>::ResultType;

	TForEachNStep(int32 InBatchSize, FOnlineAsyncExecutionPolicy&& InExecutionPolicy)
		: TForEachStepBase<ResultElementType>(MoveTemp(InExecutionPolicy))
		, BatchSize(InBatchSize)
	{
	}

	template <typename OpType, typename LastResultType, typename CallableType>
	void SetExecFunction(TOnlineAsyncOp<OpType>& InOperation, LastResultType&& InLastResult, CallableType&& InCallable)
	{
		this->ExecFunction = [this, WeakOperation = TWeakPtr<TOnlineAsyncOp<OpType>>(InOperation.AsShared()), &LastResult = InLastResult, Callable = MoveTemp(InCallable)]() mutable
		{
			if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation = WeakOperation.Pin())
			{
				new(this->Result.GetTypedPtr()) ResultType();
				const int32 ResultArraySize = TIsContiguousContainer<CallableResultType>::Value ? LastResult.Num()
					: (LastResult.Num() + BatchSize - 1) / BatchSize;
				const int32 OutputBatchSize = TIsContiguousContainer<CallableResultType>::Value ? BatchSize : 1;
				this->GetResultRef().SetNumUninitialized(ResultArraySize);
				this->bResultConstructed = true;

				if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bRequiresPromise)
				{
					for (int32 Index = 0, OutputIndex = 0; Index < LastResult.Num(); Index += BatchSize, OutputIndex += OutputBatchSize)
					{
						int32 NumElementsInBatch = BatchSize;
						int32 NumOutputElementsInBatch = BatchSize;
						if (Index + BatchSize > LastResult.Num())
						{
							NumElementsInBatch = LastResult.Num() - Index;
							if constexpr (TIsContiguousContainer<CallableResultType>::Value)
							{
								NumOutputElementsInBatch = NumElementsInBatch;
							}
						}

						TPromise<CallableResultType> Promise;
						// set promise continuation before calling the callable so that we will complete the step as soon as the value is set
						Promise.GetFuture()
						.Next([this, WeakOperation, Index, OutputIndex, NumElementsInBatch, NumOutputElementsInBatch](CallableResultType&& BatchResult)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if constexpr (TIsContiguousContainer<CallableResultType>::Value)
								{
									if (NumElementsInBatch != GetNum(BatchResult))
									{
										PinnedOperation2->SetError(Online::Errors::InvalidResults());
										return;
									}

									MoveConstructItems(&this->GetResultRef()[Index], GetData(BatchResult), GetNum(BatchResult));
								}
								else
								{
									::new (&this->GetResultRef()[OutputIndex]) CallableResultType(MoveTemp(BatchResult));
								}

								if ((this->NumResultsSet += NumOutputElementsInBatch) == this->GetResultRef().Num())
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});

						Callable(*PinnedOperation, MakeArrayView(&LastResult[Index], NumElementsInBatch), MoveTemp(Promise));
					}
				}
				else if constexpr (TOnlineAsyncOpCallableTraits<CallableType>::bAsyncResult)
				{
					for (int32 Index = 0, OutputIndex = 0; Index < LastResult.Num(); Index += BatchSize, OutputIndex += OutputBatchSize)
					{
						int32 NumElementsInBatch = BatchSize;
						int32 NumOutputElementsInBatch = OutputBatchSize;
						if (Index + BatchSize > LastResult.Num())
						{
							NumElementsInBatch = LastResult.Num() - Index;
							if constexpr (TIsContiguousContainer<CallableResultType>::Value)
							{
								NumOutputElementsInBatch = NumElementsInBatch;
							}
						}

						Callable(*PinnedOperation, MakeArrayView(&LastResult[Index], NumElementsInBatch))
						.Next([this, WeakOperation, Index, OutputIndex, NumElementsInBatch, NumOutputElementsInBatch](CallableResultType&& BatchResult)
						{
							if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOperation2 = WeakOperation.Pin())
							{
								if constexpr (TIsContiguousContainer<CallableResultType>::Value)
								{
									if (NumElementsInBatch != GetNum(BatchResult))
									{
										PinnedOperation2->SetError(Online::Errors::InvalidResults());
										return;
									}

									MoveConstructItems(&this->GetResultRef()[OutputIndex], GetData(BatchResult), GetNum(BatchResult));
								}
								else
								{
									::new (&this->GetResultRef()[OutputIndex]) CallableResultType(MoveTemp(BatchResult));
								}

								if ((this->NumResultsSet += NumOutputElementsInBatch) == this->GetResultRef().Num())
								{
									PinnedOperation2->ExecuteNextStep();
								}
							}
						});
					}
				}
				else
				{
					for (int32 Index = 0, OutputIndex = 0; Index < LastResult.Num(); Index += BatchSize, OutputIndex += OutputBatchSize)
					{
						int32 NumElementsInBatch = BatchSize;
						if (Index + BatchSize > LastResult.Num())
						{
							NumElementsInBatch = LastResult.Num() - Index;
						}

						if constexpr (TIsContiguousContainer<CallableResultType>::Value)
						{
							CallableResultType BatchResult = Callable(*PinnedOperation, MakeArrayView(&LastResult[Index], NumElementsInBatch));
							if (NumElementsInBatch != GetNum(BatchResult))
							{
								PinnedOperation->SetError(Online::Errors::InvalidResults());
								break;
							}
							MoveConstructItems(&this->GetResultRef()[OutputIndex], GetData(BatchResult), GetNum(BatchResult));
						}
						else
						{
							::new (&this->GetResultRef()[OutputIndex]) CallableResultType(Callable(*PinnedOperation, MakeArrayView(&LastResult[Index], NumElementsInBatch)));
						}
					}
					PinnedOperation->ExecuteNextStep();
				}
			}
		};
	}

private:
	int32 BatchSize;
};

// Provides Then continuation for both TOnlineAsyncOp and TOnlineChainableAsyncOp
template <typename Outer, typename OpType, typename LastResultType>
class TOnlineAsyncOpBase
{
public:
	TOnlineAsyncOpBase(LastResultType& InLastResult)
		: LastResult(InLastResult)
	{
	}

	// Callable can take one of the following forms, where the second form is used when an asynchronous
	//     call can set the promise with a value that is only valid for the duration of the Callable call
	//   ResultType(AsyncOp, LastResult)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResult is a LastResultType or const LastResultType&
	//     ResultType is any type, or a TFuture to allow for an asynchronous result
	//   void(AsyncOp, LastResult, TPromise<ResultType>&&)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResult is a LastResultType or const LastResultType&
	//     ResultType is any non-void type
	// For either ResultType can be TContinuationResult<T>. 
	//   Retuning TContinuationResult<T>::Repeat() will cause the step to repeat.
	//   Returning TContinuationResult<T>::Complete(T) will complete the step and allow the task to proceed to the next.
	//	 The type T in TContinuationResult<T>::Complete(T) will be the LastResult type for a subsequent step.
	template <typename CallableType>
	auto Then(CallableType&& Callable, FOnlineAsyncExecutionPolicy ExecutionPolicy = FOnlineAsyncExecutionPolicy::RunOnGameThread());

	// Callable can take one of the following forms, where the second form is used when an asynchronous
	//     call can set the promise with a value that is only valid for the duration of the Callable call.
	//     This can be used when the previous continuation returned a TArray. The Callable will be called
	//     once per element in the previous result. The results will be combined into a TArray
	//   ResultType(AsyncOp, LastResultElement)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResultElement is a LastResultElementType or const LastResultElementType&
	//     ResultType is any type, or a TFuture to allow for an asynchronous result
	//   void(AsyncOp, LastResultElement, TPromise<ResultType>&&)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResultElement is a LastResultType or const LastResultType&
	//     ResultType is any non-void type


	template <typename CallableType> // previous value was TArray
	auto ForEach(CallableType&& Callable, FOnlineAsyncExecutionPolicy ExecutionPolicy = FOnlineAsyncExecutionPolicy::RunOnGameThread());

	// Callable can take one of the following forms, where the second form is used when an asynchronous
	//     call can set the promise with a value that is only valid for the duration of the Callable call.
	//     This can be used when the previous continuation returned a TArray. The Callable will be called
	//     once per BatchSize elements in the previous result. The results will be combined into a TArray
	//   ResultType(AsyncOp, LastResultElements)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResultElements is a TArray<LastResultElementType> or const TArray<LastResultElementType>&
	//     ResultType is any type, or a TFuture to allow for an asynchronous result
	//   void(AsyncOp, LastResultElements, TPromise<ResultType>&&)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     LastResultElements is a TArray<LastResultElementType> or const TArray<LastResultElementType>&
	//     ResultType is any non-void type
	template <typename CallableType> // previous value was TArray
	auto ForEachN(int32 BatchSize, CallableType&& Callable, FOnlineAsyncExecutionPolicy ExecutionPolicy = FOnlineAsyncExecutionPolicy::RunOnGameThread());

protected:
	LastResultType& LastResult;
};

template <typename Outer, typename OpType>
class TOnlineAsyncOpBase<Outer, OpType, void>
{
public:
	TOnlineAsyncOpBase() {}

	// Callable can take one of the following forms, where the second form is used when an asynchronous
	//     call can set the promise with a value that is only valid for the duration of the Callable call
	//   ResultType(AsyncOp)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     ResultType is any type, or a TFuture to allow for an asynchronous result
	//   void(AsyncOp, TPromise<ResultType>&&)
	//     AsyncOp is a FOnlineAsyncOp& or TOnlineAsyncOp<OpType>&
	//     ResultType is any non-void type
	template <typename CallableType>
	auto Then(CallableType&& Callable, FOnlineAsyncExecutionPolicy ExecutionPolicy = FOnlineAsyncExecutionPolicy::RunOnGameThread());

};


/* Private */ }

template <typename OpType, typename T>
class TOnlineChainableAsyncOp : public Private::TOnlineAsyncOpBase<TOnlineChainableAsyncOp<OpType, T>, OpType, TContinuationResultType_T<T>>
{
public:
	using Super = Private::TOnlineAsyncOpBase<TOnlineChainableAsyncOp<OpType, T>, OpType, TContinuationResultType_T<T>>;

	TOnlineChainableAsyncOp(TOnlineAsyncOp<OpType>& InOwningOperation, std::enable_if_t<!std::is_same_v<TContinuationResultType_T<T>, void>, TContinuationResultType_T<T>>& InLastResult)
		: Super(InLastResult)
		, OwningOperation(InOwningOperation)
	{
	}

	TOnlineChainableAsyncOp(TOnlineChainableAsyncOp&& Other)
		: OwningOperation(Other.OwningOperation)
	{
	}

	TOnlineChainableAsyncOp& operator=(TOnlineChainableAsyncOp&& Other)
	{
		check(&OwningOperation == &Other.OwningOperation); // Can't reassign this
		Super::operator=(MoveTemp(Other));
		return *this;
	}

	template <typename QueueType>
	void Enqueue(QueueType& Queue, double DelayInSeconds = 0.0)
	{
		static_assert(std::is_same_v<T, void>, "Continuation result discarded. Continuation prior to calling Enqueue must have a void or TFuture<void> return type.");
		OwningOperation.Enqueue(Queue, DelayInSeconds);
	}

	TOnlineAsyncOp<OpType>& GetOwningOperation()
	{
		return OwningOperation;
	}

protected:
	TOnlineAsyncOp<OpType>& OwningOperation;
};

template <typename OpType>
class TOnlineChainableAsyncOp<OpType, void> : public Private::TOnlineAsyncOpBase<TOnlineChainableAsyncOp<OpType, void>, OpType, void>
{
public:
	using Super = Private::TOnlineAsyncOpBase<TOnlineChainableAsyncOp<OpType, void>, OpType, void>;

	TOnlineChainableAsyncOp(TOnlineAsyncOp<OpType>& InOwningOperation)
		: Super()
		, OwningOperation(InOwningOperation)
	{
	}

	TOnlineChainableAsyncOp(TOnlineChainableAsyncOp&& Other)
		: OwningOperation(Other.OwningOperation)
	{
	}

	TOnlineChainableAsyncOp& operator=(TOnlineChainableAsyncOp&& Other)
	{
		check(&OwningOperation == &Other.OwningOperation); // Can't reassign this
		Super::operator=(MoveTemp(Other));
		return *this;
	}

	template <typename QueueType>
	void Enqueue(QueueType& Queue, double DelayInSeconds = 0.0)
	{
		OwningOperation.Enqueue(Queue, DelayInSeconds);
	}

	TOnlineAsyncOp<OpType>& GetOwningOperation()
	{
		return OwningOperation;
	}

protected:
	TOnlineAsyncOp<OpType>& OwningOperation;
};

class FOnlineAsyncOp
{
public:
	virtual ~FOnlineAsyncOp() = default;

	Private::FOnlineOperationData Data;

	virtual void SetError(FOnlineError&& Error) = 0;
	virtual void Cancel(const FOnlineError& Reason) = 0;
	virtual void ClearCallback() = 0;

	void AddStep(TUniquePtr<Private::IStep>&& Step)
	{
		Steps.Add(MoveTemp(Step));
	}

protected:
	template <typename CallableType>
	void ExecuteImpl(const TCHAR* DebugName, FOnlineAsyncExecutionPolicy ExecutionPolicy, CallableType&& Callable)
	{
		switch (ExecutionPolicy.GetExecutionPolicy())
		{
		case EOnlineAsyncExecutionPolicy::RunOnGameThread:
			if (IsInGameThread())
			{
				Callable();
			}
			else
			{
				ExecuteOnGameThread(DebugName, MoveTemp(Callable));
			}
			break;

		case EOnlineAsyncExecutionPolicy::RunOnNextTick:
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([DebugName, LambdaCallable = Callable](float dt) mutable
				{
					ExecuteOnGameThread(DebugName, MoveTemp(LambdaCallable));
					return false;
				}));
			break;

		case EOnlineAsyncExecutionPolicy::RunOnThreadPool:
			Async(EAsyncExecution::ThreadPool, MoveTemp(Callable));
			break;

		case EOnlineAsyncExecutionPolicy::RunOnTaskGraph:
			Async(EAsyncExecution::TaskGraph, MoveTemp(Callable));
			break;

		case EOnlineAsyncExecutionPolicy::RunImmediately:
			Callable();
			break;
		}
	}

	void ExecuteNextStepImpl(const TCHAR* DebugName, const TSharedRef<FOnlineAsyncOp>& SharedThis)
	{
		const int32 StepToExecute = NextStep;
		++NextStep;
		if (StepToExecute < Steps.Num())
		{
			ExecuteImpl(DebugName, Steps[StepToExecute]->GetExecutionPolicy(),
				[this, StepToExecute, WeakThis = TWeakPtr<FOnlineAsyncOp>(SharedThis)]()
				{
					if (TSharedPtr<FOnlineAsyncOp> PinnedThis = WeakThis.Pin())
					{
						Steps[StepToExecute]->Execute();
					}
				});
		}
	}

	void ExecuteRepeatStepImpl(const TCHAR* DebugName, const TSharedRef<FOnlineAsyncOp>& SharedThis)
	{
		const int32 StepToExecute = NextStep == 0 ? 0 : NextStep - 1;
		if (StepToExecute < Steps.Num())
		{
			ExecuteImpl(DebugName, Steps[StepToExecute]->GetExecutionPolicy(),
				[this, StepToExecute, WeakThis = TWeakPtr<FOnlineAsyncOp>(SharedThis)]()
				{
					if (TSharedPtr<FOnlineAsyncOp> PinnedThis = WeakThis.Pin())
					{
						Steps[StepToExecute]->Execute();
					}
				});
		}
	}

	TArray<TUniquePtr<Private::IStep>> Steps;
	int32 NextStep = 0;
};

// This class represents an async operation on the public interface
// There may be one or more handles pointing to one instance
template <typename OpType>
class TOnlineAsyncOp 
	: public Private::TOnlineAsyncOpBase<TOnlineAsyncOp<OpType>, OpType, void>
	, public FOnlineAsyncOp
	, public TSharedFromThis<TOnlineAsyncOp<OpType>>
{
public:
	using ParamsType = typename OpType::Params;
	using ResultType = typename OpType::Result;

	TOnlineAsyncOp(IOnlineServices& InServices, ParamsType&& Params)
		: Services(InServices)
		, SharedState(MakeShared<FAsyncOpSharedState>(MoveTemp(Params)))
		, OpStartTimeAbsoluteSeconds(FPlatformTime::Seconds())
	{
	}

	~TOnlineAsyncOp() = default;

	bool IsReady() const
	{
		return SharedState->State != EAsyncOpState::Invalid;
	}

	bool IsComplete() const
	{
		return SharedState->State >= EAsyncOpState::Complete;
	}

	EAsyncOpState GetState() const
	{
		return SharedState->State;
	}

	const ParamsType& GetParams() const
	{
		return SharedState->Params;
	}

	TOnlineAsyncOp<OpType>& GetOwningOperation()
	{
		return *this;
	}

	static TOnlineAsyncOp<OpType> CreateError(const FOnlineError& Error) { return TOnlineAsyncOp<OpType>(); }

	TOnlineAsyncOpHandle<OpType> GetHandle()
	{
		return TOnlineAsyncOpHandle<OpType>(CreateSharedState());
	}

	template<typename OtherOpType>
	TOnlineAsyncOpHandle<OtherOpType> GetWrappedHandle(TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)> ResultConversionFunction)
	{
		return TOnlineAsyncOpHandle<OtherOpType>(WrapSharedState<OtherOpType>(MoveTemp(ResultConversionFunction)));
	}

	virtual void ClearCallback() override
	{
		UE_LOGF(LogOnlineServices, Log, "%p %ls clear complete callbacks while in state %ls", this, OpType::Name, LexToString(SharedState->State));
		TArray<TSharedRef<FAsyncOpSharedHandleState>> SharedHandleStatesCopy(SharedHandleStates);
		for (TSharedRef<FAsyncOpSharedHandleState>& SharedHandleState : SharedHandleStatesCopy)
		{
			SharedHandleState->SetOnComplete(TDelegate<void(const TOnlineResult<OpType>&)>());
			SharedHandleState->SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OpType>& Handle, const FWillRetry&)>());
			SharedHandleState->SetOnProgress(TDelegate<void(const FAsyncProgress&)>());
		}
		OnCompleteEvent.Clear();
	}

	virtual void Cancel(const FOnlineError& Reason) override
	{
		if (SharedState->State < EAsyncOpState::Complete)
		{
			SetResultAndState(TOnlineResult<OpType>(Reason), EAsyncOpState::Cancelled);
		}
	}

	void SetResult(ResultType&& InResult)
	{
		if (SharedState->State < EAsyncOpState::Complete)
		{
			SetResultAndState(TOnlineResult<OpType>(MoveTemp(InResult)), EAsyncOpState::Complete);
		}
	}

	virtual void SetError(FOnlineError&& Error) override
	{
		if (SharedState->State < EAsyncOpState::Complete)
		{
			SetResultAndState(TOnlineResult<OpType>(MoveTemp(Error)), EAsyncOpState::Complete);
		}
	}

	UE_DEPRECATED(5.6, "GetServices has been deprecated and we will no longer return FOnlineServicesCommon, use GetOnlineServices() instead and convert to FOnlineServicesCommon when needed")
	FOnlineServicesCommon& GetServices() const
	{ 
		return *reinterpret_cast<FOnlineServicesCommon*>(&Services); 
	}

	IOnlineServices& GetOnlineServices() const { return Services; }

	template <typename QueueType>
	void Enqueue(QueueType& Queue, double DelayInSeconds = 0.0)
	{
		check(SharedState->State < EAsyncOpState::Queued);
		SharedState->State = EAsyncOpState::Queued;
		UE_LOGF(LogOnlineServices, Verbose, "%p %ls op state set to: %ls", this, OpType::Name, LexToString(SharedState->State));
		Queue.Enqueue(*this, DelayInSeconds);
	}

	void Start()
	{
		SharedState->State = EAsyncOpState::Running;
		UE_LOGF(LogOnlineServices, Verbose, "%p %ls op state set to: %ls", this, OpType::Name, LexToString(SharedState->State));
		OnStartEvent.Broadcast(*this);
		ExecuteNextStep();
	}

	void ExecuteNextStep()
	{
		if (!IsComplete())
		{
			ExecuteNextStepImpl(OpType::Name, this->AsShared());
		}
	}

	void ExecuteRepeatStep()
	{
		if (!IsComplete())
		{
			ExecuteRepeatStepImpl(OpType::Name, this->AsShared());
		}
	}

	template <typename CallableType>
	void Execute(FOnlineAsyncExecutionPolicy ExecutionPolicy, CallableType&& Callable)
	{
		ExecuteImpl(OpType::Name, ExecutionPolicy, Callable);
	}

	TOnlineEvent<void(const TOnlineAsyncOp<OpType>&)> OnStart() { return OnStartEvent; }
	TOnlineEvent<void(const TOnlineAsyncOp<OpType>&, const TOnlineResult<OpType>&)> OnComplete() { return OnCompleteEvent; }

	const FString& GetInterfaceName() { return InterfaceName; }
	void SetInterfaceName(FStringView InName) { InterfaceName = InName; }

protected:
	IOnlineServices& Services;

	void SetResultAndState(TOnlineResult<OpType>&& Result, EAsyncOpState State)
	{
		if (IsInGameThread())
		{
			if (SharedState->State <= EAsyncOpState::Queued)
			{
				OnStartEvent.Broadcast(*this);
			}

			SharedState->Result = MoveTemp(Result);
			SharedState->State = State;
			UE_LOGF(LogOnlineServices, Verbose, "%p %ls op state set to: %ls", this, OpType::Name, LexToString(State));

			TriggerOnComplete(SharedState->Result);
		}
		else
		{
			Execute(FOnlineAsyncExecutionPolicy::RunOnGameThread(),
				[State, LambdaResult = MoveTemp(Result), WeakThis = TWeakPtr<TOnlineAsyncOp<OpType>>(this->AsShared())]() mutable
				{
					if (TSharedPtr<TOnlineAsyncOp<OpType>> PinnedThis = WeakThis.Pin())
					{
						PinnedThis->SetResultAndState(MoveTemp(LambdaResult), State);
					}
				});
		}
	}

	void TriggerOnComplete(const TOnlineResult<OpType>& Result)
	{
		TArray<TSharedRef<FAsyncOpSharedHandleState>> SharedHandleStatesCopy(SharedHandleStates);
		for (TSharedRef<FAsyncOpSharedHandleState>& SharedHandleState : SharedHandleStatesCopy)
		{
			SharedHandleState->TriggerOnComplete(Result);
		}

		// The general callback OnAsyncOpCompleted needs to be called before OnCompleteEvent, because 
		// user code triggered by OnCompleteEvent could potentially remove the last reference of OnlineServices
		// instance and destroy it, which will make it invalid for OnAsyncOpCompleted callback
		double DurationInSeconds = FPlatformTime::Seconds() - OpStartTimeAbsoluteSeconds;
		TOptional<FOnlineError> ErrorValue;
		if (Result.IsError())
		{
			ErrorValue = Result.GetErrorValue();
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FOnOnlineAsyncOpCompletedParams Params(GetServices(), ErrorValue);
		Params.DurationInSeconds = DurationInSeconds;
		Params.InterfaceName = InterfaceName;
		Params.OpName = OpType::Name;
		OnOnlineAsyncOpCompletedV2.Broadcast(Params);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FOnAsyncOpCompletedParams OnAsyncOpCompletedParams;
		OnAsyncOpCompletedParams.OnlineServices = Services.AsWeak();
		OnAsyncOpCompletedParams.OnlineError = ErrorValue;
		OnAsyncOpCompletedParams.DurationInSeconds = DurationInSeconds;
		OnAsyncOpCompletedParams.InterfaceName = InterfaceName;
		OnAsyncOpCompletedParams.OpName = OpType::Name;
		OnAsyncOpCompleted.Broadcast(OnAsyncOpCompletedParams);

		OnCompleteEvent.Broadcast(*this, Result);
	}

	class FAsyncOpSharedState
	{
	public:
		FAsyncOpSharedState(ParamsType&& InParams)
			: Params(MoveTemp(InParams))
		{
		}

		ParamsType Params;
		// This will need to be protected with a mutex if we want to allow this to be set from multiple threads (eg, set result from a task graph thread, while allowing this to be cancelled from the game thread)
		TOnlineResult<OpType> Result{ Errors::Unknown() };
		EAsyncOpState State = EAsyncOpState::Invalid;

		bool IsComplete() const
		{
			return State >= EAsyncOpState::Complete;
		}
	};

	class FAsyncOpSharedHandleState : public Private::IOnlineAsyncOpSharedState<OpType>, public TSharedFromThis<FAsyncOpSharedHandleState>
	{
	public:
		FAsyncOpSharedHandleState(const TSharedRef<TOnlineAsyncOp<OpType>>& InAsyncOp)
			: SharedState(InAsyncOp->SharedState)
			, AsyncOp(InAsyncOp)
		{
		}

		~FAsyncOpSharedHandleState()
		{
			Detach();
		}

		virtual void Cancel(const FOnlineError& Reason) override
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOP = AsyncOp.Pin();
			if (PinnedOP.IsValid())
			{
				bCancelled = true;

				// When canceling an operation the outer reason must always be Errors::Cancelled.
				if (Reason.GetErrorCode() == Errors::ErrorCode::Common::Cancelled)
				{
					PinnedOP->Cancel(Reason);
				}
				else
				{
					PinnedOP->Cancel(Errors::Cancelled(Reason));
				}
			}
		}

		virtual EAsyncOpState GetState() const override
		{
			return bCancelled ? EAsyncOpState::Cancelled : SharedState->State;
		}

		virtual void SetOnProgress(TDelegate<void(const FAsyncProgress&)>&& Function) override
		{
			OnProgressFn = MoveTemp(Function);
		}

		virtual void SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OpType>& Handle, const FWillRetry&)>&& Function) override
		{
			OnWillRetryFn = MoveTemp(Function);
		}

		virtual void SetOnComplete(TDelegate<void(const TOnlineResult<OpType>&)>&& Function) override
		{
			OnCompleteFn = MoveTemp(Function);
			if (SharedState->IsComplete())
			{
				TriggerOnComplete(SharedState->Result);
			}
		}

		void TriggerOnProgress(const FAsyncProgress& Progress)
		{
			OnProgressFn.ExecuteIfBound(Progress);
		}

		void TriggerOnWillRetry(const FWillRetry& RetryDetails)
		{
			if (OnWillRetryFn.IsBound())
			{
				TSharedRef<TOnlineAsyncOpHandle<OpType>> Handle = TOnlineAsyncOpHandle<OpType>(StaticCastSharedRef<Private::IOnlineAsyncOpSharedState<OpType>>(this->AsShared()));
				OnWillRetryFn.ExecuteIfBound(Handle, RetryDetails);
			}
		}

		void TriggerOnComplete(const TOnlineResult<OpType>& Result)
		{
			// TODO: Execute OnCompleteFn next tick on game thread
			if (OnCompleteFn.IsBound())
			{
				OnCompleteFn.ExecuteIfBound(Result);
				OnCompleteFn.Unbind();
				Detach();
			}
		}

	private:
		void Detach()
		{
			TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOp = AsyncOp.Pin();
			AsyncOp.Reset();
			if (PinnedOp.IsValid())
			{
				PinnedOp->Detach(this->AsShared());
			}
		}

		TDelegate<void(const FAsyncProgress&)> OnProgressFn;
		TDelegate<void(TOnlineAsyncOpHandle<OpType>& Handle, const FWillRetry&)> OnWillRetryFn;
		TDelegate<void(const TOnlineResult<OpType>&)> OnCompleteFn;

		bool bCancelled = false;
		TSharedRef<FAsyncOpSharedState> SharedState;
		TWeakPtr<TOnlineAsyncOp<OpType>> AsyncOp;
	};

	template <typename OtherOpType>	class TWrappedAsyncOpSharedHandleStateContainer;

	template <typename OtherOpType>
	class TWrappedAsyncOpSharedHandleState : public Private::IOnlineAsyncOpSharedState<OtherOpType>
	{
	public:
		TWrappedAsyncOpSharedHandleState(TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>& InSharedHandleState, TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)>&& InResultConversionFunction);
		virtual void Cancel(const FOnlineError& Reason) override;
		virtual EAsyncOpState GetState() const override;
		virtual void SetOnProgress(TDelegate<void(const FAsyncProgress&)>&& Function) override;
		virtual void SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OtherOpType>&, const FWillRetry&)>&& WillRetryFunction) override;
		virtual void SetOnComplete(TDelegate<void(const TOnlineResult<OtherOpType>&)>&& OnCompleteFunction) override;

	protected:
		TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>& SharedHandleState;
		TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)> ResultConversionFunction;
	};

	template <typename OtherOpType>
	class TWrappedAsyncOpSharedHandleStateContainer : public FAsyncOpSharedHandleState
	{
	public:
		TWrappedAsyncOpSharedHandleStateContainer(const TSharedRef<TOnlineAsyncOp<OpType>>& InAsyncOp, TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)>&& InResultConversionFunction);
		TSharedRef<TWrappedAsyncOpSharedHandleState<OtherOpType>> GetWrappedState();

	private:
		TWrappedAsyncOpSharedHandleState<OtherOpType> WrappedState;
	};

	void Detach(const TSharedRef<FAsyncOpSharedHandleState>& SharedHandleState)
	{
		SharedHandleStates.Remove(SharedHandleState);
	}

	TSharedRef<Private::IOnlineAsyncOpSharedState<OpType>> CreateSharedState()
	{
		TSharedRef<FAsyncOpSharedHandleState> SharedHandleState = MakeShared<FAsyncOpSharedHandleState>(this->AsShared());
		SharedHandleStates.Add(SharedHandleState);
		return StaticCastSharedRef<Private::IOnlineAsyncOpSharedState<OpType>>(SharedHandleState);
	}

	template <typename OtherOpType>
	TSharedRef<Private::IOnlineAsyncOpSharedState<OtherOpType>> WrapSharedState(TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)>&& ResultConversionFunction)
	{
		TSharedRef<TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>> SharedHandleState = MakeShared<TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>>(this->AsShared(), MoveTemp(ResultConversionFunction));
		SharedHandleStates.Add(SharedHandleState);
		return SharedHandleState->GetWrappedState();
	}

	TSharedRef<FAsyncOpSharedState> SharedState;
	TArray<TSharedRef<FAsyncOpSharedHandleState>> SharedHandleStates;
	TOnlineEventCallable<void(const TOnlineAsyncOp<OpType>&)> OnStartEvent;
	TOnlineEventCallable<void(const TOnlineAsyncOp<OpType>&, const TOnlineResult<OpType>&)> OnCompleteEvent;
	double OpStartTimeAbsoluteSeconds;

	friend class FOnlineAsyncOpCache;
	FOnlineEventDelegateHandle OpCacheHandle; // Delegate handle for FOnlineAsyncOpCache internal usage
	FString InterfaceName;
};

template <typename OpType>
template <typename OtherOpType>
TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::TWrappedAsyncOpSharedHandleState(TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>& InSharedHandleState, TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)>&& InResultConversionFunction)
	: SharedHandleState(InSharedHandleState)
	, ResultConversionFunction(MoveTemp(InResultConversionFunction))
{
	check(ResultConversionFunction.IsSet())
}

template <typename OpType>
template <typename OtherOpType>
void TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::Cancel(const FOnlineError& Reason)
{
	SharedHandleState.Cancel(Reason);
}

template <typename OpType>
template <typename OtherOpType>
EAsyncOpState TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::GetState() const
{
	return SharedHandleState.GetState();
}

template <typename OpType>
template <typename OtherOpType>
void TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::SetOnProgress(TDelegate<void(const FAsyncProgress&)>&& Function)
{
	SharedHandleState.SetOnProgress(MoveTemp(Function));
}

template <typename OpType>
template <typename OtherOpType>
void TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OtherOpType>&, const FWillRetry&)>&& WillRetryFunction)
{
	SharedHandleState.SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OpType>&, const FWillRetry&)>::CreateLambda(
		[this, WillRetryFunction](TOnlineAsyncOpHandle<OpType>& Handle, const FWillRetry& RetryDetails)
		{
			if (WillRetryFunction.IsBound())
			{
				TOnlineAsyncOpHandle<OtherOpType> WrappedHandle = TOnlineAsyncOpHandle<OtherOpType>(StaticCastSharedRef<Private::IOnlineAsyncOpSharedState<OtherOpType>>(SharedHandleState.GetWrappedState()));
				WillRetryFunction.Execute(WrappedHandle, RetryDetails);
			}
		}));
}

template <typename OpType>
template <typename OtherOpType>
void TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleState<OtherOpType>::SetOnComplete(TDelegate<void(const TOnlineResult<OtherOpType>&)>&& OnCompleteFunction)
{
	SharedHandleState.SetOnComplete(TDelegate<void(const TOnlineResult<OpType>&)>::CreateLambda(
		[this, OnCompleteFunction](const TOnlineResult<OpType>& Result)
		{
			if (OnCompleteFunction.IsBound())
			{
				TOnlineResult<OtherOpType> ConvertedResult = ResultConversionFunction(Result);
				OnCompleteFunction.Execute(ConvertedResult);
			}
		}));
}

template <typename OpType>
template <typename OtherOpType>
TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>::TWrappedAsyncOpSharedHandleStateContainer(const TSharedRef<TOnlineAsyncOp<OpType>>& InAsyncOp, TFunction<TOnlineResult<OtherOpType>(const TOnlineResult<OpType>&)>&& InResultConversionFunction)
	: FAsyncOpSharedHandleState(InAsyncOp)
	, WrappedState(*this, MoveTemp(InResultConversionFunction))
{
}

template <typename OpType>
template <typename OtherOpType>
TSharedRef<typename TOnlineAsyncOp<OpType>::template TWrappedAsyncOpSharedHandleState<OtherOpType>> TOnlineAsyncOp<OpType>::TWrappedAsyncOpSharedHandleStateContainer<OtherOpType>::GetWrappedState()
{
	// Use the aliasing constructor so the lifetime of the shared handle state is tied to the container's lifetime
	return TSharedPtr<TWrappedAsyncOpSharedHandleState<OtherOpType>>(this->AsShared(), &WrappedState).ToSharedRef();
}

template <typename OpType>
using TOnlineAsyncOpRef = TSharedRef<TOnlineAsyncOp<OpType>>;
template <typename OpType>
using TOnlineAsyncOpPtr = TSharedPtr<TOnlineAsyncOp<OpType>>;

namespace Private {

template <typename Outer, typename OpType, typename LastResultType>
template <typename CallableType>
auto TOnlineAsyncOpBase<Outer, OpType, LastResultType>::Then(CallableType&& InCallable, FOnlineAsyncExecutionPolicy ExecutionPolicy)
{
	using ResultType = typename TOnlineAsyncOpCallableTraits<CallableType>::ResultType;

	TOnlineAsyncOp<OpType>& Op = static_cast<Outer*>(this)->GetOwningOperation();

	TStep<ResultType>* Step = new TStep<ResultType>(MoveTemp(ExecutionPolicy));
	TUniquePtr<IStep> StepPtr(Step);
	Step->SetExecFunction(Op, LastResult, MoveTemp(InCallable));

	Op.AddStep(MoveTemp(StepPtr));

	if constexpr (std::is_same_v<TContinuationResultType_T<ResultType>, void>)
	{
		return TOnlineChainableAsyncOp<OpType, void>(Op);
	}
	else
	{
		return TOnlineChainableAsyncOp<OpType, ResultType>(Op, Step->GetResultRef());
	}
}

template <typename Outer, typename OpType, typename LastResultType>
template <typename CallableType>
auto TOnlineAsyncOpBase<Outer, OpType, LastResultType>::ForEach(CallableType&& InCallable, FOnlineAsyncExecutionPolicy ExecutionPolicy)
{
	using ResultType = typename TOnlineAsyncOpCallableTraits<CallableType>::ResultType;

	TOnlineAsyncOp<OpType>& Op = static_cast<Outer*>(this)->GetOwningOperation();

	TForEachStep<ResultType>* Step = new TForEachStep<ResultType>(MoveTemp(ExecutionPolicy));
	TUniquePtr<IStep> StepPtr(Step);
	Step->SetExecFunction(Op, LastResult, MoveTemp(InCallable));

	Op.AddStep(MoveTemp(StepPtr));

	return TOnlineChainableAsyncOp<OpType, TArray<ResultType>>(Op, Step->GetResultRef());
}

template <typename Outer, typename OpType, typename LastResultType>
template <typename CallableType>
auto TOnlineAsyncOpBase<Outer, OpType, LastResultType>::ForEachN(int32 BatchSize, CallableType&& InCallable, FOnlineAsyncExecutionPolicy ExecutionPolicy)
{
	using CallableResultType = typename TOnlineAsyncOpCallableTraits<CallableType>::ResultType;

	TOnlineAsyncOp<OpType>& Op = static_cast<Outer*>(this)->GetOwningOperation();

	TForEachNStep<CallableResultType>* Step = new TForEachNStep<CallableResultType>(BatchSize, MoveTemp(ExecutionPolicy));
	TUniquePtr<IStep> StepPtr(Step);
	Step->SetExecFunction(Op, LastResult, MoveTemp(InCallable));

	Op.AddStep(MoveTemp(StepPtr));

	return TOnlineChainableAsyncOp<OpType, typename TForEachNStep<CallableResultType>::ResultType>(Op, Step->GetResultRef());
}


template <typename Outer, typename OpType>
template <typename CallableType>
auto TOnlineAsyncOpBase<Outer, OpType, void>::Then(CallableType&& InCallable, FOnlineAsyncExecutionPolicy ExecutionPolicy)
{
	using ResultType = typename TOnlineAsyncOpCallableTraits<CallableType>::ResultType;

	TOnlineAsyncOp<OpType>& Op = static_cast<Outer*>(this)->GetOwningOperation();

	TStep<ResultType>* Step = new TStep<ResultType>(MoveTemp(ExecutionPolicy));
	TUniquePtr<IStep> StepPtr(Step);
	Step->SetExecFunction(Op, MoveTemp(InCallable));

	Op.AddStep(MoveTemp(StepPtr));
	if constexpr (std::is_same_v<TContinuationResultType_T<ResultType>, void>)
	{
		return TOnlineChainableAsyncOp<OpType, void>(Op);
	}
	else
	{
		return TOnlineChainableAsyncOp<OpType, ResultType>(Op, Step->GetResultRef());
	}
}

/* Private */ }

/* UE::Online */ }
