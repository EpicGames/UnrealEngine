// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineMeta.h" // IWYU pragma: keep
#include "Containers/Ticker.h"

namespace UE::Online {

struct CLocalAccountIdDefined
{
	template <typename T>
	auto Requires() -> decltype(T::LocalAccountId);
};

enum class EOperationCacheExpirationPolicy : uint8
{
	// Expire when the operation completes
	UponCompletion,
	// Expire after a certain amount of time has elapsed
	Duration,
	// Only expire if the cache is cleared
	Never
};

inline void LexFromString(EOperationCacheExpirationPolicy& Value, const TCHAR* const String)
{
	if (FCString::Stricmp(String, TEXT("Duration")) == 0)
	{
		Value = EOperationCacheExpirationPolicy::Duration;
	}
	else if (FCString::Stricmp(String, TEXT("Never")) == 0)
	{
		Value = EOperationCacheExpirationPolicy::Never;
	}
	else //if (FCString::Stricmp(String, TEXT("UponCompletion")) == 0)
	{
		Value = EOperationCacheExpirationPolicy::UponCompletion;
	}
}

struct FOperationConfig
{
	EOperationCacheExpirationPolicy CacheExpiration = EOperationCacheExpirationPolicy::UponCompletion;
	double CacheExpirySeconds = 0.0f;
	bool bCacheError = false;
};

class IOnlineAnyData
{
public:
	virtual ~IOnlineAnyData() = default;
	virtual FOnlineTypeName GetTypeName() const = 0;

	template <typename T>
	const T* Get() const
	{
		if (GetTypeName() == TOnlineTypeInfo<T>::GetTypeName())
		{
			return static_cast<const T*>(GetData());
		}

		return nullptr;
	}

	template <typename T>
	const T& GetRef() const
	{
		const T* Value = Get<T>();
		check(Value != nullptr);
		return *Value;
	}

protected:
	virtual const void* GetData() const = 0;
};

template <typename T, typename BaseType = IOnlineAnyData>
class TOnlineAnyData : public BaseType
{
public:
	using ValueType = std::remove_reference_t<T>;

	TOnlineAnyData(const ValueType& InData)
		: Data(InData)
	{
	}

	TOnlineAnyData(ValueType&& InData)
		: Data(MoveTemp(InData))
	{
	}

	virtual FOnlineTypeName GetTypeName() const override
	{
		return  TOnlineTypeInfo<ValueType>::GetTypeName();
	}

	const T& GetDataRef() const 
	{
		return Data;
	}

protected:
	virtual const void* GetData() const override
	{
		return &Data;
	}

	T Data;
};

namespace Private {

/**
 * TArray GetTypeHash specialization used for TAsyncOpParamsFuncs::GetTypeHash
 */
template <typename T>
inline uint32 GetTypeHash(const TArray<T>& Array)
{
	using ::GetTypeHash;
	uint32 TypeHash = 0;
	for (const T& Value : Array)
	{
		TypeHash = HashCombine(TypeHash, GetTypeHash(Value));
	}
	return TypeHash;
}

// Different than the standard CGetTypeHashable as we want to detect GetTypeHash using more than the standard name lookup rules
struct COnlineGetTypeHashable
{
	template <typename T>
	auto Requires(uint32& Result, const T& Val) -> decltype(
		Result = GetTypeHash(Val)
		);

	template <typename T>
	auto Requires(uint32& Result, const T& Val) -> decltype(
		Result = ::GetTypeHash(Val)
		);

	template <typename T>
	auto Requires(uint32& Result, const T& Val) -> decltype(
		Result = ::UE::Online::GetTypeHash(Val)
		);

	template <typename T>
	auto Requires(uint32& Result, const T& Val) -> decltype(
		Result = ::UE::Online::Private::GetTypeHash(Val)
		);
};

/* Private */ }

template <typename OpType>
struct TJoinableOpParamsFuncs
{
	inline static bool Compare(const typename OpType::Params& First, const typename OpType::Params& Second)
	{
		bool bResult = true;
		Meta::VisitFields<typename OpType::Params>([&bResult, &First, &Second](const auto& Field)
		{
			bResult = bResult && (First.*Field.Pointer) == (Second.*Field.Pointer);
		});
		return bResult;
	}

	inline static uint32 GetTypeHash(const typename OpType::Params& Params)
	{
		using ::GetTypeHash;
		using Private::GetTypeHash;
		uint32 CombinedHash = 0;
		Meta::VisitFields(Params,
			[&CombinedHash](const TCHAR* FieldName, const auto& Field)
			{
				CombinedHash = HashCombine(CombinedHash, GetTypeHash(Field));
			});
		return CombinedHash;
	}
};

// MergeableOps Params contain a field of Mutations that implements operator+= that performs the merge
template <typename OpType>
struct TMergeableOpParamsFuncs
{
	inline static bool Compare(const typename OpType::Params& First, const typename OpType::Params& Second)
	{
		bool bResult = true;
		Meta::VisitFields<typename OpType::Params>([&bResult, &First, &Second](const auto& Field)
			{
				constexpr bool bIsMutationFieldType = std::is_same_v<decltype(Field.Pointer), decltype(&OpType::Params::Mutations)>;
				if constexpr (TModels_V<Private::COnlineGetTypeHashable, decltype(First.*Field.Pointer)> || !bIsMutationFieldType)
				{
					if constexpr (bIsMutationFieldType)
					{
						if (&OpType::Params::Mutations == Field.Pointer)
						{
							return;
						}
					}
					bResult = bResult && (First.*Field.Pointer) == (Second.*Field.Pointer);
				}
			});
		return bResult;
	}

	inline static uint32 GetTypeHash(const typename OpType::Params& Params)
	{
		using ::GetTypeHash;
		using Private::GetTypeHash;
		uint32 CombinedHash = 0;
		Meta::VisitFields<typename OpType::Params>([&Params, &CombinedHash](const auto& Field)
			{
				constexpr bool bIsMutationFieldType = std::is_same_v<decltype(Field.Pointer), decltype(&OpType::Params::Mutations)>;
				if constexpr (TModels_V<Private::COnlineGetTypeHashable, decltype(Params.*Field.Pointer)> || !bIsMutationFieldType)
				{
					if constexpr (bIsMutationFieldType)
					{
						if (&OpType::Params::Mutations == Field.Pointer)
						{
							return;
						}
					}
					CombinedHash = HashCombine(CombinedHash, GetTypeHash(Params.*Field.Pointer));
				}
			});
		return CombinedHash;
	}

	inline static void Merge(typename OpType::Params& CurrentParams, typename OpType::Params&& NewParams)
	{
		CurrentParams.Mutations += MoveTemp(NewParams.Mutations);
	}
};

class FOnlineAsyncOpCache
{
public:
	FOnlineAsyncOpCache(const FString& ConfigPath, class FOnlineServicesCommon& InServices)
		: Services(InServices)
	{
	}

	// Create an operation
	template <typename OpType>
	TOnlineAsyncOpRef<OpType> GetOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		TUniquePtr<TWrappedOperation<OpType>> WrappedOp = MakeUnique<TWrappedOperation<OpType>>(Services, MoveTemp(Params));
		TSharedRef<TOnlineAsyncOp<OpType>> Op = WrappedOp->GetDataRef();

		if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
		{
			// add LocalAccountId to the Op Data
			Op->Data.template Set<decltype(Params.LocalAccountId)>(TEXT("LocalAccountId"), Op->GetParams().LocalAccountId);
		}

		Op->OpCacheHandle = Op->OnComplete().Add(GetSharedThis(), [this](const TOnlineAsyncOp<OpType>& ThisOp, const TOnlineResult<OpType>& Result) {
			const TOnlineAsyncOp<OpType>* ThisOpPtr = &ThisOp;
			IndependentOperations.RemoveAll([ThisOpPtr](const TUniquePtr<IWrappedOperation>& WrappedOp) {
				const TSharedRef<TOnlineAsyncOp<OpType>>* Op = WrappedOp->Get<TSharedRef<TOnlineAsyncOp<OpType>>>();
				if (Op)
				{
					return ThisOpPtr == &(*Op).Get();
				}
				return false;
			});
		});

		IndependentOperations.Add(MoveTemp(WrappedOp));

		return Op;
	}

	// Join an existing operation or use a non-expired cached result, or create an operation that can later be joined
	template <typename OpType, typename ParamsFuncsType /*= TJoinableOpParamsFuncs<OpType>*/>
	TOnlineAsyncOpRef<OpType> GetJoinableOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		TSharedPtr<TOnlineAsyncOp<OpType>> Op;

		// find existing op
		if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
		{
			if (TUniquePtr<IWrappedOperation>* OpPtr = UserOperations.FindOrAdd(Params.LocalAccountId).Find(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Params)))
			{
				if (!(*OpPtr)->IsExpired())
				{
					Op = (*OpPtr)->GetRef<TSharedRef<TOnlineAsyncOp<OpType>>>();
				}
			}
		}
		else
		{
			if (TUniquePtr<IWrappedOperation>* OpPtr = Operations.Find(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Params)))
			{
				if (!(*OpPtr)->IsExpired())
				{
					Op = (*OpPtr)->GetRef<TSharedRef<TOnlineAsyncOp<OpType>>>();
				}
			}
		}
		
		if (!Op)
		{
			FOperationConfig Config;
			check(LoadConfigFn);
			LoadConfigFn(Config, ConfigSectionHierarchy);
			
			Op = CreateOp<OpType, ParamsFuncsType>(MoveTemp(Params));
			Op->OpCacheHandle = Op->OnComplete().Add(GetSharedThis(), [this, Config](const TOnlineAsyncOp<OpType>& ThisOp, const TOnlineResult<OpType>& Result)
				{
					if ((ThisOp.GetState() == EAsyncOpState::Cancelled && !Config.bCacheError) ||
						(Result.IsError() && !Config.bCacheError) ||
						(Config.CacheExpiration == EOperationCacheExpirationPolicy::UponCompletion))
					{
						if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
						{
							UserOperations.FindOrAdd(ThisOp.GetParams().LocalAccountId).Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(ThisOp.GetParams()));
						}
						else
						{
							Operations.Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(ThisOp.GetParams()));
						}
					}
					else if (Config.CacheExpiration == EOperationCacheExpirationPolicy::Duration)
					{
						// Set timer with duration to remove cached operation
						FTSTicker::GetCoreTicker().AddTicker(TEXT("OnlineAsyncOpCacheExpiry"), Config.CacheExpirySeconds,
							[this, WeakOp = ThisOp.AsWeak()](float) {
								TSharedPtr<const TOnlineAsyncOp<OpType>> PinnedOp = WeakOp.Pin();
								if (PinnedOp.IsValid())
								{
									if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
									{
										UserOperations.FindOrAdd(PinnedOp->GetParams().LocalAccountId).Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(PinnedOp->GetParams()));
									}
									else
									{
										Operations.Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(PinnedOp->GetParams()));
									}
								}
 								return false;
							});
					}
				});
		}

		return Op.ToSharedRef();
	}

	// Merge with a pending operation, or create an operation
	template <typename OpType, typename ParamsFuncsType /*= TMergeableOpParamsFuncs<OpType>*/>
	TOnlineAsyncOpRef<OpType> GetMergeableOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		TSharedPtr<TOnlineAsyncOp<OpType>> Op;

		// find existing op
		if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
		{
			if (TUniquePtr<IWrappedOperation>* OpPtr = UserOperations.FindOrAdd(Params.LocalAccountId).Find(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Params)))
			{
				Op = (*OpPtr)->GetRef<TSharedRef<TOnlineAsyncOp<OpType>>>();
			}
		}
		else
		{
			if (TUniquePtr<IWrappedOperation>* OpPtr = Operations.Find(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Params)))
			{
				Op = (*OpPtr)->GetRef<TSharedRef<TOnlineAsyncOp<OpType>>>();
			}
		}
		
		if (Op)
		{
			ParamsFuncsType::Merge(Op->SharedState->Params, MoveTemp(Params));
		}
		else
		{
			FOperationConfig Config;
			check(LoadConfigFn);
			LoadConfigFn(Config, ConfigSectionHierarchy);

			Op = CreateOp<OpType, ParamsFuncsType>(MoveTemp(Params));
			// remove from cache once operation has started. It is no longer mergeable at that point
			Op->OpCacheHandle = Op->OnStart().Add(GetSharedThis(), [this](const TOnlineAsyncOp<OpType>& ThisOp)
				{
					if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
					{
						UserOperations.FindOrAdd(ThisOp.GetParams().LocalAccountId).Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(ThisOp.GetParams()));
					}
					else
					{
						Operations.Remove(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(ThisOp.GetParams()));
					}
				});
		}

		return Op.ToSharedRef();
	}

	void SetLoadConfigFn(TUniqueFunction<bool(FOperationConfig&, const TArray<FString>&)>&& InLoadConfigFn)
	{
		LoadConfigFn = MoveTemp(InLoadConfigFn);
	}

	FOnlineServicesCommon& Services;

	// Internal use only {
	// TODO: Move into package scope instead
	void ClearAllCallbacks();
	void CancelAll();
	bool HasAnyRunningOperation() const;
	// }

private:
	ONLINESERVICESCOMMON_API  TSharedRef<FOnlineAsyncOpCache> GetSharedThis();

	TUniqueFunction<bool(FOperationConfig&, const TArray<FString>&)> LoadConfigFn;

	template <typename OpType, typename ParamsFuncsType>
	TOnlineAsyncOpRef<OpType> CreateOp(typename OpType::Params&& Params)
	{
		TUniquePtr<TWrappedOperation<OpType>> WrappedOp = MakeUnique<TWrappedOperation<OpType>>(Services, MoveTemp(Params));

		TSharedRef<TOnlineAsyncOp<OpType>> Op = WrappedOp->GetDataRef();

		if constexpr (TModels_V<CLocalAccountIdDefined, typename OpType::Params>)
		{
			// add LocalAccountId to the Op Data
			Op->Data.template Set<decltype(Params.LocalAccountId)>(TEXT("LocalAccountId"), Op->GetParams().LocalAccountId);

			UserOperations.FindOrAdd(Op->GetParams().LocalAccountId).Add(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Op->GetParams()), MoveTemp(WrappedOp));
		}
		else
		{
			Operations.Add(FWrappedOperationKey::Create<OpType, ParamsFuncsType>(Op->GetParams()), MoveTemp(WrappedOp));
		}

		return MoveTemp(Op);
	}

	class IWrappedOperation : public IOnlineAnyData
	{
	public:
		virtual bool IsExpired() = 0;
		virtual void Cancel() = 0;
		virtual void ClearCallback() = 0;
		virtual EAsyncOpState GetAsyncOpState() const = 0;
	};

	template <typename OpType>
	class TWrappedOperation : public TOnlineAnyData<TSharedRef<TOnlineAsyncOp<OpType>>, IWrappedOperation>
	{
	public:
		template <typename... ParamTypes>
		TWrappedOperation(ParamTypes&&... Params)
			: TOnlineAnyData<TSharedRef<TOnlineAsyncOp<OpType>>, IWrappedOperation>(MakeShared<TOnlineAsyncOp<OpType>>(Forward<ParamTypes>(Params)...))
		{
		}

		virtual EAsyncOpState GetAsyncOpState() const override
		{
			return this->GetDataRef()->GetState();
		}

		virtual bool IsExpired() override
		{
			if (this->GetDataRef()->GetState() == EAsyncOpState::Cancelled)
			{
				return true;
			}

			// other expiry conditions are handled by removing the cached entry

			return false;
		}

		virtual void ClearCallback() override
		{
			this->GetDataRef()->ClearCallback();
		}

		virtual void Cancel() override
		{
			this->GetDataRef()->Cancel(UE::Online::FOnlineError(Errors::ErrorCode::Common::Cancelled));
		}
	};

	class FWrappedOperationKey
	{
	public:
		template<typename OpType, typename ParamsFuncsType>
		static FWrappedOperationKey Create(const typename OpType::Params& Params)
		{
			return FWrappedOperationKey(TUniquePtr<IWrappedOperationKeyImpl>(new TWrappedOperationKeyImpl<OpType, ParamsFuncsType>(Params)));
		}

		bool operator==(const FWrappedOperationKey& Other) const
		{
			return Impl->Compare(*Other.Impl);
		}

		uint32 GetTypeHash() const
		{
			return Impl->GetTypeHash();
		}

	private:
		class IWrappedOperationKeyImpl : public IOnlineAnyData
		{
		public:
			virtual bool Compare(const IWrappedOperationKeyImpl& Other) const = 0;
			virtual uint32 GetTypeHash() const = 0;
		};

		FWrappedOperationKey(TUniquePtr<IWrappedOperationKeyImpl>&& InImpl)
			: Impl(MoveTemp(InImpl))
		{
		}

		template <typename OpType, typename ParamsFuncsType>
		class TWrappedOperationKeyImpl : public TOnlineAnyData<const typename OpType::Params&, IWrappedOperationKeyImpl>
		{
		public:
			using DataType = const typename OpType::Params&;
			using ValueType = const typename OpType::Params;

			TWrappedOperationKeyImpl(DataType& ParamsRef)
				: TOnlineAnyData<DataType&, IWrappedOperationKeyImpl>(ParamsRef)
			{
			}

			virtual bool Compare(const IWrappedOperationKeyImpl& Other) const override
			{
				if (Other.GetTypeName() == this->GetTypeName())
				{
					return ParamsFuncsType::Compare(this->template GetRef<ValueType>(), Other.template GetRef<ValueType>());
				}

				return false;
			}

			virtual uint32 GetTypeHash() const override
			{
				const uint32 Hash = ParamsFuncsType::GetTypeHash(this->template GetRef<ValueType>());
				return HashCombine(::UE::Online::GetTypeHash(this->GetTypeName()), Hash);
			}
		};

		TUniquePtr<IWrappedOperationKeyImpl> Impl;
	};

	friend uint32 GetTypeHash(const FWrappedOperationKey& Key);

	bool HasAnyRunningOperationIn(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations) const;
	void ClearCallbacks(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations);
	void CancelOperations(const TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>& InOperations);

	TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>> Operations;
	TMap<FAccountId, TMap<FWrappedOperationKey, TUniquePtr<IWrappedOperation>>> UserOperations;
	TArray<TUniquePtr<IWrappedOperation>> IndependentOperations;
};

inline uint32 GetTypeHash(const FOnlineAsyncOpCache::FWrappedOperationKey& Key)
{
	return Key.GetTypeHash();
}

/* UE::Online */ }

#include "OnlineAsyncOpCache_Meta.inl"
