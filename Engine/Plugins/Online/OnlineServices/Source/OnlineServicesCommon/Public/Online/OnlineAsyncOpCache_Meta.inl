// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineMeta.h"

namespace UE::Online { struct FOperationConfig; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online::Meta {

BEGIN_ONLINE_STRUCT_META(FOperationConfig)
	ONLINE_STRUCT_FIELD(FOperationConfig, CacheExpiration),
	ONLINE_STRUCT_FIELD(FOperationConfig, CacheExpirySeconds),
	ONLINE_STRUCT_FIELD(FOperationConfig, bCacheError)
END_ONLINE_STRUCT_META()

/* UE::Online::Meta*/ }
