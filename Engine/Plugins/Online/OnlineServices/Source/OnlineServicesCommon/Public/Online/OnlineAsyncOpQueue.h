// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Containers/Ticker.h"
#include "Containers/TransactionallySafeSpscQueue.h"
#include "HAL/PlatformTime.h"
#include "Online/OnlineAsyncOpHandle.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOp; }

namespace UE::Online {

class FOnlineAsyncOpQueue
{
public:
	FOnlineAsyncOpQueue()
	{
	}

	virtual ~FOnlineAsyncOpQueue()
	{
		for (TOptional<TSharedRef<IWrappedOp>> Operation = QueuedOperations.Dequeue(); Operation.IsSet(); Operation = QueuedOperations.Dequeue())
		{
			Operation.GetValue()->Cancel(Errors::Cancelled());
		}
	}

	template <typename OpType>
	void Enqueue(TOnlineAsyncOp<OpType>& Op, double DelayInSeconds = 0.0)
	{
		QueuedOperations.Enqueue(MakeShared<TWrappedOp<OpType>>(Op, DelayInSeconds));
		TryStartOperations();
	}

	virtual bool IsIdle() const = 0;

	virtual void TryStartOperations() = 0;

protected:
	class IWrappedOp
	{
	public:
		virtual ~IWrappedOp() = default;
		virtual void Start() = 0;
		virtual void Cancel(const FOnlineError& Reason) = 0;
		virtual bool IsComplete() const = 0;
		virtual TOnlineEvent<void(void)> OnComplete() = 0;
	};

	template <typename OpType>
	class TWrappedOp : public IWrappedOp, public TSharedFromThis<TWrappedOp<OpType>>
	{
	public:
		virtual ~TWrappedOp() = default;

		TWrappedOp(TOnlineAsyncOp<OpType>& InOp, double InDelayInSeconds)
			: Op(InOp.AsShared())
			, DelayInSeconds(InDelayInSeconds)
			, EnqueueCycles(InDelayInSeconds > 0.0 ? FPlatformTime::Cycles64() : 0)
		{
		}

		virtual void Start() override final
		{
			OnCompleteHandle = Op->OnComplete().Add([this, WeakThis = TWeakPtr<TWrappedOp<OpType>>(this->AsShared())](const TOnlineAsyncOp<OpType>&, const TOnlineResult<OpType>&)
			{
				TSharedPtr<TWrappedOp<OpType>> PinnedThis = WeakThis.Pin();
				if (PinnedThis.IsValid())
				{
					OnCompleteEvent.Broadcast();
				}
			});
			if (DelayInSeconds > 0.0)
			{
				const double ElapsedTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - EnqueueCycles);
				if (DelayInSeconds > ElapsedTime)
				{
					TickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("StartDelayedOnlineAsyncOp"), DelayInSeconds - ElapsedTime,
						[WeakOp = Op.ToWeakPtr()](float)
						{
							TSharedPtr<TOnlineAsyncOp<OpType>> PinnedOp = WeakOp.Pin();
							if (PinnedOp.IsValid())
							{
								PinnedOp->Start();
							}
							return false;
						});
				}
				else
				{
					Op->Start();
				}
			}
			else
			{
				Op->Start();
			}
		}

		virtual void Cancel(const FOnlineError& Reason) override final
		{
			Op->Cancel(Reason);

			if (TickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(TickerHandle);
			}
		}

		virtual bool IsComplete() const override final
		{
			return Op->IsComplete();
		}

		virtual TOnlineEvent<void(void)> OnComplete() override final
		{
			return OnCompleteEvent;
		}

	private:
		TSharedRef<TOnlineAsyncOp<OpType>> Op;
		const double DelayInSeconds = 0.0;
		const uint64 EnqueueCycles = 0;
		TOnlineEventCallable<void(void)> OnCompleteEvent;
		FOnlineEventDelegateHandle OnCompleteHandle;
		FTSTicker::FDelegateHandle TickerHandle;
	};

	TTransactionallySafeSpscQueue<TSharedRef<IWrappedOp>> QueuedOperations;
};

class FOnlineAsyncOpQueueParallel : public FOnlineAsyncOpQueue
{
public:
	virtual void TryStartOperations() override
	{
	}

	virtual bool IsIdle() const override 
	{ 
		return QueuedOperations.IsEmpty() && InFlightOperations.IsEmpty();
	}

	using FOnlineAsyncOpQueue::Enqueue;

	void Enqueue(TSharedRef<IWrappedOp>& Operation)
	{
		QueuedOperations.Enqueue(Operation);
	}

	void Tick(float DeltaSeconds)
	{
		while (InFlightOperations.Num() < MaxConcurrentOperations)
		{
			TOptional<TSharedRef<IWrappedOp>> Operation = QueuedOperations.Dequeue();
			if (!Operation.IsSet())
			{
				break;
			}

			InFlightOperations.Emplace(Operation.GetValue(), Operation.GetValue()->OnComplete().Add([this, WeakOp = TWeakPtr<IWrappedOp>(Operation.GetValue())]()
				{
					InFlightOperations.RemoveAll([WeakOp](const TPair<TSharedRef<IWrappedOp>, FOnlineEventDelegateHandle>& InFlightOperation)
						{
							return InFlightOperation.Key == WeakOp;
						});
				}));
			Operation.GetValue()->Start();
		}
	}

	void SetMaxConcurrentOperations(int InMaxConcurrentOperations) { check(InMaxConcurrentOperations > 0); MaxConcurrentOperations = InMaxConcurrentOperations; }

protected:
	TArray<TPair<TSharedRef<IWrappedOp>, FOnlineEventDelegateHandle>> InFlightOperations;
	int MaxConcurrentOperations = 16;
};

class FOnlineAsyncOpQueueSerial : public FOnlineAsyncOpQueue
{
public:
	FOnlineAsyncOpQueueSerial(FOnlineAsyncOpQueueParallel& InParentQueue)
		: ParentQueue(InParentQueue)
	{
	}

	~FOnlineAsyncOpQueueSerial()
	{
		if (InFlightOperation.IsValid())
		{
			ContinuationDelegateHandle.Unbind();
			InFlightOperation.Reset();
		}
	}

	virtual void TryStartOperations() override
	{
		if (!InFlightOperation.IsValid() || InFlightOperation->IsComplete())
		{
			if (TOptional<TSharedRef<IWrappedOp>> Operation = QueuedOperations.Dequeue())
			{
				InFlightOperation = Operation.GetValue();
				// push to the parent parallel queue
				ParentQueue.Enqueue(Operation.GetValue());
				// when the operation completes, start the next one
				ContinuationDelegateHandle = InFlightOperation->OnComplete().Add([this]()
					{
						InFlightOperation.Reset();
						TryStartOperations();
					});
			}
		}
	}

	virtual bool IsIdle() const override
	{
		return QueuedOperations.IsEmpty() && !InFlightOperation.IsValid();
	}

protected:
	FOnlineAsyncOpQueueParallel& ParentQueue;

	TSharedPtr<IWrappedOp> InFlightOperation;
	FOnlineEventDelegateHandle ContinuationDelegateHandle;
};

/* UE::Online */ }
