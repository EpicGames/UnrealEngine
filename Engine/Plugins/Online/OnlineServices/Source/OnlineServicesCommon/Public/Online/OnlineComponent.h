// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/IOnlineComponent.h"
#include "Online/OnlineExecHandler.h"
#include "Online/OnlineServicesCommon.h"

namespace UE::Online {

template <typename ComponentType>
class TOnlineComponent
	: public ComponentType
	, public IOnlineComponent
{
public:
	using Super = ComponentType;

	TOnlineComponent(const TOnlineComponent&) = delete;
	TOnlineComponent(TOnlineComponent&&) = delete;

	/**
	 */
	TOnlineComponent(const FString& ComponentName, FOnlineServicesCommon& InServices)
		: Services(InServices)
		, WeakThis(TSharedPtr<ComponentType>(InServices.AsShared(), static_cast<ComponentType*>(this)))
		, InterfaceName(ComponentName)
		, SerialQueue(InServices.GetParallelQueue())
		{
		}

	virtual void Initialize() override
	{
		Services.RegisterExecHandler(InterfaceName, MakeUnique<TOnlineComponentExecHandler<TOnlineComponent<ComponentType>>>(this));
		RegisterCommands();
		UpdateConfig();
	}
	virtual void PostInitialize() override {}
	virtual void UpdateConfig() override {}
	virtual void Tick(float DeltaSeconds) override {}
	virtual void PreShutdown() override {}
	virtual void Shutdown() override {}

	template <typename StructType>
	bool LoadConfig(StructType& Struct, const FString& OperationName = FString()) const
	{
		return GetServices().LoadConfig(Struct, GetConfigName(), OperationName);
	}

	const FString& GetConfigName() const { return InterfaceName; }

	template <typename OpType>
	TOnlineAsyncOpRef<OpType> GetOp(typename OpType::Params&& Params)
	{
		TOnlineAsyncOpRef<OpType> Op = Services.template GetOp<OpType>(MoveTemp(Params), GetConfigSectionHierarchy(OpType::Name));
		Op->SetInterfaceName(InterfaceName);
		return Op;
	}

	template <typename OpType, typename ParamsFuncsType = TJoinableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetJoinableOp(typename OpType::Params&& Params)
	{
		TOnlineAsyncOpRef<OpType> Op = Services.template GetJoinableOp<OpType, ParamsFuncsType>(MoveTemp(Params), GetConfigSectionHierarchy(OpType::Name));
		Op->SetInterfaceName(InterfaceName);
		return Op;
	}

	template <typename OpType, typename ParamsFuncsType = TMergeableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetMergeableOp(typename OpType::Params&& Params)
	{
		TOnlineAsyncOpRef<OpType> Op = Services.template GetMergeableOp<OpType, ParamsFuncsType>(MoveTemp(Params), GetConfigSectionHierarchy(OpType::Name));
		Op->SetInterfaceName(InterfaceName);
		return Op;
	}

	template <typename ServicesType = FOnlineServicesCommon>
	ServicesType& GetServices()
	{
		return static_cast<ServicesType&>(Services);
	}

	template <typename ServicesType = FOnlineServicesCommon>
	const ServicesType& GetServices() const
	{
		return static_cast<const ServicesType&>(Services);
	}

	/* Queue for executing tasks in serial */
	FOnlineAsyncOpQueue& GetSerialQueue()
	{
		return SerialQueue;
	}

	/* Queues for executing per-user tasks in serial */
	FOnlineAsyncOpQueue& GetSerialQueue(const FAccountId& AccountId)
	{
		TUniquePtr<FOnlineAsyncOpQueueSerial>* Queue = PerUserSerialQueue.Find(AccountId);
		if (Queue == nullptr)
		{
			Queue = &PerUserSerialQueue.Emplace(AccountId, MakeUnique<FOnlineAsyncOpQueueSerial>(GetServices().GetParallelQueue()));
		}

		return **Queue;
	}

	UE_DEPRECATED(5.6, "GetConfigSectionHeiarchy has been renamed GetConfigSectionHierarchy")
	TArray<FString> GetConfigSectionHeiarchy(const FString& OperationName = FString())
	{
		return GetConfigSectionHierarchy(OperationName);
	}

	UE_DEPRECATED(5.8, "GetConfigSectionHeirarchy has been renamed GetConfigSectionHierarchy")
	TArray<FString> GetConfigSectionHeirarchy(const FString& OperationName = FString())
	{
		return GetConfigSectionHierarchy(OperationName);
	}

	TArray<FString> GetConfigSectionHierarchy(const FString& OperationName = FString())
	{
		TArray<FString> SectionHierarchy;
		FString SectionName = TEXT("OnlineServices");
		SectionHierarchy.Add(SectionName);
		SectionName += TEXT(".") + GetServices().GetServiceConfigName();
		SectionHierarchy.Add(SectionName);
		SectionName += TEXT(".") + GetConfigName();
		SectionHierarchy.Add(SectionName);
		if (!OperationName.IsEmpty())
		{
			SectionName += TEXT(".") + OperationName;
			SectionHierarchy.Add(SectionName);
		}
		return SectionHierarchy;
	}

	virtual void RegisterCommands() {}

	// Default handler: generic parsing of Params, log Result
	template <typename T>
	void RegisterCommand(T MemberFunction)
	{
		using InterfaceType = typename Private::TOnlineInterfaceOperationMemberFunctionPtrTraits<T>::InterfaceType;
		static_assert(std::is_base_of_v<ComponentType, InterfaceType> || std::is_same_v<ComponentType, InterfaceType>);
		
		RegisterCommand(static_cast<InterfaceType*>(this), MemberFunction);
	}

	// Default handler: generic parsing of Params, log Result
	template <typename InterfaceType, typename T>
	void RegisterCommand(InterfaceType* Interface, T MemberFunction)
	{
		TOnlineInterfaceOperationExecHandler<InterfaceType, T>* Handler = new TOnlineInterfaceOperationExecHandler<InterfaceType, T>(Interface, MemberFunction);

		const TCHAR* Name = Private::TOnlineInterfaceOperationMemberFunctionPtrTraits<T>::OpType::Name;
		ExecCommands.Emplace(Name, Handler);
	}

	// Custom handler
	void RegisterExecHandler(const FString& Name, TUniquePtr<IOnlineExecHandler>&& Handler)
	{
		ExecCommands.Emplace(Name, MoveTemp(Handler));
	}

	bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		FString Command;
		if (FParse::Token(Cmd, Command, false))
		{
			if (TUniquePtr<IOnlineExecHandler>* ExecHandler = ExecCommands.Find(Command))
			{
				return (*ExecHandler)->Exec(World, Cmd, Ar);
			}
			else if (Command == TEXT("HELP"))
			{
				return Help(World, Cmd, Ar);
			}
		}

		return false;
	}

	bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar)
	{
		for (const TPair<FString, TUniquePtr<IOnlineExecHandler>>& Command : ExecCommands)
		{
			Command.Value->Help(World, Cmd, Ar);
		}

		return false;
	}

	// TSharedFromThis-like behavior
	TSharedRef<ComponentType> AsShared()
	{
		TSharedPtr<ComponentType> SharedThis(WeakThis.Pin());
		check(SharedThis.Get() == this);
		return MoveTemp(SharedThis).ToSharedRef();
	}

	TSharedRef<ComponentType const> AsShared() const
	{
		TSharedPtr<ComponentType const> SharedThis(WeakThis.Pin());
		check(SharedThis.Get() == this);
		return MoveTemp(SharedThis).ToSharedRef();
	}

	TWeakPtr<ComponentType> AsWeak()
	{
		return WeakThis;
	}

	TWeakPtr<ComponentType const> AsWeak() const
	{
		return WeakThis;
	}

protected:
	template <class OtherType>
	static TSharedRef<OtherType> SharedThis(OtherType* ThisPtr)
	{
		return StaticCastSharedRef<OtherType>(ThisPtr->AsShared());
	}

	template <class OtherType>
	static TSharedRef<OtherType const> SharedThis(const OtherType* ThisPtr)
	{
		return StaticCastSharedRef<OtherType const>(ThisPtr->AsShared());
	}

	FOnlineServicesCommon& Services;

private:
	TWeakPtr<ComponentType> WeakThis;
	FString InterfaceName;
	TMap<FString, TUniquePtr<IOnlineExecHandler>> ExecCommands;

	FOnlineAsyncOpQueueSerial SerialQueue;
	TMap<FAccountId, TUniquePtr<FOnlineAsyncOpQueueSerial>> PerUserSerialQueue;
};

/* UE::Online */ }
