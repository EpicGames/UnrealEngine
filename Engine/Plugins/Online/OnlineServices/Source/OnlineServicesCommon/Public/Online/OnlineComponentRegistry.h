// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Online/OnlineTypeInfo.h"
#include "Online/OnlineMeta.h"
#include "Templates/SharedPointer.h"

namespace UE::Online { class IOnlineComponent; }

namespace UE::Online {

class FOnlineComponentRegistry
{
public:
	/**
	 * Create and register a component of type ComponentType if one has not already been registered
	 */
	template <typename ComponentType, typename... ParamTypes>
	void Register(ParamTypes&&... Params)
	{
		if (Get<ComponentType>() == nullptr)
		{
			TUniquePtr<IComponentWrapper> Component = MakeUnique<TComponentWrapper<ComponentType>>(Forward<ParamTypes>(Params)...);

			Components.Add(MoveTemp(Component));
		}
	}

	/**
	 * Destroy and unregister a component of type ComponentType
	 */
	template <typename ComponentType>
	void Unregister()
	{
		Components.RemoveAll([](const TUniquePtr<IComponentWrapper>& WrappedComponent)
		{
			return WrappedComponent->IsA(TOnlineTypeInfo<ComponentType>::GetTypeName());
		});
	}

	/**
	 * Get the component of type ComponentType
	 * @return The component or null if one of ComponentType has not been registered
	 */
	template <typename ComponentType>
	ComponentType* Get() const
	{
		const FOnlineTypeName TypeName = TOnlineTypeInfo<ComponentType>::GetTypeName();
		const TUniquePtr<IComponentWrapper>* WrappedComponent = Components.FindByPredicate([&TypeName](const TUniquePtr<IComponentWrapper>& WrappedComponent) { return WrappedComponent->IsA(TypeName); });
		if (WrappedComponent)
		{
			typename Meta::TBaseClass<ComponentType>::Type* BasePtr = static_cast<typename Meta::TBaseClass<ComponentType>::Type*>((*WrappedComponent)->GetBase());
			return static_cast<ComponentType*>(BasePtr);
		}
		return nullptr;
	}

	// Internal use only. Used to implement IOnlineServices::GetInterface()
	void AssignBaseSharedPtr(const FOnlineTypeName& TypeName, void* OutBaseInterfaceSP)
	{
		const TUniquePtr<IComponentWrapper>* WrappedComponent = Components.FindByPredicate([&TypeName](const TUniquePtr<IComponentWrapper>& WrappedComponent) { return WrappedComponent->IsA(TypeName); });
		if (WrappedComponent)
		{
			(*WrappedComponent)->AssignBaseSharedPtr(OutBaseInterfaceSP);
		}
	}

	/**
	 * Call a callable for each of the components
	 */
	template <typename CallableType, typename... ParamTypes>
	void Visit(CallableType&& Callable, ParamTypes&&... Params)
	{
		for (TUniquePtr<IComponentWrapper>& Component : Components)
		{
			Invoke(Callable, *Component, Forward<ParamTypes>(Params)...);
		}
	}

private:
	class IComponentWrapper
	{
	public:
		virtual ~IComponentWrapper() = default;
		virtual void* GetBase() = 0;
		virtual void AssignBaseSharedPtr(void* OutSP) = 0;
		virtual bool IsA(FOnlineTypeName TypeName) const = 0;
		virtual FOnlineTypeName GetTypeName() const = 0;
		virtual IOnlineComponent* Get() = 0;
		virtual const IOnlineComponent* Get() const = 0;
		IOnlineComponent& operator*() { return *Get(); }
		const IOnlineComponent& operator*() const { return *Get(); }
		IOnlineComponent* operator->() { return Get(); }
		const IOnlineComponent* operator->() const { return Get(); }
	};

	template <typename T>
	class TComponentWrapper : public IComponentWrapper
	{
	public:
		template <typename... TArgs>
		TComponentWrapper(TArgs&&... Args)
			: Object(Forward<TArgs>(Args)...)
		{
		}

		virtual ~TComponentWrapper() = default;

		virtual void* GetBase() override final
		{
			return static_cast<void*>(static_cast<Meta::TBaseClass_T<T>*>(&Object));
		}

		virtual void AssignBaseSharedPtr(void* OutSP) override final
		{
			// assumes that component has AsShared that returns a shared pointer for the base class
			TSharedPtr<Meta::TBaseClass_T<T>>& SP = *static_cast<TSharedPtr<Meta::TBaseClass_T<T>>*>(OutSP);
			SP = Object.AsShared();
		}

		virtual bool IsA(FOnlineTypeName TypeName) const override final
		{
			return TOnlineTypeInfo<T>::IsA(TypeName);
		}

		virtual FOnlineTypeName GetTypeName() const override final
		{
			return TOnlineTypeInfo<T>::GetTypeName();
		}

		virtual IOnlineComponent* Get() override final
		{
			return &Object;
		}

		virtual const IOnlineComponent* Get() const override final
		{
			return &Object;
		}

	private:
		T Object;
	};

	TArray<TUniquePtr<IComponentWrapper>> Components;
};

/* UE::Online */ }
