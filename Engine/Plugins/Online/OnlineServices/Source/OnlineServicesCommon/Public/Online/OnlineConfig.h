// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Online/OnlineMeta.h"
#include "Online/OnlineServicesLog.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online { class FSchemaVariant; }

namespace UE::Online {

namespace Private {

template <typename T> constexpr bool MissingCase = false;

template <typename T>
T ParseValueFromString(const FString& StringValue)
{
	if constexpr (std::is_same_v<T, FText>)
	{
		FText TextValue;
		FTextStringHelper::ReadFromBuffer(*StringValue, TextValue);
		return TextValue;
	}
	else if constexpr (std::is_same_v<T, FName>)
	{
		return FName(*StringValue);
	}
	else if constexpr (std::is_same_v<T, bool>)
	{
		return FCString::ToBool(*StringValue);
	}
	else if constexpr (std::is_same_v<T, int32>)
	{
		return FCString::Strtoi(*StringValue, nullptr, 10);
	}
	else if constexpr (std::is_same_v<T, uint32>)
	{
		return (uint32)FCString::Strtoui64(*StringValue, nullptr, 10);
	}
	else if constexpr (std::is_same_v<T, int64>)
	{
		return FCString::Strtoi64(*StringValue, nullptr, 10);
	}
	else if constexpr (std::is_same_v<T, uint64>)
	{
		return FCString::Strtoui64(*StringValue, nullptr, 10);
	}
	else if constexpr (std::is_same_v<T, float>)
	{
		return FCString::Atof(*StringValue);
	}
	else if constexpr (std::is_same_v<T, double>)
	{
		return FCString::Atod(*StringValue);
	}
	else if constexpr (std::is_enum_v<T>
		|| std::is_same_v<T, FSchemaVariant>)
	{
		T OutValue;
		using ::LexFromString;
		LexFromString(OutValue, *StringValue);
		return OutValue;
	}
	else
	{
		static_assert(MissingCase<T>, "ParseValueFromString missing implementation for type T");
	}
}

/* Private */ }

class IOnlineConfigStruct;
using IOnlineConfigStructPtr = TSharedPtr<IOnlineConfigStruct>;

class IOnlineConfigStruct
{
public:
	virtual ~IOnlineConfigStruct() = default;

	virtual bool GetValue(const TCHAR* Key, FString& Value) = 0;
	virtual int32 GetValue(const TCHAR* Key, TArray<FString>& Value) = 0;
	virtual bool GetValue(const TCHAR* Key, IOnlineConfigStructPtr& Value) = 0;
	virtual int32 GetValue(const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value) = 0;

	template <typename T>
	bool GetValue(const TCHAR* Key, T& Value)
	{
		if constexpr (TModels_V<Meta::COnlineMetadataAvailable, T>)
		{
			IOnlineConfigStructPtr ConfigStructPtr;
			if (GetValue(Key, ConfigStructPtr))
			{
				T Result;
				Meta::VisitFields(Result,
					[&ConfigStructPtr](const TCHAR* FieldName, auto& Field)
				{
					ConfigStructPtr->GetValue(FieldName, Field);
				});
				Value = MoveTemp(Result);
				return true;
			}
		}
		else
		{
			FString StringValue;
			if (GetValue(Key, StringValue))
			{
				Value = Private::ParseValueFromString<T>(StringValue);
				return true;
			}
		}
		return false;
	}

	template <typename T>
	int32 GetValue(const TCHAR* Key, TArray<T>& Value)
	{
		Value.Empty();

		if constexpr (TModels_V<Meta::COnlineMetadataAvailable, T>)
		{
			TArray<IOnlineConfigStructPtr> ConfigStructPtrArray;
			if (GetValue(Key, ConfigStructPtrArray) > 0)
			{
				Value.Reserve(ConfigStructPtrArray.Num());
				for (const IOnlineConfigStructPtr& ConfigStructPtr : ConfigStructPtrArray)
				{
					T& Struct = Value.Emplace_GetRef();
					Meta::VisitFields(Struct,
						[&ConfigStructPtr](const TCHAR* FieldName, auto& Field)
					{
						ConfigStructPtr->GetValue(FieldName, Field);
					});
				}
			}
		}
		else
		{
			TArray<FString> StringArray;
			if (GetValue(Key, StringArray) > 0)
			{
				Value.Reserve(StringArray.Num());
				for (const FString& StringValue : StringArray)
				{
					Value.Emplace(Private::ParseValueFromString<T>(StringValue));
				}
			}
		}

		return Value.Num();
	}
};

/**
 * Interface for retrieving config values used by OnlineServices implementations
 * 
 * For all the GetValue functions, Section will correspond to the section in an ini file, but can be
 * interpreted in other ways by an IOnlineConfigProvider implementation to allow for OnlineServices
 * implementations to use an alternative configuration file.
 * 
 * By convention, all sections of the OnlineServices implementations will take the following format:
 *   OnlineServices.<ServicesProvider>.<OptionalInterface>.<OptionalOperation> <OptionalOverride>
 * For example, valid sections include the following:
 *   OnlineServices.Null
 *   OnlineServices.Null.Auth
 *   OnlineServices.Null.Auth.Login
 *   OnlineServices.Null.Auth.Login Prod
 * 
 * Implementations must at a minimum implement the FString and TArray<FString> GetValue. The others are
 * optional, and the default implementation will convert strings to the appropriate data type, but can 
 * be overridden in cases where the underlying configuration system stores integers, floats, etc directly
 * instead of as strings
 */
class IOnlineConfigProvider
{
public:
	virtual ~IOnlineConfigProvider() = default;

	/**
	 * Get a FString value
	 * 
	 * @param Section Section to read the value from
	 * @param Key Key in the section for the value
	 * @param Value FString value read from the config. Will be unchanged if not present
	 * 
	 * @return true if a value was read
	 */
	virtual bool GetValue(const TCHAR* Section, const TCHAR* Key, FString& Value) = 0;

	/**
	 * Get an value consisting of a TArray of FStrings
	 *
	 * @param Section Section to read the value from
	 * @param Key Key in the section for the value
	 * @param Value Array of FStrings read from the config. Will be empty if not present
	 *
	 * @return number of values in the array
	 */
	virtual int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value) = 0;

	virtual bool GetValue(const TCHAR* Section, const TCHAR* Key, IOnlineConfigStructPtr& Value) = 0;

	virtual int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value) = 0;

	template <typename T>
	bool GetValue(const TCHAR* Section, const TCHAR* Key, T& Value)
	{
		if constexpr (TModels_V<Meta::COnlineMetadataAvailable, T>)
		{
			IOnlineConfigStructPtr ConfigStructPtr;
			if (GetValue(Section, Key, ConfigStructPtr))
			{
				T Result;
				Meta::VisitFields(Result,
					[&ConfigStructPtr](const TCHAR* FieldName, auto& Field)
				{
					ConfigStructPtr->GetValue(FieldName, Field);
				});
				Value = MoveTemp(Result);
				return true;
			}
		}
		else
		{
			FString StringValue;
			if (GetValue(Section, Key, StringValue))
			{
				Value = Private::ParseValueFromString<T>(StringValue);
				return true;
			}
		}
		return false;
	}

	template <typename T>
	int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<T>& Value)
	{
		int32 NumInitialElements = Value.Num();
		if constexpr (TModels_V<Meta::COnlineMetadataAvailable, T>)
		{
			TArray<IOnlineConfigStructPtr> ConfigStructPtrArray;
			if (GetValue(Section, Key, ConfigStructPtrArray) > 0)
			{
				Value.Reserve(ConfigStructPtrArray.Num() + NumInitialElements);
				for (const IOnlineConfigStructPtr& ConfigStructPtr : ConfigStructPtrArray)
				{
					T& Struct = Value.Emplace_GetRef();
					Meta::VisitFields(Struct,
						[&ConfigStructPtr](const TCHAR* FieldName, auto& Field)
					{
						ConfigStructPtr->GetValue(FieldName, Field);
					});
				}
			}
		}
		else
		{
			TArray<FString> StringArray;
			if (GetValue(Section, Key, StringArray) > 0)
			{
				Value.Reserve(StringArray.Num() + NumInitialElements);
				for (const FString& StringValue : StringArray)
				{
					Value.Emplace(Private::ParseValueFromString<T>(StringValue));
				}
			}
		}

		return Value.Num() - NumInitialElements;
	}
};

class FOnlineConfigStructGConfig : public IOnlineConfigStruct
{
public:
	virtual ~FOnlineConfigStructGConfig() = default;

	UE_API virtual bool GetValue(const TCHAR* Key, FString& Value) override;
	UE_API virtual int32 GetValue(const TCHAR* Key, TArray<FString>& Value) override;
	UE_API virtual bool GetValue(const TCHAR * Key, IOnlineConfigStructPtr& Value) override;
	UE_API virtual int32 GetValue(const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value) override;

private:
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FOnlineConfigStructGConfig(FPrivateToken)
	{
	}

private:
	friend class FOnlineConfigProviderGConfig;

	static UE_API IOnlineConfigStructPtr CreateStruct(const FString& InConfigValue);
	static UE_API TArray<IOnlineConfigStructPtr> CreateStructArray(const FString& InConfigValue);
	static UE_API const TCHAR* ParseStruct(const TCHAR* InStr, IOnlineConfigStructPtr& OutPtr);
	static UE_API TArray<FString> CreateValueArray(const FString& InConfigValue);

	TMap<FString, FString> StructMembers;
};

/**
 * IOnlineConfigProvider implementation that uses GConfig to retrieve values
 */
class FOnlineConfigProviderGConfig : public IOnlineConfigProvider
{
public:
	FOnlineConfigProviderGConfig(const FString& InConfigFile)
		: ConfigFile(InConfigFile)
	{
	}
	virtual ~FOnlineConfigProviderGConfig() = default;

	UE_API virtual bool GetValue(const TCHAR* Section, const TCHAR* Key, FString& Value) override;
	UE_API virtual int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<FString>& Value) override;
	UE_API virtual bool GetValue(const TCHAR* Section, const TCHAR* Key, IOnlineConfigStructPtr& Value) override;
	UE_API virtual int32 GetValue(const TCHAR* Section, const TCHAR* Key, TArray<IOnlineConfigStructPtr>& Value) override;

private:
	FString ConfigFile;
};

namespace Private {

template <typename T>
bool LoadConfigStructMemberValue(IOnlineConfigStruct& ConfigStruct, const TCHAR* Key, T& Value)
{
	if constexpr (std::is_same_v<decltype(ConfigStruct.GetValue(Key, Value)), bool>)
	{
		return ConfigStruct.GetValue(Key, Value);
	}
	else
	{
		static_assert(std::is_same_v<decltype(ConfigStruct.GetValue(Key, Value)), int32>, "IOnlineConfigStruct::GetValue must return bool or int32");
		return ConfigStruct.GetValue(Key, Value) > 0;
	}
}

template <typename T>
bool LoadConfigValue(IOnlineConfigProvider& Provider, const TCHAR* Section, const TCHAR* Key, T& Value)
{
	if constexpr (std::is_same_v<decltype(Provider.GetValue(Section, Key, Value)), bool>)
	{
		return Provider.GetValue(Section, Key, Value);
	}
	else 
	{
		static_assert(std::is_same_v<decltype(Provider.GetValue(Section, Key, Value)), int32>, "IOnlineConfigProvider::GetValue must return bool or int32");
		return Provider.GetValue(Section, Key, Value) > 0;
	}
}

/* Private */ }

/**
 * Populate a struct from a config provider. Loads each field in the struct by matching field names with keys in the config section.
 * Requires that the type metadata is specified for StructType
 *
 * @param Provider The config provider
 * @param Section The section of the config to load the values from
 * @param Value The struct whose fields will be loaded from config
 *
 * @return true if any values were loaded
 */
template <typename T>
auto LoadConfig(IOnlineConfigProvider& Provider, const FString& Section, T& Value)
	-> std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool>
{
	bool bLoadedValue = false;
	Meta::VisitFields(Value,
		[&Provider, &Section, &bLoadedValue](const TCHAR* FieldName, auto& Field)
	{
		bLoadedValue |= Private::LoadConfigValue(Provider, *Section, FieldName, Field);
	});
	return bLoadedValue;
}

/**
 * Populate a struct from a config provider. Iterates through each section in SectionHierarchy, loading each field in the struct by matching field names with keys in the section.
 * Requires that the type metadata is specified for StructType
 *
 * @param Provider The config provider
 * @param SectionHierarchy The config sections to load values from
 * @param Value The struct whose fields will be loaded from config
 *
 * @return true if any values were loaded
 */
template <typename T>
auto LoadConfig(IOnlineConfigProvider& Provider, const TArray<FString>& SectionHierarchy, T& Value)
	-> std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool>
{
	bool bLoadedValue = false;
	for (const FString& Section : SectionHierarchy)
	{
		bool bLoadedInCurrentSection = LoadConfig(Provider, Section, Value);
		if (bLoadedInCurrentSection && Section.Contains(" "))
		{
			UE_LOGF(LogOnlineServices, Warning, "Space in online config section has been deprecated, use : instead in [%ls]", *Section);
		}
		bLoadedValue |= bLoadedInCurrentSection;
	}
	return bLoadedValue;
}

/**
 * Populate a struct from a config provider. Loads the struct from the _value_ of the given key in the given config section.
 * Requires that the type metadata is specified for StructType, and that the config value is in config struct syntax.
 * 
 * @param Provider The config provider
 * @param Section The section of the config where the struct resides
 * @param Key The key in the config section where the struct resides
 * @param Value The struct whose fields will be loaded from config
 * 
 * @return true if any values were loaded
 */
template <typename T>
auto LoadConfig(IOnlineConfigProvider& Provider, const FString& Section, const TCHAR* Key, T& OutValue)
-> std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool>
{
	bool bLoadedValue = false;
	IOnlineConfigStructPtr ConfigStructPtr;
	if (Provider.GetValue(*Section, Key, ConfigStructPtr))
	{
		T Value;
		Meta::VisitFields(Value,
			[&ConfigStructPtr, &bLoadedValue](const TCHAR* FieldName, auto& Field)
		{
			bLoadedValue |= Private::LoadConfigStructMemberValue(*ConfigStructPtr, FieldName, Field);
		});
		if (bLoadedValue)
		{
			OutValue = MoveTemp(Value);
		}
	}
	return bLoadedValue;
}

/**
 * Populate a struct from a config provider. Iterates through each section in SectionHierarchy, loading the struct from the _value_ of the given key in the given config section.
 * Requires that the type metadata is specified for StructType, and that the config value is in config struct syntax.
 *
 * @param Provider The config provider
 * @param SectionHierarchy The config sections to load values from
 * @param Key The key in the config section where the struct resides
 * @param Value The struct whose fields will be loaded from config
 *
 * @return true if any values were loaded
 */
template <typename T>
auto LoadConfig(IOnlineConfigProvider& Provider, const TArray<FString>& SectionHierarchy, const TCHAR* Key, T& Value)
-> std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool>
{
	bool bLoadedValue = false;
	for (const FString& Section : SectionHierarchy)
	{
		bLoadedValue |= LoadConfig(Provider, Section, Key, Value);
	}
	return bLoadedValue;
}

/* UE::Online */ }

#undef UE_API
