// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Misc/DateTime.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineServicesLog.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineResult.h"
#include "Templates/Function.h"

#include "Online/Auth.h"

#include "Online/OnlineServices.h"
#include "Trace/Trace.inl"

template <typename OptionalType> struct TOptional;

/*
	This file facilitates online command execution.
	Online commands must be prefixed with OnlineServices, and Index=# where the # is which service you want in context
		- The list of numbers can be obtained with OnlineServices List
	Next argument is the interface- i.e. "Auth", "Presence"
	Next argument is the name of the function being called, i.e. "QueryPresence"
	Arguments after that are the function's params in declared order. Special types are as follows:
		for FAccountId, either pass in 0-9 to grab the nth local user, or pass in the output from a handle's tostring that looks like "epic:3"
		for TSharedPtr, simply type in the parameter as normal
			- TSharedRef *technically* is implemented however will crash because you cannot default construct shared refs. At the moment, we recommend all console commands with shared ptr parameters to use tsharedptr instead of tsharedref.
		for TOptional, pass in "null" and the optional will be unset
		for TVariant, the syntax is type:value. (example: i:350) Currently implemented types below:
			- s: for string
			- i: for int(64)
			- i32 for int32
			- b for bool
			- s for string
			- u for user (FAccountId)
			- d for double
			- f for float
			- e for enum
				- note- if you have a custom enum, you must use the macro MAKE_VARIANT_ENUM_INFO(YourEnumName) for it to work properly
				- if you have more than one enum you must qualify the entire enum name. using e: with a variant with more than one enum is undefined behavior
					- e.g. "ELoginStatus:Unknown"
		for objects (with online metadata), you can use the {} syntax and declare all parameters flatly
			- i.e. {true "AuthLogin" s:username s:password}
			- objects support internal objects/arrays, e.g. {{5 3} {4 false}}
		for TArray, you can use [] syntax
			- "[5, 3, 7, 9]"
			- commas are optional- [5 3 7 9] is also valid
			- arrays support internal objects/recursive arrays, e.g. [[2,3], [4,5]]
		for TMap, you can use {} syntax. 
			- {licenseData=licenseDataText, kNumUsers=5}
			- please note: TMap does not currently support rich syntax so nested objects/arrays/etc may not parse properly

	Full command examples:
		OnlineServices Index=0 Auth Login Null username s:pwd s:token []
		OnlineServices Index=0 Presence UpdatePresence 0 {0 Unknown Unknown Unknown Test1 Test2 {a=b}}
		OnlineServices Index=0 Presence QueryPresence 0 0 true
*/

class UWorld;
class FOutputDevice;

namespace UE::Online {

class IOnlineExecHandler
{
public:
	virtual ~IOnlineExecHandler() = default;
	virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) = 0;
	virtual bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) = 0;
};

namespace Private {

template <typename T>
struct TOnlineInterfaceOperationMemberFunctionPtrTraits;

template <typename TInterface, typename TOp>
struct TOnlineInterfaceOperationMemberFunctionPtrTraits<TOnlineResult<TOp>(TInterface::*)(typename TOp::Params&&)>
{
	using InterfaceType = TInterface;
	using OpType = TOp;
	static constexpr bool bAsync = false;
};

template <typename TInterface, typename TOp>
struct TOnlineInterfaceOperationMemberFunctionPtrTraits<TOnlineResult<TOp>(TInterface::*)(typename TOp::Params&&) const>
{
	using InterfaceType = TInterface;
	using OpType = TOp;
	static constexpr bool bAsync = false;
};

template <typename TInterface, typename TOp>
struct TOnlineInterfaceOperationMemberFunctionPtrTraits<TOnlineAsyncOpHandle<TOp>(TInterface::*)(typename TOp::Params&&)>
{
	using InterfaceType = TInterface;
	using OpType = TOp;
	static constexpr bool bAsync = true;
};

template <typename TInterface, typename TOp>
struct TOnlineInterfaceOperationMemberFunctionPtrTraits<TOnlineAsyncOpHandle<TOp>(TInterface::*)(typename TOp::Params&&) const>
{
	using InterfaceType = TInterface;
	using OpType = TOp;
	static constexpr bool bAsync = true;
};

// Forward declarations for dependent type resolution
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FString& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FName& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint8& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, int32& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint32& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, int64& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint64& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, bool& Value, IOnlineServices* Services = nullptr);
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FDateTime& Value, IOnlineServices* Services = nullptr);
template <typename T> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TArray<T>& Array, IOnlineServices* Services = nullptr);
template <typename T, typename U> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TMap<T, U>& Map, IOnlineServices* Services = nullptr);
template <typename T> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSet<T>& Set, IOnlineServices* Services = nullptr);
template <typename T> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSharedPtr<T>& Ptr, IOnlineServices* Services = nullptr);
template <typename T> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSharedRef<T>& Ref, IOnlineServices* Services = nullptr);
template <typename T> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TOptional<T>& Optional, IOnlineServices* Services = nullptr);
template <typename... Ts> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TVariant<Ts...>& Variant, IOnlineServices* Services = nullptr);
template <typename IdType> inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TOnlineId<IdType>& Value, IOnlineServices* Services = nullptr);
template <typename T> std::enable_if_t<!TModels_V<Meta::COnlineMetadataAvailable, T>, bool> ParseOnlineExecParams(const TCHAR*& Cmd, T& Value, IOnlineServices* Services = nullptr);
template <typename T> std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool> ParseOnlineExecParams(const TCHAR*& Cmd, T& Value, IOnlineServices* Services = nullptr);

template<typename T, typename VariantObject>
struct TOnlineVariantVisitInfo
{
	// todo: how to get this to work without it triggering on every class?
	//static_assert(sizeof(T) == -1, "If you are hitting this, you are using a TVariant with a value that does not have a TOnlineVariantVisitInfo! Make sure you are adding a TOnlineVariantVisitInfo for your class! (Check LastType or FirstType below for the type that needs attention)");

public:
	inline static const TCHAR* Prefixes[] = {TEXT("[GARBAGE]")};

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		return false;
	}
};

template< typename VariantObject >
struct TOnlineVariantVisitInfo<int64, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("i"),TEXT("i64"),TEXT("int"),TEXT("int64") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<int64>(FCString::Atoi64(*VariantValue));
		return true;
	}
};

template< typename VariantObject >
struct TOnlineVariantVisitInfo<int32, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("i32"),TEXT("int32") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<int32>(FCString::Atoi(*VariantValue));
		return true;
	}
};


template< typename VariantObject >
struct TOnlineVariantVisitInfo<double, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("d"),TEXT("double") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<double>(FCString::Atod(*VariantValue));
		return true;
	}
};

template< typename VariantObject >
struct TOnlineVariantVisitInfo<float, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("f"),TEXT("float") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<float>(FCString::Atof(*VariantValue));
		return true;
	}
};

template< typename VariantObject >
struct TOnlineVariantVisitInfo<bool, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("b"),TEXT("bool"),TEXT("boolean") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<bool>(FCString::ToBool(*VariantValue));
		return true;
	}
};


template< typename VariantObject >
struct TOnlineVariantVisitInfo<FString, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = { TEXT("s"),TEXT("str"),TEXT("string") };

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		Variant.template Set<FString>(VariantValue);
		return true;
	}
};

template< typename VariantObject >
struct TOnlineVariantVisitInfo<FAccountId, VariantObject>
{
public:
	inline static const TCHAR* Prefixes[] = {TEXT("u"),TEXT("user")};

	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)
	{
		FAccountId AccountId;
		const TCHAR* ValueTCHAR = *VariantValue;
		if (ParseOnlineExecParams<TOnlineId<OnlineIdHandleTags::FAccount>>(ValueTCHAR, AccountId, Services))
		{
			Variant.template Set<FAccountId>(AccountId);
			return true;
		}
		return false;
	}
};


#define MAKE_VARIANT_ENUM_INFO_SHORTNAME(EnumName, ShortName) template< typename VariantObject >\
struct UE::Online::Private::TOnlineVariantVisitInfo<EnumName, VariantObject>\
{\
public:\
	inline static const TCHAR* Prefixes[] = {TEXT(#ShortName),TEXT("e")};\
	static bool Assign(FString& VariantValue, VariantObject& Variant, IOnlineServices* Services)\
	{\
		EnumName Status;\
		LexFromString(Status, *VariantValue);\
		Variant.template Set<EnumName>(Status);\
		return true;\
	}\
};

#define MAKE_VARIANT_ENUM_INFO(EnumName) MAKE_VARIANT_ENUM_INFO_SHORTNAME(EnumName, EnumName);


inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FString& Value, IOnlineServices* Services)
{
	if (!FParse::Token(Cmd, Value, true))
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FName& Value, IOnlineServices* Services)
{
	FString StringValue;
	if (!ParseOnlineExecParams(Cmd, StringValue))
	{
		return false;
	}

	Value = FName(StringValue);
	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FUtf8String& Value, IOnlineServices* Services = nullptr)
{
	FString StringValue;
	if (!ParseOnlineExecParams(Cmd, StringValue))
	{
		return false;
	}

	Value = StringCast<UTF8CHAR>(*StringValue, StringValue.Len());
	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint8& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = static_cast<uint8>(FCString::Strtoui64(*Token, nullptr, 10));
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, int32& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = FCString::Strtoi(*Token, nullptr, 10);
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint32& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = static_cast<uint32>(FCString::Strtoui64(*Token, nullptr, 10));
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, int64& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = FCString::Strtoi64(*Token, nullptr, 10);
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, uint64& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = FCString::Strtoui64(*Token, nullptr, 10);
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, bool& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = FCString::ToBool(*Token);
	}
	else
	{
		return false;
	}

	return true;
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FDateTime& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		return FDateTime::Parse(Token, Value);
	}
	else
	{
		return false;
	}
}

inline bool ParseOnlineExecParams(const TCHAR*& Cmd, FPlatformUserId& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		Value = FPlatformMisc::GetPlatformUserForUserIndex(FCString::Strtoi(Cmd, nullptr, 10));
	}
	else
	{
		return false;
	}

	return true;
}

inline TCHAR PeekNextChar(const TCHAR* Cmd)
{
	while (FChar::IsWhitespace(*Cmd))
	{
		Cmd++;
	}
	return *Cmd;
}

// taken from FParse::Token, functionality added
// (possibly can be merged back?)
inline bool ParseTokenExt(const TCHAR*& Str, FString& Arg, bool UseEscape, bool bCommaSeparated = false, bool bIncludeDelimiters = false)
{
	Arg.Empty();

	// Skip preceeding spaces and tabs. (edit: also optional comma removal)
	while (FChar::IsWhitespace(*Str) || (bCommaSeparated && *Str == TEXT(',')))
	{
		Str++;
	}

	// EDITS Begin
	bool bHasEnteredObject = false;
	// Support for defining a nested object-like syntax, e,g, {Cats Dogs {Tigers Bears}}- ParseTokenExt will return "Cats Dogs {Tigers Bears}"
	if (*Str == TEXT('{') || *Str == TEXT('['))
	{
		bHasEnteredObject = true;
		TCHAR Incrementor = (*Str == TEXT('{')) ? TEXT('{') : TEXT('[');
		TCHAR Decrementor = (*Str == TEXT('{')) ? TEXT('}') : TEXT(']');
		bool bInQuote = false;
		int Layer = 1;
		if (bIncludeDelimiters)
		{
			Arg += *Str;
		}
		Str++; 

		while (1)
		{
			TCHAR Character = *Str;
			if (Character == 0)
			{
				break;
			}
			Str++;

			// Preserve escapes if they're in a quoted string (the check for " is in the else to let \" work as expected)
			if (Character == TEXT('\\') && UseEscape && bInQuote)
			{
				Arg += Character;

				Character = *Str;
				if (!Character)
				{
					break;
				}
				Str++;
			}
			else if (Character == TEXT('"'))
			{
				bInQuote = !bInQuote;
			}
			else if (!bInQuote)
			{
				if (Character == Decrementor)
				{
					Layer--;
				}
				else if (Character == Incrementor)
				{
					Layer++;
				}
				if (Layer == 0)
				{
					if (bIncludeDelimiters)
					{
						Arg += Character;
					}
					break;
				}
			}
			Arg += Character;
		}
	}
	else
	// Edits End
	if (*Str == TEXT('"'))
	{
		// Get quoted string.
		if (bIncludeDelimiters)
		{
			Arg += *Str;
		}
		Str++;
		while (*Str && *Str != TCHAR('"'))
		{
			TCHAR c = *Str++;
			if (c == TEXT('\\') && UseEscape)
			{
				// Get escape.
				c = *Str++;
				if (!c)
				{
					break;
				}
			}

			Arg += c;
		}

		if (*Str == TEXT('"'))
		{
			if (bIncludeDelimiters)
			{
				Arg += *Str;
			}
			Str++;
		}
	}
	else
	{
		// Get unquoted string (that might contain a quoted part, which will be left intact).
		// For example, -ARG="foo bar baz", will be treated as one token, with quotes intact
		bool bInQuote = false;

		while (1)
		{
			TCHAR Character = *Str;
			if ((Character == 0) || (FChar::IsWhitespace(Character) && !bInQuote))
			{
				break;
			}
			Str++;

			// Preserve escapes if they're in a quoted string (the check for " is in the else to let \" work as expected)
			if (Character == TEXT('\\') && UseEscape && bInQuote)
			{
				Arg += Character;

				Character = *Str;
				if (!Character)
				{
					break;
				}
				Str++;
			}
			else if (Character == TEXT('"'))
			{
				bInQuote = !bInQuote;
			}
			// EDITS begin- if we're doing comma-separated-values, break if we see a comma
			if (bCommaSeparated)
			{
				if (!bInQuote && Character == TEXT(','))
				{
					break;
				}
			}
			// EDITS end

			Arg += Character;
		}
	}

	// bHasEnteredObject allows us to declare empty arrays/objects like [] that are prefectly valid even though they are empty...
	return Arg.Len() > 0 || bHasEnteredObject;
}

template <typename T>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TArray<T>& Array, IOnlineServices* Services)
{
	FString Token;
	if (ParseTokenExt(Cmd, Token, true))
	{
		FString NextToken;
		const TCHAR* TokenTCHAR = *Token;
		while (ParseTokenExt(TokenTCHAR, NextToken, true, true, true))
		{
			T Value;
			FString Trimmed = NextToken.TrimStartAndEnd();
			const TCHAR* ArrayTokenTCHAR = *Trimmed;
			if (ParseOnlineExecParams(ArrayTokenTCHAR, Value, Services))
			{
				Array.Emplace(MoveTempIfPossible(Value));
			}
		}
	}
	else
	{
		return false;
	}
	return true;

}

template <typename T, typename U>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TMap<T, U>& Map, IOnlineServices* Services)
{
	FString Token;
	if (ParseTokenExt(Cmd, Token, true))
	{
		TArray<FString> TokenArray;
		Token.ParseIntoArray(TokenArray, TEXT(","));
		Map.Reserve(TokenArray.Num());
		for (const FString& ArrayToken : TokenArray)
		{
			TArray<FString> TokenValuePair;
			ArrayToken.ParseIntoArray(TokenValuePair, TEXT("="));

			if (TokenValuePair.Num() == 2)
			{
				T Key;
				U Value;

				FString TrimmedKey = TokenValuePair[0].TrimStartAndEnd();
				const TCHAR* ArrayTokenKeyTCHAR = *TrimmedKey;
				if (ParseOnlineExecParams(ArrayTokenKeyTCHAR, Key, Services))
				{
					FString TrimmedValue = TokenValuePair[1].TrimStartAndEnd();
					const TCHAR* ArrayTokenValueTCHAR = *TrimmedValue;
					if (ParseOnlineExecParams(ArrayTokenValueTCHAR, Value, Services))
					{
						Map.Emplace(MoveTempIfPossible(Key), MoveTempIfPossible(Value));
					}
					else
					{
						UE_LOGF(LogOnlineServices, Warning, "Failed to parse exec param value for map: %ls", *TrimmedValue);
						return false;
					}
				}
				else
				{
					UE_LOGF(LogOnlineServices, Warning, "Failed to parse exec param key for map: %ls", *TrimmedKey);
					return false;
				}
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "Received map key/value with invalid number of [=]: %ls", *ArrayToken);
				return false;
			}
		}
	}
	else
	{
		return false;
	}
	return true;
}

template <typename T>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSet<T>& Set, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		TArray<FString> TokenArray;
		Token.ParseIntoArray(TokenArray, TEXT(","));
		Set.Reserve(TokenArray.Num());
		for (const FString& ArrayToken : TokenArray)
		{
			T Key;

			const TCHAR* ArrayTokenKeyTCHAR = *ArrayToken;
			if (ParseOnlineExecParams(ArrayTokenKeyTCHAR, Key))
			{
				Set.Emplace(MoveTemp(Key));
			}
		}
	}
	else
	{
		return false;
	}
	return true;
}

template <typename T>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSharedPtr<T>& Ptr, IOnlineServices* Services)
{
	if (Ptr.IsValid())
	{
		return ParseOnlineExecParams(Cmd, *Ptr, Services);
	}
	else 
	{
		T* RawPtr = new T();
		bool bSuccess = ParseOnlineExecParams(Cmd, *RawPtr, Services);
		if (bSuccess)
		{
			Ptr = MakeShareable<T>(RawPtr);
		}
		else
		{
			delete RawPtr;
		}
		return bSuccess;
	}
}

template <typename T>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TSharedRef<T>& Ref, IOnlineServices* Services)
{
	return ParseOnlineExecParams(Cmd, *Ref, Services);
}

template <typename T>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TOptional<T>& Optional, IOnlineServices* Services)
{
	FString PrecedingToken;
	const TCHAR* OldCmd = Cmd;
	FParse::Token(Cmd, PrecedingToken, true);

	if (PrecedingToken.Equals(TEXT("null"))) 
	{
		Optional.Reset();
		return true;
	}

	// go back to before the FParse call because we need that value again (no way to "peek" a token sadly)
	Cmd = OldCmd;

	T Temp;
	bool bSuccess = ParseOnlineExecParams(Cmd, Temp, Services);
	if (bSuccess)
	{
		Optional = MoveTemp(Temp);
	}
	
	return bSuccess;
}

template<typename VariantType, typename LastType>
static bool VisitVariantTypes(VariantType& Variant, FString& VariantPrefix, FString& VariantValue, IOnlineServices* Services)
{
	for(const TCHAR* Prefix : TOnlineVariantVisitInfo<LastType, VariantType>::Prefixes)
	{
		if (!VariantPrefix.IsEmpty() && FCString::Stricmp(Prefix, *VariantPrefix) == 0)
		{
			return TOnlineVariantVisitInfo<LastType, VariantType>::Assign(VariantValue, Variant, Services);
		}
	}
	return false;
}

template<typename VariantType, typename FirstType, typename SecondType, typename... RemainingTypes>
inline bool VisitVariantTypes(VariantType& Variant, FString& VariantPrefix, FString& VariantValue, IOnlineServices* Services)
{
	for(const TCHAR* Prefix : TOnlineVariantVisitInfo<FirstType, VariantType>::Prefixes)
	{
		if (!VariantPrefix.IsEmpty() && FCString::Stricmp(Prefix, *VariantPrefix) == 0)
		{
			return TOnlineVariantVisitInfo<FirstType, VariantType>::Assign(VariantValue, Variant, Services);
		}
	}
	return VisitVariantTypes<VariantType, SecondType, RemainingTypes...>(Variant, VariantPrefix, VariantValue, Services);
}


template <typename... Ts>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TVariant<Ts...>& Variant, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		FString VariantType;
		FString VariantValue;

		if (Token.Split(TEXT(":"), &VariantType, &VariantValue, ESearchCase::CaseSensitive))
		{
			return VisitVariantTypes<TVariant<Ts...>, Ts...>(Variant, VariantType, VariantValue, Services);
		}
	}
	return false;
}

template<typename IdType>
inline bool ParseOnlineExecParams(const TCHAR*& Cmd, TOnlineId<IdType>& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		FString ServicesString;
		FString HandleString;

		if (Token.Split(TEXT(":"), &ServicesString, &HandleString, ESearchCase::CaseSensitive))
		{
			EOnlineServices ServiceEnum;
			LexFromString(ServiceEnum, *ServicesString);
			uint32 Handle = static_cast<uint32>(FCString::Strtoui64(*HandleString, nullptr, 10));
			Value = TOnlineId<IdType>(ServiceEnum, Handle);
			return true;
		}
	}

	return false;
}

template<>
inline bool ParseOnlineExecParams<OnlineIdHandleTags::FAccount>(const TCHAR*& Cmd, TOnlineId<OnlineIdHandleTags::FAccount>& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		FString ServicesString;
		FString HandleString;

		if (Token.Split(TEXT(":"), &ServicesString, &HandleString, ESearchCase::CaseSensitive))
		{
			EOnlineServices ServicesEnum;
			LexFromString(ServicesEnum, *ServicesString);
			uint32 Handle = static_cast<uint32>(FCString::Strtoui64(*HandleString, nullptr, 10));
			Value = TOnlineId<OnlineIdHandleTags::FAccount>(ServicesEnum, Handle);
			return true;
		}
		else if (Token.Len() == 1 && **Token >= TEXT('0') && **Token <= TEXT('9'))
		{
			FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(FCString::Strtoi(*Token, nullptr, 10));
			
			// may need to move this to its own .cpp file to avoid circular dependencies

			if (Services)
			{
				IAuthPtr AuthPtr = Services->GetAuthInterface();
				if (AuthPtr.IsValid())
				{
					TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> AuthResult = AuthPtr->GetLocalOnlineUserByPlatformUserId({PlatformUserId});
					if (AuthResult.IsOk())
					{
						Value = AuthResult.GetOkValue().AccountInfo->AccountId;
						return true;
					}
					else
					{
						UE_LOGF(LogOnlineServices, Warning, "Failed to resolve user id %d - not logged in!", PlatformUserId.GetInternalId());
					}
				}
			}
		}
	}

	return false;
}

template <typename T>
std::enable_if_t<!TModels_V<Meta::COnlineMetadataAvailable, T>, bool> ParseOnlineExecParams(const TCHAR*& Cmd, T& Value, IOnlineServices* Services)
{
	FString Token;
	if (FParse::Token(Cmd, Token, true))
	{
		using ::LexFromString;
		LexFromString(Value, *Token);
	}
	else
	{
		return false;
	}

	return true;
}

template <typename T>
std::enable_if_t<TModels_V<Meta::COnlineMetadataAvailable, T>, bool> ParseOnlineExecParams(const TCHAR*& Cmd, T& Value, IOnlineServices* Services)
{
	bool bSuccess = true;

	FString Token;
	if (ParseTokenExt(Cmd, Token, true))
	{
		const TCHAR* TokenTCHAR =  *Token;
		Meta::VisitFields(Value, [&TokenTCHAR, &bSuccess, Services](const TCHAR* Name, auto& Field)
		{
			bool bResult = ParseOnlineExecParams(TokenTCHAR, Field, Services);
			if (!bResult)
			{
				UE_LOGF(LogOnlineServices, Warning, "Failed to resolve inner-field %ls", Name);
			}
			bSuccess &= bResult;
		});
	}
	else
	{
		bSuccess = false;
	}

	return bSuccess;
}

/* Private */ }

template <typename InterfaceType, typename MemberFunctionPtrType>
class TOnlineInterfaceOperationExecHandler : public IOnlineExecHandler
{
public:
	using OpType = typename Private::TOnlineInterfaceOperationMemberFunctionPtrTraits<MemberFunctionPtrType>::OpType;

	TOnlineInterfaceOperationExecHandler(InterfaceType* InInterface, MemberFunctionPtrType InFunction)
		: Interface(InInterface)
		, Function(InFunction)
	{
	}

	virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		IOnlineServices* Services = &Interface->GetServices();

		typename OpType::Params Params;
		bool bSuccess = true;
		Meta::VisitFields(Params, [&Cmd, &bSuccess, &Ar, &Services](const TCHAR* Name, auto& Field)
		{
			bool bResult = Private::ParseOnlineExecParams(Cmd, Field, Services);
#if WITH_ENGINE
			if (!bResult)
			{
				UE_LOGF(LogConsoleResponse, Warning, "Failed to resolve outer field %ls", Name);
			}
#endif
			bSuccess &= bResult;
		});

		if (!bSuccess)
		{
			Ar.Log(TEXT("Failed to parse params"));
			return false;
		}
#if WITH_ENGINE
		if constexpr (Private::TOnlineInterfaceOperationMemberFunctionPtrTraits<MemberFunctionPtrType>::bAsync)
		{
			TOnlineAsyncOpHandle<OpType> AsyncOpHandle = Invoke(Function, Interface, MoveTemp(Params));

			AsyncOpHandle.OnComplete([&Ar](const TOnlineResult<OpType>& Result)
				{
					UE_LOGF(LogConsoleResponse, Display, "%ls result: %ls", OpType::Name, *ToLogString(Result));
				});
		}
		else
		{
			TOnlineResult<OpType> Result = Invoke(Function, Interface, MoveTemp(Params));
			UE_LOGF(LogConsoleResponse, Display, "%ls result: %ls", OpType::Name, *ToLogString(Result));
		}
#endif

		return true;
	}

	virtual bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		FString HelpString = OpType::Name;
		Meta::VisitFields<typename OpType::Params>([&HelpString](const auto& Field)
			{
				HelpString += TEXT(" ");
				HelpString += Field.Name;
			});

		Ar.Log(HelpString);
		
		return true;
	}

private:
	InterfaceType* Interface;
	MemberFunctionPtrType Function;
};

/*** Creates an exec handler for an object that provides Exec and Help */
template <typename T>
class TOnlineComponentExecHandler : public IOnlineExecHandler
{
public:
	TOnlineComponentExecHandler(T* InComponent)
		: Component(InComponent)
	{
	}

	virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return Component->Exec(World, Cmd, Ar);
	}

	virtual bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return Component->Help(World, Cmd, Ar);
	}

private:
	T* Component;
};

/** Creates an exec handler for anything that can be stored in a TFunction */
class FOnlineFunctionExecHandler : public IOnlineExecHandler
{
public:
	using FunctionType = TFunction<bool(UWorld*,const TCHAR*,FOutputDevice&)>;

	FOnlineFunctionExecHandler(FunctionType&& InFunction, FString&& InHelpString)
		: Function(MoveTemp(InFunction))
		, HelpString(MoveTemp(InHelpString))
	{
	}

	virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (Function)
		{
			return Function(World, Cmd, Ar);
		}
		else
		{
			return false;
		}
	}

	virtual bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		Ar.Log(HelpString);
		return true;
	}

private:
	FunctionType Function;
	FString HelpString;
};

/** Creates an exec handler that calls a class member function */
template <typename T>
class TOnlineMemberFunctionExecHandler : public IOnlineExecHandler
{
public:
	using MemberFunctionPtrType = bool(T::*)(UWorld*,const TCHAR*,FOutputDevice&);

	TOnlineMemberFunctionExecHandler(T* InObject, MemberFunctionPtrType InFunction, FString&& InHelpString)
		: Object(InObject)
		, Function(InFunction)
		, HelpString(MoveTemp(InHelpString))
	{
	}

	virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return Invoke(Function, Object, World, Cmd, Ar);
	}

	virtual bool Help(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		Ar.Log(HelpString);
		return true;
	}

private:
	T* Object;
	MemberFunctionPtrType Function;
	FString HelpString;
};

/** Creates an exec handler for anything that can be stored in a TFunction */
inline TUniquePtr<IOnlineExecHandler> MakeExecHandler(TFunction<bool(UWorld*, const TCHAR*, FOutputDevice&)>&& Function, FString&& HelpString = FString())
{
	return MakeUnique<FOnlineFunctionExecHandler>(MoveTempIfPossible(Function), MoveTemp(HelpString));
}

/** Creates an exec handler that calls a class member function */
template <typename T>
TUniquePtr<IOnlineExecHandler> MakeExecHandler(T* Object, bool(T::*Function)(UWorld*, const TCHAR*, FOutputDevice&), FString&& HelpString = FString())
{
	return MakeUnique<TOnlineMemberFunctionExecHandler<T>>(Object, Function, MoveTemp(HelpString));
}

/* UE::Online */ }
