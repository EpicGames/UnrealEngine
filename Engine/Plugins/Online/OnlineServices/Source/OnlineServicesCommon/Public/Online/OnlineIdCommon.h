// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Online/CoreOnline.h"

namespace UE::Online {

enum class EOnlineServices : uint8;

/**
 * A net id registry suitable for use with trivial immutable keys
 */
template<typename IdType, typename IdValueType>
class TOnlineBasicIdRegistry
{
public:
	TOnlineBasicIdRegistry(EOnlineServices InOnlineServicesType)
		: OnlineServicesType(InOnlineServicesType)
	{

	}

	virtual ~TOnlineBasicIdRegistry() = default;

	using HandleType = TOnlineId<IdType>;

	HandleType GetInvalidHandle() const
	{
		return HandleType(OnlineServicesType, 0);
	}

	HandleType FindHandle(const IdValueType& IdValue) const
	{
		UE::TReadScopeLock ReadLock(Lock);
		if (const HandleType* FoundHandle = IdValueToHandleMap.Find(IdValue))
		{
			return *FoundHandle;
		}
		return GetInvalidHandle();
	}

	HandleType FindOrAddHandle(const IdValueType& IdValue)
	{
		HandleType Handle = FindHandle(IdValue);
		if (!Handle.IsValid())
		{
			// Take a write lock, check again if we already have a handle, or insert a new element.
			UE::TWriteScopeLock WriteLock(Lock);
			if (const HandleType* FoundHandle = IdValueToHandleMap.Find(IdValue))
			{
				Handle = *FoundHandle;
			}

			if(!Handle.IsValid())
			{
				IdValues.Emplace(IdValue);
				Handle = HandleType(OnlineServicesType, IdValues.Num());
				IdValueToHandleMap.Emplace(IdValue, Handle);
			}
		}
		return Handle;
	}

	// Returns a copy as it's not thread safe to return a pointer/ref to an element of an array that can be relocated by another thread.
	IdValueType FindIdValue(const HandleType& Handle) const
	{
		if (ValidateOnlineId(Handle))
		{
			UE::TReadScopeLock ReadLock(Lock);
			if (IdValues.IsValidIndex(Handle.GetHandle() - 1))
			{
				return IdValues[Handle.GetHandle() -1];
			}
		}
		return IdValueType();
	}

	inline bool ValidateOnlineId(const HandleType& Handle) const
	{
		return ensure(Handle.GetOnlineServicesType() == OnlineServicesType) && Handle.IsValid();
	}

private:
	EOnlineServices OnlineServicesType;

	mutable FTransactionallySafeRWLock Lock;

	TArray<IdValueType> IdValues;
	TMap<IdValueType, HandleType> IdValueToHandleMap;
};

template<typename IdValueType>
using TOnlineBasicAccountIdRegistry = TOnlineBasicIdRegistry<OnlineIdHandleTags::FAccount, IdValueType>;

template<typename IdValueType>
using TOnlineBasicSessionIdRegistry = TOnlineBasicIdRegistry<OnlineIdHandleTags::FSession, IdValueType>;

template<typename IdValueType>
using TOnlineBasicSessionInviteIdRegistry = TOnlineBasicIdRegistry<OnlineIdHandleTags::FSessionInvite, IdValueType>;

/* UE::Online */ }

