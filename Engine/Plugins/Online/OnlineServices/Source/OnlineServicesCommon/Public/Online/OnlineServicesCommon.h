// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/NoneOf.h"
#include "Containers/Ticker.h"
#include "Online/OnlineAsyncOpCache.h"
#include "Online/OnlineAsyncOpQueue.h"
#include "Online/OnlineComponentRegistry.h"
#include "Online/OnlineConfig.h"
#include "Online/OnlineServices.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online { class IOnlineExecHandler; }

namespace UE::Online {

class FOnlineServicesCommon
	: public IOnlineServices
	, public FTSTickerObjectBase
	, public FSelfRegisteringExec
{
public:
	using Super = IOnlineServices;
	
	UE_API FOnlineServicesCommon(const FString& InServiceConfigName, FName InInstanceName, FName InInstanceConfigName);
	FOnlineServicesCommon(const FOnlineServicesCommon&) = delete;
	FOnlineServicesCommon(FOnlineServicesCommon&&) = delete;
	UE_API virtual ~FOnlineServicesCommon();

	// IOnlineServices
	UE_API virtual void Init() override;
	UE_API virtual void Destroy() override;
	UE_API virtual IAchievementsPtr GetAchievementsInterface() override;
	UE_API virtual IAuthPtr GetAuthInterface() override;
	UE_API virtual IUserInfoPtr GetUserInfoInterface() override;
	UE_API virtual ICommercePtr GetCommerceInterface() override;
	UE_API virtual ISocialPtr GetSocialInterface() override;
	UE_API virtual IPresencePtr GetPresenceInterface() override;
	UE_API virtual IExternalUIPtr GetExternalUIInterface() override;
	UE_API virtual ILeaderboardsPtr GetLeaderboardsInterface() override;
	UE_API virtual ILobbiesPtr GetLobbiesInterface() override;
	UE_API virtual ISessionsPtr GetSessionsInterface() override;
	UE_API virtual IStatsPtr GetStatsInterface() override;
	UE_API virtual IConnectivityPtr GetConnectivityInterface() override;
	UE_API virtual IPrivilegesPtr GetPrivilegesInterface() override;
	UE_API virtual ITitleFilePtr GetTitleFileInterface() override;
	UE_API virtual IUserFilePtr GetUserFileInterface() override;
	UE_API virtual TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(FGetResolvedConnectString::Params&& Params) override;
	UE_API virtual FName GetInstanceName() const override;
	UE_API virtual FName GetInstanceConfigName() const override;
	UE_API virtual void AssignBaseInterfaceSharedPtr(const FOnlineTypeName& TypeName, void* OutBaseInterfaceSP) override final;

	// FOnlineServicesCommon

	/**
	 * Retrieve any of the Interface IOnlineComponents
	 */
	template <typename ComponentType>
	ComponentType* Get()
	{
		return Components.Get<ComponentType>();
	}

	/**
	 * Called to register all the IOnlineComponents with the IOnlineService, called after this is constructed
	 */
	UE_API virtual void RegisterComponents();

	/**
	 * Calls Initialize on all the components, called after RegisterComponents
	 */
	UE_API virtual void Initialize();

	/**
	 * Calls PostInitialize on all the components, called after Initialize
	 */
	UE_API virtual void PostInitialize();

	/**
	 * Calls UpdateConfig on all the components
	 */
	UE_API virtual void UpdateConfig();

	/**
	 * Calls Tick on all the components
	 */
	UE_API virtual bool Tick(float DeltaSeconds) override;

	/**
	 * Calls PreShutdown on all the components, called prior to Shutdown
	 */
	UE_API virtual void PreShutdown();

	/**
	 * Calls Shutdown on all the components, called before this is destructed
	 */
	UE_API virtual void Shutdown();

	/**
	 * Call a callable according to a specified execution policy
	 */
	template <typename CallableType>
	void Execute(FOnlineAsyncExecutionPolicy ExecutionPolicy, CallableType&& Callable)
	{
		switch (ExecutionPolicy.GetExecutionPolicy())
		{
			case EOnlineAsyncExecutionPolicy::RunOnGameThread:
				ExecuteOnGameThread(MoveTemp(Callable));
				break;

			case EOnlineAsyncExecutionPolicy::RunOnNextTick:
				Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(Callable));
				break;

			case EOnlineAsyncExecutionPolicy::RunOnThreadPool:
				Async(EAsyncExecution::ThreadPool, MoveTemp(Callable));
				break;

			case EOnlineAsyncExecutionPolicy::RunOnTaskGraph:
				Async(EAsyncExecution::TaskGraph, MoveTemp(Callable));
				break;

			case EOnlineAsyncExecutionPolicy::RunImmediately:
				Callable();
				break;
		}
	}

	/**
	 * Call a callable on the game thread
	 */
	template <typename CallableType>
	void ExecuteOnGameThread(CallableType&& Callable)
	{
		if (IsInGameThread())
		{
			Callable();
		}
		else
		{
			Async(EAsyncExecution::TaskGraphMainThread, MoveTemp(Callable));
		}
	}

	/**
	 * Override the default config provider (FOnlineConfigProviderGConfig(GEngineini))
	 */
	void SetConfigProvider(TUniquePtr<IOnlineConfigProvider>&& InConfigProvider)
	{
		ConfigProvider = MoveTemp(InConfigProvider);
	}

	/**
	 * Clear the list of config overrides
	 */
	void ResetConfigSectionOverrides()
	{
		ConfigSectionOverrides.Reset();
	}

	/**
	 * Add a config section override. These will be used in the order they are added
	 */
	void AddConfigSectionOverride(const FString& Override)
	{
		ConfigSectionOverrides.Add(Override);
	}

	/**
	 * Get the ini config name for the Subsystem
	 */
	const FString& GetServiceConfigName() const { return ServiceConfigName; }

	UE_DEPRECATED(5.6, "GetConfigSectionHeiarchy has been renamed GetConfigSectionHierarchy")
	TArray<FString> GetConfigSectionHeiarchy(const FString& OperationName = FString()) const
	{
		return GetConfigSectionHierarchy(OperationName);
	}

	UE_DEPRECATED(5.8, "GetConfigSectionHeirarchy has been renamed GetConfigSectionHierarchy")
	TArray<FString> GetConfigSectionHeirarchy(const FString& OperationName = FString()) const
	{
		return GetConfigSectionHierarchy(OperationName);
	}
	
	TArray<FString> GetConfigSectionHierarchy(const FString& OperationName = FString()) const
	{
		TArray<FString> SectionHierarchy;
		FString SectionName = TEXT("OnlineServices");
		SectionHierarchy.Add(SectionName);
		SectionName += TEXT(".") + GetServiceConfigName();
		SectionHierarchy.Add(SectionName);
		if (!OperationName.IsEmpty())
		{
			SectionName += TEXT(".") + OperationName;
			SectionHierarchy.Add(SectionName);
		}
		return SectionHierarchy;
	}

	/**
	 * Load a config struct for an interface + operation
	 * Will load values from the following sections, skipping all with any empty strings:
	 *   OnlineServices
	 *   OnlineServices.<InterfaceName>
	 *   OnlineServices.<InterfaceName>.<OperationName>
	 *   OnlineServices.<ServiceProvider>
	 *   OnlineServices.<ServiceProvider>.<InterfaceName>
	 *   OnlineServices.<ServiceProvider>.<InterfaceName>.<OperationName>
	 *   OnlineServices.<ServiceProvider>.<InstanceConfigName>
	 *   OnlineServices.<ServiceProvider>.<InstanceConfigName>.<InterfaceName>
	 *   OnlineServices.<ServiceProvider>.<InstanceConfigName>.<InterfaceName>.<OperationName>
	 * 
	 * @param Struct Struct to populate with values from config
	 * @param InterfaceName Optional interface name to append to the config section name
	 * @param OperationName Optional operation name to append to the config section name
	 * 
	 * @return true if a value was loaded
	 */
	template <typename StructType>
	bool LoadConfig(StructType& Struct, const FString& InterfaceName = FString(), const FString& OperationName = FString()) const
	{
		TArray<FString> SectionHierarchy;

		auto PushSection = [&SectionHierarchy](const TArray<const FStringView>& SectionNames)
		{
			if (Algo::NoneOf(SectionNames, UE_PROJECTION_MEMBER(FStringView, IsEmpty)))
			{
				SectionHierarchy.Emplace(FString::Join(SectionNames, TEXT(".")));
			}
		};

		const FString& BaseSection = TEXT("OnlineServices");
		const FString& InstanceConfigNameStr = InstanceConfigName.ToString();

		PushSection({ BaseSection });
		PushSection({ BaseSection, InterfaceName });
		PushSection({ BaseSection, InterfaceName, OperationName });
		PushSection({ BaseSection, ServiceConfigName });
		PushSection({ BaseSection, ServiceConfigName, InterfaceName });
		PushSection({ BaseSection, ServiceConfigName, InterfaceName, OperationName });
		PushSection({ BaseSection, ServiceConfigName, InstanceConfigNameStr });
		PushSection({ BaseSection, ServiceConfigName, InstanceConfigNameStr, InterfaceName });
		PushSection({ BaseSection, ServiceConfigName, InstanceConfigNameStr, InterfaceName, OperationName });

		return LoadConfig(Struct, SectionHierarchy);
	}
	
	UE_DEPRECATED(5.6, "GetConfigSectionHeirachWithOverrides has been renamed GetConfigSectionHierarchyWithOverrides")
	TArray<FString> GetConfigSectionHeirachWithOverrides(const TArray<FString>& SectionHierarchy) const
	{
		return GetConfigSectionHierarchyWithOverrides(SectionHierarchy);
	}

	UE_DEPRECATED(5.8, "GetConfigSectionHeirarchyWithOverrides has been renamed GetConfigSectionHierarchyWithOverrides")
	TArray<FString> GetConfigSectionHeirarchyWithOverrides(const TArray<FString>& SectionHierarchy) const
	{
		return GetConfigSectionHierarchyWithOverrides(SectionHierarchy);
	}

	/**
	 * Get an array of a config section with the overrides added in
	 * 
	 * @param SectionHierarchy Array of config sections to load values from
	 * 
	 * @return Array of the sections with overrides for values to be loaded from
	 */
	TArray<FString> GetConfigSectionHierarchyWithOverrides(const TArray<FString>& SectionHierarchy) const
	{
		TArray<FString> AllConfigSectionOverrides = ConfigSectionOverrides;

		if (GIsEditor)
		{
			AllConfigSectionOverrides.Add("Editor");
		}

		TArray<FString> SectionHierarchyWithOverrides;
		for (const FString& Section : SectionHierarchy)
		{
			SectionHierarchyWithOverrides.Add(Section);

			for (const FString& Override : AllConfigSectionOverrides)
			{
				// Using of space in online config section is deprecated
				FString OverrideSection = Section + TEXT(" ") + Override;
				SectionHierarchyWithOverrides.Add(OverrideSection);

				OverrideSection = Section + TEXT(":") + Override;
				SectionHierarchyWithOverrides.Add(OverrideSection);
			}
		}

		return SectionHierarchyWithOverrides;
	}

	/**
	 * Load a config struct for a section hierarchy, also using the ConfigSectionOverrides
	 *
	 * @param Struct Struct to populate with values from config
	 * @param SectionHierarchy Array of config sections to load values from
	 *
	 * @return true if a value was loaded
	 */
	template <typename StructType>
	bool LoadConfig(StructType& Struct, const TArray<FString>& SectionHierarchy) const
	{
		return ::UE::Online::LoadConfig(*ConfigProvider, GetConfigSectionHierarchyWithOverrides(SectionHierarchy), Struct);
	}

	/* Get op (OnlineServices) */
	template <typename OpType>
	TOnlineAsyncOpRef<OpType> GetOp(typename OpType::Params&& Params)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetOp<OpType>(MoveTemp(Params), GetConfigSectionHierarchy());
	}

	template <typename OpType, typename ParamsFuncsType = TJoinableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetJoinableOp(typename OpType::Params&& Params)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetJoinableOp<OpType, ParamsFuncsType>(MoveTemp(Params), GetConfigSectionHierarchy());
	}

	template <typename OpType, typename ParamsFuncsType = TMergeableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetMergeableOp(typename OpType::Params&& Params)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetMergeableOp<OpType, ParamsFuncsType>(MoveTemp(Params), GetConfigSectionHierarchy());
	}

	/* Get op (Interface) */
	template <typename OpType>
	TOnlineAsyncOpRef<OpType> GetOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetOp<OpType>(MoveTemp(Params), ConfigSectionHierarchy);
	}

	template <typename OpType, typename ParamsFuncsType = TJoinableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetJoinableOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetJoinableOp<OpType, ParamsFuncsType>(MoveTemp(Params), ConfigSectionHierarchy);
	}

	template <typename OpType, typename ParamsFuncsType = TMergeableOpParamsFuncs<OpType>>
	TOnlineAsyncOpRef<OpType> GetMergeableOp(typename OpType::Params&& Params, const TArray<FString>& ConfigSectionHierarchy)
	{
		LogErrorIfOnlineServicesHasShutdown<OpType>();
		return OpCache.GetMergeableOp<OpType, ParamsFuncsType>(MoveTemp(Params), ConfigSectionHierarchy);
	}

	/* Queue for executing tasks in parallel. Serial queues feed into this */
	UE_API FOnlineAsyncOpQueueParallel& GetParallelQueue();

	/* Queue for executing tasks in serial */
	UE_API FOnlineAsyncOpQueue& GetSerialQueue();

	/* Queues for executing per-user tasks in serial */
	UE_API FOnlineAsyncOpQueue& GetSerialQueue(const FAccountId& AccountId);
	
	UE_API void RegisterExecHandler(const FString& Name, TUniquePtr<IOnlineExecHandler>&& Handler);

#if UE_ALLOW_EXEC_COMMANDS
	UE_API virtual bool Exec(UWorld* World, const TCHAR* Cmd, FOutputDevice& Ar) override;
#endif

	FOnlineAsyncOpCache OpCache;

protected:
	enum class EAsyncOpFlushReason : uint8
	{
		Default,
		Shutdown
	};

	template <typename OpType>
	void LogErrorIfOnlineServicesHasShutdown()
	{
		UE_CLOGF(bPreShutdownComplete, LogOnlineServices, Error, "Can't add op %ls anymore after shutdown in %p %ls online services!", OpType::Name, this, LexToString(GetServicesProvider()));
	}

	UE_API void Flush(EAsyncOpFlushReason FlushReason);
	UE_API virtual void FlushTick(float DeltaSeconds);
	UE_API void LoadCommonConfig();

	TMap<FString, TUniquePtr<IOnlineExecHandler>> ExecCommands;

	static UE_API uint32 NextInstanceIndex;
	uint32 InstanceIndex;
	FName InstanceName;
	FName InstanceConfigName;

	FOnlineComponentRegistry Components;
	TUniquePtr<IOnlineConfigProvider> ConfigProvider;

	/* Config section overrides */
	TArray<FString> ConfigSectionOverrides;
	FString ServiceConfigName;

	FOnlineAsyncOpQueueParallel ParallelQueue;
	FOnlineAsyncOpQueueSerial SerialQueue;
	TMap<FAccountId, TUniquePtr<FOnlineAsyncOpQueueSerial>> PerUserSerialQueue;
	bool bPreShutdownComplete;
};

/* UE::Online */ }

#undef UE_API
