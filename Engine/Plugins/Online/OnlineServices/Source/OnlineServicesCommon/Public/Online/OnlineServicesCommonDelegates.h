// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Online/OnlineError.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

// Notification params of OnOnlineAsyncOpCompletedV2 delegate
struct FOnOnlineAsyncOpCompletedParams
{
	/**
	 * FOnOnlineAsyncOpCompletedParams ctor
	 *
	 * @param OnlineServicesCommon the OnlineServices instance
	 * @param OnlineError the result of completed operation
	 */
	UE_API FOnOnlineAsyncOpCompletedParams(FOnlineServicesCommon& InOnlineServicesCommon, const TOptional<FOnlineError>& InOnlineError);

	// The name of completed operation
	FString OpName;
	// The name of interface
	FString InterfaceName;
	// The OnlineServices instance
	TWeakPtr<FOnlineServicesCommon> OnlineServicesCommon;
	// The result of completed operation
	TOptional<UE::Online::FOnlineError> OnlineError;
	// The duration of the operation from start to complete
	double DurationInSeconds = -1.0;
};

/**
 * Notification that an online operation has completed
 *
 * !!!NOTE!!! The notification can happen on off-game threads, make sure the callbacks are thread-safe
 *
 * @param Params the struct FOnOnlineAsyncOpCompletedParams which contains related info
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnOnlineAsyncOpCompletedV2, const FOnOnlineAsyncOpCompletedParams& Params);
UE_DEPRECATED(5.7, "OnOnlineAsyncOpCompletedV2 has been deprecated, use OnAsyncOpCompleted instead")
extern ONLINESERVICESCOMMON_API FOnOnlineAsyncOpCompletedV2 OnOnlineAsyncOpCompletedV2;

/* UE::Online */ }

#undef UE_API
