// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineAsyncOp.h"

namespace UE::Online {

template <typename DataType, typename OpType>
const DataType& GetOpDataChecked(const TOnlineAsyncOp<OpType>& Op, const FString& Key)
{
	const DataType* Data = Op.Data.template Get<DataType>(Key);
	check(Data);
	return *Data;
}

/* UE::Online */ }
