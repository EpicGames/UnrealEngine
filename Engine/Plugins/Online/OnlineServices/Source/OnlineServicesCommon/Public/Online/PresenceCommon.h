// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Presence.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FPresenceCommon : public TOnlineComponent<IPresence>
{
public:
	using Super = IPresence;

	UE_API FPresenceCommon(FOnlineServicesCommon& InServices);

	// IOnlineComponent
	UE_API virtual void RegisterCommands() override;
	UE_API virtual void UpdateConfig() override;

	// IPresence
	UE_API virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FBatchQueryPresence> BatchQueryPresence(FBatchQueryPresence::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) override;
	UE_API virtual TOnlineEvent<void(const FPresenceUpdated&)> OnPresenceUpdated() override;

protected:
	FOnlineServicesCommon& Services;

	TOnlineEventCallable<void(const FPresenceUpdated&)> OnPresenceUpdatedEvent;
};

/* UE::Online */ }

#undef UE_API
