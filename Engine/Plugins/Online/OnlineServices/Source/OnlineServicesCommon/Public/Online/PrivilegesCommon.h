// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Privileges.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FPrivilegesCommon : public TOnlineComponent<IPrivileges>
{
public:
	using Super = IPrivileges;

	UE_API FPrivilegesCommon(FOnlineServicesCommon& InServices);

	// Begin TOnlineComponent
	UE_API virtual void RegisterCommands() override;
	// End TOnlineComponent

	// Begin IPrivileges
	UE_API virtual TOnlineAsyncOpHandle<FQueryUserPrivilege> QueryUserPrivilege(FQueryUserPrivilege::Params&& Params) override;
	// End IPrivileges
};

/* UE::Online */}

#undef UE_API
