// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Sessions.h"
#include "Online/OnlineComponent.h"
#include "Online/OnlineIdCommon.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

static FName CONNECT_STRING_TAG = TEXT("CONNECT_STRING");

class FOnlineSessionIdStringRegistry : public IOnlineSessionIdRegistry
{
public:
	// Begin IOnlineSessionIdRegistry
	virtual inline FString ToString(const FOnlineSessionId& SessionId) const override
	{
		FString IdValue = BasicRegistry.FindIdValue(SessionId);

		if (IdValue.IsEmpty())
		{
			IdValue = FString(TEXT("Invalid"));
		}

		return IdValue;
	};

	virtual inline FString ToLogString(const FOnlineSessionId& SessionId) const override
	{
		return ToString(SessionId);
	};

	virtual inline TArray<uint8> ToReplicationData(const FOnlineSessionId& SessionId) const override
	{
		const FString IdValue = BasicRegistry.FindIdValue(SessionId);
		const FTCHARToUTF8 IdValueUtf8(*IdValue);

		TArray<uint8> ReplicationData;
		ReplicationData.SetNumUninitialized(IdValueUtf8.Length());

		FMemory::Memcpy(ReplicationData.GetData(), IdValueUtf8.Get(), IdValueUtf8.Length());

		return ReplicationData;
	}

	virtual inline FOnlineSessionId FromReplicationData(const TArray<uint8>& ReplicationData) override
	{
		const FUTF8ToTCHAR IdValueTCHAR((char*)ReplicationData.GetData(), ReplicationData.Num());
		const FString IdValue = FString::ConstructFromPtrSize(IdValueTCHAR.Get(), IdValueTCHAR.Length());
		return FromStringData(IdValue);
	}

	virtual inline FOnlineSessionId FromStringData(const FString& StringData) override
	{
		if (!StringData.IsEmpty())
		{
			return BasicRegistry.FindOrAddHandle(StringData);
		}

		return FOnlineSessionId();
	}

	// End IOnlineSessionIdRegistry

	inline bool IsSessionIdExpired(const FOnlineSessionId& InSessionId) const
	{
		return BasicRegistry.FindIdValue(InSessionId).IsEmpty();
	}

	FOnlineSessionIdStringRegistry(EOnlineServices OnlineServicesType)
		: BasicRegistry(OnlineServicesType)
	{

	}

	virtual ~FOnlineSessionIdStringRegistry() = default;

public:
	TOnlineBasicSessionIdRegistry<FString> BasicRegistry;
};

class FOnlineSessionInviteIdStringRegistry : public IOnlineSessionInviteIdRegistry
{
public:
	// Begin IOnlineSessionIdRegistry
	virtual inline FString ToString(const FSessionInviteId& SessionInviteId) const override
	{
		FString IdValue = BasicRegistry.FindIdValue(SessionInviteId);

		if (IdValue.IsEmpty())
		{
			IdValue = TEXT("Invalid");
		}

		return IdValue;
	}

	virtual inline FString ToLogString(const FSessionInviteId& SessionInviteId) const override
	{
		return ToString(SessionInviteId);
	};

	virtual inline TArray<uint8> ToReplicationData(const FSessionInviteId& SessionInviteId) const override
	{
		const FString IdValue = BasicRegistry.FindIdValue(SessionInviteId);
		const FTCHARToUTF8 IdValueUtf8(IdValue);

		TArray<uint8> ReplicationData;
		ReplicationData.SetNumUninitialized(IdValueUtf8.Length());

		FMemory::Memcpy(ReplicationData.GetData(), IdValueUtf8.Get(), IdValueUtf8.Length());

		return ReplicationData;
	}

	virtual inline FSessionInviteId FromReplicationData(const TArray<uint8>& ReplicationData) override
	{
		const FUTF8ToTCHAR IdValueTCHAR((char*)ReplicationData.GetData(), ReplicationData.Num());
		const FString IdValue = FString::ConstructFromPtrSize(IdValueTCHAR.Get(), IdValueTCHAR.Length());
		return FromStringData(IdValue);
	}

	virtual inline FSessionInviteId FromStringData(const FString& StringData) override
	{
		if (!StringData.IsEmpty())
		{
			return BasicRegistry.FindOrAddHandle(StringData);
		}

		return FSessionInviteId();
	}

	// End IOnlineSessionIdRegistry

	FOnlineSessionInviteIdStringRegistry(EOnlineServices OnlineServicesType)
		: BasicRegistry(OnlineServicesType)
	{

	}

	virtual ~FOnlineSessionInviteIdStringRegistry() = default;

public:
	TOnlineBasicSessionInviteIdRegistry<FString> BasicRegistry;
};

class FSessionCommon : public ISession
{
public:
	FSessionCommon() = default;
	FSessionCommon(const FSessionCommon& InSession) = default;
	virtual ~FSessionCommon() = default;

	// ISession
	virtual const FAccountId GetOwnerAccountId() const override				{ return OwnerAccountId; }
	virtual const FOnlineSessionId GetSessionId() const override			{ return GetSessionInfo().SessionId; }
	virtual const uint32 GetNumOpenConnections() const override				{ return SessionSettings.NumMaxConnections - SessionMembers.Num(); }
	virtual const FSessionInfo& GetSessionInfo() const override				{ return SessionInfo; }
	virtual const FSessionSettings& GetSessionSettings() const override		{ return SessionSettings; }
	virtual const FSessionMemberIdsSet& GetSessionMembers() const override	{ return SessionMembers; }

	virtual bool IsJoinable() const override								{ return GetNumOpenConnections() > 0 && SessionSettings.bAllowNewMembers; }

	UE_API virtual FString ToLogString() const override;

	UE_API virtual void DumpState() const override;

protected:
	UE_API void DumpMemberData() const;
	UE_API void DumpSessionInfo() const;
	UE_API void DumpSessionSettings() const;

public:
	/** Session information that will remain constant throughout the session's lifetime */
	FSessionInfo SessionInfo;
	
	/** The following members can be updated, and such update will be transmitted via a FSessionUpdated event */

	/** The user who currently owns the session */
	FAccountId OwnerAccountId;

	/** Set of session properties that can be altered by the session owner */
	FSessionSettings SessionSettings;

	/* Set containing user ids for all the session members */
	FSessionMemberIdsSet SessionMembers;

	UE_API FSessionCommon& operator+=(const FSessionUpdate& SessionUpdate);
};

class FSessionInviteCommon : public ISessionInvite
{
public:
	FSessionInviteCommon() = default;
	UE_API FSessionInviteCommon(const FAccountId& InRecipientId, const FAccountId& InSenderId, const FSessionInviteId& InInviteId, const FOnlineSessionId& InSessionId);

	virtual const FAccountId GetRecipientId() const override { return RecipientId; }
	virtual const FAccountId GetSenderId() const override { return SenderId; }
	virtual const FSessionInviteId GetInviteId() const override { return InviteId; }
	virtual const FOnlineSessionId GetSessionId() const override { return SessionId; }

	UE_API virtual FString ToLogString() const override;

public:
	/* The id handle for the user which the invite got sent to */
	FAccountId RecipientId;

	/* The id handle for the user which sent the invite */
	FAccountId SenderId;

	/* The invite id handle, needed for retrieving invite information and rejecting the invite */
	FSessionInviteId InviteId;

	/* The session id handle, needed for retrieving the session information */
	FOnlineSessionId SessionId;
};

struct FGetMutableSessionByName
{
	static constexpr TCHAR Name[] = TEXT("GetMutableSessionByName");

	struct Params
	{
		FName LocalName;
	};

	struct Result
	{
		TSharedRef<FSessionCommon> Session;
	};
};

struct FGetMutableSessionById
{
	static constexpr TCHAR Name[] = TEXT("GetMutableSessionById");

	struct Params
	{
		FOnlineSessionId SessionId;
	};

	struct Result
	{
		TSharedRef<FSessionCommon> Session;
	};
};

class FSessionsCommon : public TOnlineComponent<ISessions>
{
public:
	using Super = ISessions;

	UE_API FSessionsCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void Initialize() override;
	UE_API virtual void RegisterCommands() override;

	// ISessions
	UE_API virtual TOnlineResult<FGetAllSessions> GetAllSessions(FGetAllSessions::Params&& Params) const override;
	UE_API virtual TOnlineResult<FGetSessionByName> GetSessionByName(FGetSessionByName::Params&& Params) const override;
	UE_API virtual TOnlineResult<FGetSessionName> GetSessionName(FGetSessionName::Params&& Params) const override;
	UE_API virtual TOnlineResult<FGetSessionById> GetSessionById(FGetSessionById::Params&& Params) const override;
	UE_API virtual TOnlineResult<FGetPresenceSession> GetPresenceSession(FGetPresenceSession::Params&& Params) const override;
	UE_API virtual TOnlineResult<FIsPresenceSession> IsPresenceSession(FIsPresenceSession::Params&& Params) const override;
	UE_API virtual TOnlineResult<FSetPresenceSession> SetPresenceSession(FSetPresenceSession::Params&& Params) override;
	UE_API virtual TOnlineResult<FClearPresenceSession> ClearPresenceSession(FClearPresenceSession::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FCreateSession> CreateSession(FCreateSession::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUpdateSessionSettings> UpdateSessionSettings(FUpdateSessionSettings::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FLeaveSession> LeaveSession(FLeaveSession::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FFindSessions> FindSessions(FFindSessions::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FStartMatchmaking> StartMatchmaking(FStartMatchmaking::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FJoinSession> JoinSession(FJoinSession::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAddSessionMember> AddSessionMember(FAddSessionMember::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FRemoveSessionMember> RemoveSessionMember(FRemoveSessionMember::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FSendSessionInvite> SendSessionInvite(FSendSessionInvite::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetSessionInviteById> GetSessionInviteById(FGetSessionInviteById::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAllSessionInvites> GetAllSessionInvites(FGetAllSessionInvites::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FRejectSessionInvite> RejectSessionInvite(FRejectSessionInvite::Params&& Params) override;

	UE_API virtual TOnlineEvent<void(const FSessionCreated&)> OnSessionCreated() override;
	UE_API virtual TOnlineEvent<void(const FSessionJoined&)> OnSessionJoined() override;
	UE_API virtual TOnlineEvent<void(const FSessionLeft&)> OnSessionLeft() override;
	UE_API virtual TOnlineEvent<void(const FSessionUpdated&)> OnSessionUpdated() override;
	UE_API virtual TOnlineEvent<void(const FSessionInviteReceived&)> OnSessionInviteReceived() override;
	UE_API virtual TOnlineEvent<void(const FUISessionJoinRequested&)> OnUISessionJoinRequested() override;

protected:
	// Auxiliary methods
	UE_API TOnlineResult<FGetMutableSessionByName> GetMutableSessionByName(FGetMutableSessionByName::Params&& Params) const;
	UE_API TOnlineResult<FGetMutableSessionById> GetMutableSessionById(FGetMutableSessionById::Params&& Params) const;

	UE_API void AddSessionInvite(const TSharedRef<FSessionInviteCommon> SessionInvite, const TSharedRef<FSessionCommon> Session, const FAccountId& LocalAccountId);
	UE_API void AddSearchResult(const TSharedRef<FSessionCommon> Session, const FAccountId& LocalAccountId);
	UE_API void AddSessionWithReferences(const TSharedRef<FSessionCommon> Session, const FName& SessionName, const FAccountId& LocalAccountId, bool bIsPresenceSession);
	UE_API void AddSessionReferences(const FOnlineSessionId SessionId, const FName& SessionName, const FAccountId& LocalAccountId, bool bIsPresenceSession);

	UE_API void ClearSessionInvitesForSession(const FAccountId& LocalAccountId, const FOnlineSessionId SessionId);
	UE_API void ClearSessionReferences(const FOnlineSessionId SessionId, const FName& SessionName, const FAccountId& LocalAccountId);

	UE_API FSessionUpdate BuildSessionUpdate(const TSharedRef<FSessionCommon>& Session, const FSessionSettingsUpdate& UpdatedValues) const;

	// FSessionsCommon internal interface

	UE_API virtual TOnlineResult<FSetPresenceSession> SetPresenceSessionImpl(FSetPresenceSession::Params&& Params);
	UE_API virtual TOnlineResult<FClearPresenceSession> ClearPresenceSessionImpl(FClearPresenceSession::Params&& Params);

	UE_API virtual TFuture<TOnlineResult<FCreateSession>> CreateSessionImpl(const FCreateSession::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FCreateSession::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FCreateSession::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FUpdateSessionSettings>> UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FUpdateSessionSettings::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FUpdateSessionSettings::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FLeaveSession>> LeaveSessionImpl(const FLeaveSession::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FLeaveSession::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FLeaveSession::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FFindSessions>> FindSessionsImpl(const FFindSessions::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FFindSessions::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FFindSessions::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FStartMatchmaking>> StartMatchmakingImpl(const FStartMatchmaking::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FStartMatchmaking::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FStartMatchmaking::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FJoinSession>> JoinSessionImpl(const FJoinSession::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FJoinSession::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FJoinSession::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FAddSessionMember>> AddSessionMemberImpl(const FAddSessionMember::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FAddSessionMember::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FAddSessionMember::Params& Params) const;
	UE_API TOnlineResult<FAddSessionMember> AddSessionMemberInternal(const FAddSessionMember::Params& Params);

	UE_API virtual TFuture<TOnlineResult<FRemoveSessionMember>> RemoveSessionMemberImpl(const FRemoveSessionMember::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FRemoveSessionMember::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FRemoveSessionMember::Params& Params) const;
	UE_API TOnlineResult<FRemoveSessionMember> RemoveSessionMemberInternal(const FRemoveSessionMember::Params& Params);

	UE_API virtual TFuture<TOnlineResult<FSendSessionInvite>> SendSessionInviteImpl(const FSendSessionInvite::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FSendSessionInvite::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FSendSessionInvite::Params& Params) const;

	UE_API virtual TFuture<TOnlineResult<FRejectSessionInvite>> RejectSessionInviteImpl(const FRejectSessionInvite::Params& Params);
	UE_API virtual TOptional<FOnlineError> CheckParams(const FRejectSessionInvite::Params& Params) const;
	UE_API virtual TOptional<FOnlineError> CheckState(const FRejectSessionInvite::Params& Params) const;

private:
	template<typename MethodStruct>
	TOnlineAsyncOpHandle<MethodStruct> ExecuteAsyncSessionsMethod(typename MethodStruct::Params&& Params, TFuture<TOnlineResult<MethodStruct>>(FSessionsCommon::* ImplFunc)(const typename MethodStruct::Params& Params), bool bUseParallelQueue = false);

	void ClearSessionByName(const FName& SessionName);
	void ClearSessionById(const FOnlineSessionId& SessionId);

protected:

	struct FSessionEvents
	{
		TOnlineEventCallable<void(const FSessionCreated&)> OnSessionCreated;
		TOnlineEventCallable<void(const FSessionJoined&)> OnSessionJoined;
		TOnlineEventCallable<void(const FSessionLeft&)> OnSessionLeft;
		TOnlineEventCallable<void(const FSessionUpdated&)> OnSessionUpdated;
		TOnlineEventCallable<void(const FSessionInviteReceived&)> OnSessionInviteReceived;
		TOnlineEventCallable<void(const FUISessionJoinRequested&)> OnUISessionJoinRequested;
	} SessionEvents;

	/** Map of named sessions a user is part of, indexed by user */
	TMap<FAccountId, TArray<FName>> NamedSessionUserMap;

	/** Map of sessions that local users are part of, indexed by their local name */
	TMap<FName, FOnlineSessionId> LocalSessionsByName;

	/** Map of sessions that local users have set as their presence session to appear in the platform UI. A user may not have set any session as their presence session. */
	TMap<FAccountId, FOnlineSessionId> PresenceSessionsUserMap;

	/** Cache for received session invites, mapped per user */
	TMap<FAccountId, TMap<FSessionInviteId, TSharedRef<FSessionInviteCommon>>> SessionInvitesUserMap;

	/** Cache for the last set of session search results, mapped per user */
	TMap<FAccountId, TArray<FOnlineSessionId>> SearchResultsUserMap;

	/** Handle for an ongoing session search operation, mapped per user */
	TMap<FAccountId, TPromise<TOnlineResult<FFindSessions>>> CurrentSessionSearchPromisesUserMap;

	/** Set of every distinct FSession found, indexed by Id */
	TMap<FOnlineSessionId, TSharedRef<FSessionCommon>> AllSessionsById;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FGetMutableSessionByName::Params)
	ONLINE_STRUCT_FIELD(FGetMutableSessionByName::Params, LocalName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetMutableSessionByName::Result)
	ONLINE_STRUCT_FIELD(FGetMutableSessionByName::Result, Session)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetMutableSessionById::Params)
	ONLINE_STRUCT_FIELD(FGetMutableSessionById::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetMutableSessionById::Result)
	ONLINE_STRUCT_FIELD(FGetMutableSessionById::Result, Session)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef UE_API
