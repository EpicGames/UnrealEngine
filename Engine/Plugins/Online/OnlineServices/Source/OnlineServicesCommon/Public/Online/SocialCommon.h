// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Social.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FSocialCommon : public TOnlineComponent<ISocial>
{
public:
	using Super = ISocial;

	UE_API FSocialCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// ISocial
	UE_API virtual TOnlineAsyncOpHandle<FQueryFriends> QueryFriends(FQueryFriends::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetFriends> GetFriends(FGetFriends::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FSendFriendInvite> SendFriendInvite(FSendFriendInvite::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FAcceptFriendInvite> AcceptFriendInvite(FAcceptFriendInvite::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FRejectFriendInvite> RejectFriendInvite(FRejectFriendInvite::Params&& Params) override;
	UE_API virtual TOnlineEvent<void(const FRelationshipUpdated&)> OnRelationshipUpdated() override;
	UE_API virtual TOnlineAsyncOpHandle<FQueryBlockedUsers> QueryBlockedUsers(FQueryBlockedUsers::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetBlockedUsers> GetBlockedUsers(FGetBlockedUsers::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FBlockUser> BlockUser(FBlockUser::Params&& Params) override;

protected:
	UE_API void BroadcastRelationshipUpdated(FAccountId LocalAccountId, FAccountId RemoteAccountId, ERelationship OldRelationship, ERelationship NewRelationship);
	TOnlineEventCallable<void(const FRelationshipUpdated&)> OnRelationshipUpdatedEvent;
};

/* UE::Online */ }

#undef UE_API
