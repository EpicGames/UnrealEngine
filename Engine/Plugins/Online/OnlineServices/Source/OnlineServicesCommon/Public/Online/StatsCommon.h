// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Stats.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

enum class EStatModifyMethod : uint8
{
	/** Add the new value to the previous value */
	Sum,
	/** Overwrite previous value with the new value */
	Set,
	/** Only replace previous value if new value is larger */
	Largest,
	/** Only replace previous value if new value is smaller */
	Smallest
};

const TCHAR* LexToString(EStatModifyMethod Value);
void LexFromString(EStatModifyMethod& OutValue, const TCHAR* InStr);

struct FStatDefinition
{
	/* The name of the stat */
	FString Name;
	/* Corresponding stat id on the platform if needed */
	int32 Id = 0;
	/* How the stat will be modified */
	EStatModifyMethod ModifyMethod = EStatModifyMethod::Set;
	/* Store the default type and value */
	FStatValue DefaultValue = FStatValue(static_cast<int64>(0));
};

struct FStatsCommonConfig
{
	TArray<FStatDefinition> StatDefinitions;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FStatDefinition)
	ONLINE_STRUCT_FIELD(FStatDefinition, Name),
	ONLINE_STRUCT_FIELD(FStatDefinition, Id),
	ONLINE_STRUCT_FIELD(FStatDefinition, ModifyMethod),
	ONLINE_STRUCT_FIELD(FStatDefinition, DefaultValue)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FStatsCommonConfig)
	ONLINE_STRUCT_FIELD(FStatsCommonConfig, StatDefinitions)
END_ONLINE_STRUCT_META()

/* Meta */ }

struct FFindUserStatsByAccountId
{
	FFindUserStatsByAccountId(const FAccountId& InAccountId)
		: AccountId(InAccountId)
	{
	}

	bool operator()(const FUserStats& UserStats) const
	{
		return UserStats.AccountId == AccountId;
	}

	FAccountId AccountId;
};

class FStatsCommon : public TOnlineComponent<IStats>
{
public:
	using Super = IStats;

	UE_API FStatsCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void UpdateConfig() override;
	UE_API virtual void RegisterCommands() override;

	// IStats
	UE_API virtual TOnlineAsyncOpHandle<FUpdateStats> UpdateStats(FUpdateStats::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FQueryStats> QueryStats(FQueryStats::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FBatchQueryStats> BatchQueryStats(FBatchQueryStats::Params&& Params) override;

	// Example implementation that uses the event to stats mapping rules
	// In this version we assume that FTriggerEvent::Params is a mapping of property names to values,
	// Assumes that FTriggerEvent::Params is defined as TMap<FString, FSchemaVariant>
	// and that each stat referenced in the event has a defined modify rule: Set, Sum, Largest, Smallest
	// After applying the stat changes to the base class cache, we're also broadcasting an update through OnStatsUpdatedEvent.Broadcast().
	UE_API virtual TOnlineAsyncOpHandle<FTriggerEvent> TriggerEvent(FTriggerEvent::Params&& Params) override;

#if !UE_BUILD_SHIPPING
	UE_API virtual TOnlineAsyncOpHandle<FResetStats> ResetStats(FResetStats::Params&& Params) override;
#endif // !UE_BUILD_SHIPPING
	UE_API virtual TOnlineResult<FGetCachedStats> GetCachedStats(FGetCachedStats::Params&& Params) const override;

	UE_API virtual TOnlineEvent<void(const FStatsUpdated&)> OnStatsUpdated() override;
	UE_API const FStatDefinition* GetStatDefinition(const FString& StatName) const;

protected:
	UE_API virtual TOnlineAsyncOpRef<FUpdateStats> UpdateStats_Implementation(FUpdateStats::Params&& Params);
	UE_API virtual void ReadStatsFromCache(const FUserStats* ExistingUserStats, const TArray<FString>& StatNames, TMap<FString, FStatValue>& OutStats) const;
	UE_API virtual void CacheUserStats(const FUserStats& UserStats);

	TMap<FString, FStatDefinition> StatDefinitions;
	TOnlineEventCallable<void(const FStatsUpdated& StatsUpdated)> OnStatsUpdatedEvent;

	TArray<FUserStats> CachedUsersStats;
};

/* UE::Online */ }

#undef UE_API
