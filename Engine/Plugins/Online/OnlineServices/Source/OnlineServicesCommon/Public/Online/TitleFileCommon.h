// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/TitleFile.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FTitleFileCommon : public TOnlineComponent<ITitleFile>
{
public:
	using Super = ITitleFile;

	UE_API FTitleFileCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// ITitleFile
	UE_API virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) override;
	UE_API virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) override;
};

/* UE::Online */}

#undef UE_API
