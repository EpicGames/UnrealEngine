// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserFile.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FUserFileCommon : public TOnlineComponent<IUserFile>
{
public:
	using Super = IUserFile;

	UE_API FUserFileCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// IUserFile
	UE_API virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) override;
	UE_API virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileCopyFile> CopyFile(FUserFileCopyFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) override;
};

/* UE::Online */}

#undef UE_API
