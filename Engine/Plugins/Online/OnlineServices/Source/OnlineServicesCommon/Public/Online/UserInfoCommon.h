// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/UserInfo.h"
#include "Online/OnlineComponent.h"

#define UE_API ONLINESERVICESCOMMON_API

namespace UE::Online {

class FOnlineServicesCommon;

class FUserInfoCommon : public TOnlineComponent<IUserInfo>
{
public:
	using Super = IUserInfo;

	UE_API FUserInfoCommon(FOnlineServicesCommon& InServices);

	// TOnlineComponent
	UE_API virtual void RegisterCommands() override;

	// IUserInfo
	UE_API virtual TOnlineAsyncOpHandle<FQueryUserInfo> QueryUserInfo(FQueryUserInfo::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetUserInfo> GetUserInfo(FGetUserInfo::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FQueryUserAvatar> QueryUserAvatar(FQueryUserAvatar::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetUserAvatar> GetUserAvatar(FGetUserAvatar::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FShowUserProfile> ShowUserProfile(FShowUserProfile::Params&& Params) override;
};

/* UE::Online */ }

#undef UE_API
