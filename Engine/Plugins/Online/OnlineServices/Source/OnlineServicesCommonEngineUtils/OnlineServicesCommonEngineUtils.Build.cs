// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineServicesCommonEngineUtils : ModuleRules
{
	public OnlineServicesCommonEngineUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine"
				}
			);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreOnline",
				"CoreUObject",
				"OnlineServicesInterface",
				"OnlineServicesCommon",
				"OnlineBase",
				"Sockets"
			}
		);
	}
}
