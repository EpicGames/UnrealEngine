// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesCommonEngineUtils.h"
#include "Online/CoreOnline.h"
#include "Online/OnlineServicesLog.h"

#if WITH_ENGINE
#include "Engine/GameEngine.h"
#include "Engine/NetDriver.h"
#endif

namespace UE::Online {

#if WITH_ENGINE
UWorld* GetWorldForOnline(FName InstanceName)
{
	UWorld* World = NULL;
#if WITH_EDITOR
	if (InstanceName.ToString() != LexToString(UE::Online::EOnlineServices::Default) && InstanceName != NAME_None)
	{
		// The world context is not guaranteed to always be valid (e.g: PIE teardown)
		if (const FWorldContext* WorldContext = GEngine->GetWorldContextFromHandle(InstanceName))
		{
			check(WorldContext->WorldType == EWorldType::Game || WorldContext->WorldType == EWorldType::PIE);
			World = WorldContext->World();
		}
	}
	else
#endif
	{
		UGameEngine* GameEngine = Cast<UGameEngine>(GEngine);
		World = GameEngine ? GameEngine->GetGameWorld() : NULL;
	}

	return World;
}
#endif

int32 GetPortFromNetDriver(FName InstanceName)
{
	int32 Port = 0;
#if WITH_ENGINE
	if (GEngine)
	{
		UWorld* World = GetWorldForOnline(InstanceName);
		UNetDriver* NetDriver = World ? GEngine->FindNamedNetDriver(World, NAME_GameNetDriver) : NULL;
		if (NetDriver && NetDriver->GetNetMode() < NM_Client)
		{
			FString AddressStr = NetDriver->LowLevelGetNetworkNumber();
			int32 Colon = AddressStr.Find(":", ESearchCase::IgnoreCase, ESearchDir::FromEnd);
			if (Colon != INDEX_NONE)
			{
				FString PortStr = AddressStr.Mid(Colon + 1);
				// Require fully-numeric trailing token: rejects non-IP addresses like EOS:<PUID> where Atoi would consume hex-prefix digits and yield a bogus port.
				if (PortStr.IsNumeric())
				{
					const int32 ParsedPort = FCString::Atoi(*PortStr);
					if (ParsedPort >= 0 && ParsedPort <= MAX_uint16)
					{
						Port = ParsedPort;
					}
					else
					{
						UE_LOGF(LogOnlineServices, Verbose, "GetPortFromNetDriver: parsed port %d from address '%ls' is out of range [0, %u]; returning 0.", ParsedPort, *AddressStr, MAX_uint16);
					}
				}
				else
				{
					UE_LOGF(LogOnlineServices, Verbose, "GetPortFromNetDriver: trailing token '%ls' from address '%ls' is not numeric; returning 0.", *PortStr, *AddressStr);
				}
			}
		}
	}
#endif
	return Port;
}

/* UE::Online */ }
