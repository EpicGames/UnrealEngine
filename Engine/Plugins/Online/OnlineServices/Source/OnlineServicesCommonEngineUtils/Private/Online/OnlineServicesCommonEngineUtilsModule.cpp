// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FOnlineServicesCommonEngineUtilsModule : public IModuleInterface
{
public:
	virtual void StartupModule()
	{

	}

	virtual void ShutdownModule()
	{

	}
};

IMPLEMENT_MODULE(FOnlineServicesCommonEngineUtilsModule, OnlineServicesCommonEngineUtils);
