// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsLAN.h"

#include "Online/LANBeacon.h"
#include "Online/OnlineServicesCommonEngineUtils.h"
#include "Online/NboSerializer.h"
#include "SocketSubsystem.h"
#include "UObject/CoreNet.h"
#include "Online/NboSerializerCommonSvc.h"

namespace UE::Online {

/** FOnlineSessionIdRegistryLAN */
FOnlineSessionIdRegistryLAN::FOnlineSessionIdRegistryLAN(EOnlineServices ServicesType)
	: FOnlineSessionIdStringRegistry(ServicesType)
{

}

FOnlineSessionIdRegistryLAN& FOnlineSessionIdRegistryLAN::GetChecked(EOnlineServices ServicesType)
{
	FOnlineSessionIdRegistryLAN* SessionIdRegistry = static_cast<FOnlineSessionIdRegistryLAN*>(FOnlineIdRegistryRegistry::Get().GetSessionIdRegistry(ServicesType));
	check(SessionIdRegistry);
	return *SessionIdRegistry;
}

FOnlineSessionId FOnlineSessionIdRegistryLAN::GetNextSessionId()
{
	return BasicRegistry.FindOrAddHandle(FGuid::NewGuid().ToString());
}

/** NboSerializerLAN */

namespace NboSerializerLANSvc {

void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionLAN& Session)
{
	NboSerializerCommonSvc::SerializeToBuffer(Packet, Session);
	Packet << *Session.OwnerInternetAddr;
}

void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionLAN& Session)
{
	NboSerializerCommonSvc::SerializeFromBuffer(Packet, Session);
	Packet >> *Session.OwnerInternetAddr;
}

/* NboSerializerLANSvc */ }

/** FSessionLAN */

FSessionLAN::FSessionLAN()
{
	Initialize();
}

void FSessionLAN::Initialize()
{
	bool bCanBindAll;
	OwnerInternetAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);

	// The below is a workaround for systems that set hostname to a distinct address from 127.0.0.1 on a loop back interface.
	// See e.g. https://www.debian.org/doc/manuals/debian-reference/ch05.en.html#_the_hostname_resolution
	// and http://serverfault.com/questions/363095/why-does-my-hostname-appear-with-the-address-127-0-1-1-rather-than-127-0-0-1-in
	// Since we bind to 0.0.0.0, we won't answer on 127.0.1.1, so we need to advertise ourselves as 127.0.0.1 for any other loop back address we may have.

	uint32 HostIp = 0;
	OwnerInternetAddr->GetIp(HostIp); // Will return in host order

	// If this address is on loop back interface, advertise it as 127.0.0.1
	if ((HostIp & 0xff000000) == 0x7f000000)
	{
		OwnerInternetAddr->SetIp(0x7f000001); // 127.0.0.1
	}

	// Guard SetPort with IsPortValid: GetPortFromNetDriver can return out-of-range values for non-IP addresses (e.g. EOS:<PUID>).
	const int32 NetDriverPort = GetPortFromNetDriver(NAME_None);
	if (OwnerInternetAddr->IsPortValid(NetDriverPort))
	{
		OwnerInternetAddr->SetPort(NetDriverPort);
	}

	if (OwnerInternetAddr->GetPort() == 0)
	{
		OwnerInternetAddr->SetPort(7777); // Default port
	}

	// We'll set the connect address for the remote session as a custom parameter, so it can be read in OnlineServices' GetResolvedConnectString
	FCustomSessionSetting ConnectString;
	ConnectString.Data.Set(OwnerInternetAddr->ToString(true));
	ConnectString.Visibility = ESchemaAttributeVisibility::Public;
	SessionSettings.CustomSettings.Add(CONNECT_STRING_TAG, ConnectString);
}

FSessionLAN& FSessionLAN::Cast(FSessionCommon& InSession)
{
	return static_cast<FSessionLAN&>(InSession);
}

const FSessionLAN& FSessionLAN::Cast(const ISession& InSession)
{
	return static_cast<const FSessionLAN&>(InSession);
}

void FSessionLAN::DumpState() const
{
	FSessionCommon::DumpState();
	UE_LOGF(LogOnlineServices, Log, "OwnerInternetAddr [%ls]", *OwnerInternetAddr->ToString(true));
}

/** FSessionsLAN */

FSessionsLAN::FSessionsLAN(FOnlineServicesCommon& InServices)
	: Super(InServices)
	, LANSessionManager(MakeShared<FLANSession>())
{

}

FSessionsLAN::~FSessionsLAN()
{
	// Stop hosted LAN session if one is running
	if (HostedLANSession)
	{
		StopLANSession();
	}
}

void FSessionsLAN::Tick(float DeltaSeconds)
{
	if (LANSessionManager->GetBeaconState() > ELanBeaconState::NotUsingLanBeacon)
	{
		LANSessionManager->Tick(DeltaSeconds);
	}

	Super::Tick(DeltaSeconds);
}

TFuture<TOnlineResult<FCreateSession>> FSessionsLAN::CreateSessionImpl(const FCreateSession::Params& Params)
{
	TPromise<TOnlineResult<FCreateSession>> Promise;
	TFuture<TOnlineResult<FCreateSession>> Future = Promise.GetFuture();

	// We'll only host on the LAN beacon for public sessions
	if (Params.SessionSettings.JoinPolicy == ESessionJoinPolicy::Public)
	{
		FHostSessionResult TryHostLANSessionResult = TryHostLANSession();
		if (TryHostLANSessionResult.HasError())
		{
			Promise.EmplaceValue(TryHostLANSessionResult.StealError());
			return Future;
		}

		UE_LOGF(LogOnlineServices, Verbose, "Hosting LAN session named '%ls'", *(Params.SessionName.ToString()));
		HostedLANSession = AddLANSession(Params);
	}
	else
	{
		AddLANSession(Params);
	}

	Promise.EmplaceValue(FCreateSession::Result{ });

	return Future;
}

TFuture<TOnlineResult<FUpdateSessionSettings>> FSessionsLAN::UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params)
{
	TPromise<TOnlineResult<FUpdateSessionSettings>> Promise;
	TFuture<TOnlineResult<FUpdateSessionSettings>> Future = Promise.GetFuture();

	TOnlineResult<FGetMutableSessionByName> GetMutableSessionByNameResult = GetMutableSessionByName({ Params.SessionName });
	check(GetMutableSessionByNameResult.IsOk());

	// We'll individually check only the settings where additional logic is required

	TSharedRef<FSessionCommon> FoundSession = GetMutableSessionByNameResult.GetOkValue().Session;
	FSessionSettings& SessionSettings = FoundSession->SessionSettings;

	if (Params.Mutations.JoinPolicy.IsSet())
	{
		// Changes in the join policy setting will mean we start or stop the LAN Beacon broadcasting the session information
		// There is no FriendsLAN interface at the time of this implementation, hence the binary behavior (Public/Non-Public)
		if (SessionSettings.JoinPolicy != ESessionJoinPolicy::Public && Params.Mutations.JoinPolicy.GetValue() == ESessionJoinPolicy::Public)
		{
			FHostSessionResult TryHostLANSessionResult = TryHostLANSession();
			if (TryHostLANSessionResult.HasError())
			{
				Promise.EmplaceValue(TryHostLANSessionResult.StealError());
				return Future;
			}

			// This session already exists, we are moving it to be a hosted session
			// So we only need to update our hosting session information
			UE_LOGF(LogOnlineServices, Verbose, "Transitioning session %ls to a hosted LAN session.", *Params.SessionName.ToString());
			HostedLANSession = StaticCastSharedRef<FSessionLAN>(FoundSession).ToSharedPtr();
		}
		else if (SessionSettings.JoinPolicy == ESessionJoinPolicy::Public && Params.Mutations.JoinPolicy.GetValue() != ESessionJoinPolicy::Public)
		{
			// Stop hosting the LAN session, since it's no longer public
			LANSessionManager->StopLANSession();
			HostedLANSession.Reset();
			UE_LOGF(LogOnlineServices, Verbose, "No longer hosting LAN session %ls as it's no longer publicly accessible.", *Params.SessionName.ToString());
		}
	}

	// We update our local session
	FSessionUpdate SessionUpdateData = BuildSessionUpdate(FoundSession, Params.Mutations);

	(*FoundSession) += SessionUpdateData;

	// We set the result and fire the event
	Promise.EmplaceValue(FUpdateSessionSettings::Result{ });

	FSessionUpdated SessionUpdatedEvent{ Params.SessionName, SessionUpdateData };
	SessionEvents.OnSessionUpdated.Broadcast(SessionUpdatedEvent);

	return Future;
}

TFuture<TOnlineResult<FFindSessions>> FSessionsLAN::FindSessionsImpl(const FFindSessions::Params& Params)
{
	TPromise<TOnlineResult<FFindSessions>> Promise;
	TFuture<TOnlineResult<FFindSessions>> Future = Promise.GetFuture();

	// Before we start the search, we reset the cache and save the promise
	SearchResultsUserMap.FindOrAdd(Params.LocalAccountId).Reset();
	CurrentSessionSearchPromisesUserMap.Emplace(Params.LocalAccountId, MoveTemp(Promise));

	// If we were hosting public sessions before the search, add it to the search results
	if (HostedLANSession)
	{
		UE_LOGF(LogOnlineServices, Verbose, "[FSessionsLAN::FindSessionsImpl] Adding local hosted LAN session with ID %ls to search results.", *ToString(HostedLANSession->GetSessionId()));
		SearchResultsUserMap.FindOrAdd(Params.LocalAccountId).Add(HostedLANSession->GetSessionId());
	}

	FindLANSessions(Params.LocalAccountId);

	return Future;
}

TFuture<TOnlineResult<FJoinSession>> FSessionsLAN::JoinSessionImpl(const FJoinSession::Params& Params)
{
	TPromise<TOnlineResult<FJoinSession>> Promise;
	TFuture<TOnlineResult<FJoinSession>> Future = Promise.GetFuture();

	TOnlineResult<FGetSessionById> GetSessionByIdResult = GetSessionById({ Params.SessionId });
	if (GetSessionByIdResult.IsError())
	{
		// If no result is found, the id might be expired, which we should notify
		if (FOnlineSessionIdRegistryLAN::GetChecked(Services.GetServicesProvider()).IsSessionIdExpired(Params.SessionId))
		{
			UE_LOGF(LogOnlineServices, Warning, "[%ls] SessionId parameter [%ls] is expired. Please call FindSessions to get an updated list of available sessions ", UTF8_TO_TCHAR(__FUNCTION__), *ToLogString(Params.SessionId));
		}

		Promise.EmplaceValue(GetSessionByIdResult.GetErrorValue());
		return Future;
	}

	const TSharedRef<const ISession>& FoundSession = GetSessionByIdResult.GetOkValue().Session;

	// We set up the appropriate references for the session
	AddSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

	Promise.EmplaceValue(FJoinSession::Result{ });

	FSessionJoined SessionJoinedEvent = { Params.LocalAccountId, FoundSession->GetSessionId() };
		
	SessionEvents.OnSessionJoined.Broadcast(SessionJoinedEvent);

	return Future;
}

TFuture<TOnlineResult<FLeaveSession>> FSessionsLAN::LeaveSessionImpl(const FLeaveSession::Params& Params)
{
	TPromise<TOnlineResult<FLeaveSession>> Promise;
	TFuture<TOnlineResult<FLeaveSession>> Future = Promise.GetFuture();

	TOnlineResult<FGetSessionByName> GetSessionByNameResult = GetSessionByName({ Params.SessionName });
	if (GetSessionByNameResult.IsOk())
	{
		TSharedRef<const ISession> FoundSession = GetSessionByNameResult.GetOkValue().Session;

		if (FoundSession->GetOwnerAccountId() == Params.LocalAccountId && FoundSession->GetSessionSettings().JoinPolicy == ESessionJoinPolicy::Public)
		{
			StopLANSession();
		}

		ClearSessionReferences(FoundSession->GetSessionId(), Params.SessionName, Params.LocalAccountId);

		Promise.EmplaceValue(FLeaveSession::Result{ });

		FSessionLeft SessionLeftEvent;
		SessionLeftEvent.LocalAccountId = Params.LocalAccountId;
		SessionEvents.OnSessionLeft.Broadcast(SessionLeftEvent);
	}
	else
	{
		Promise.EmplaceValue(GetSessionByNameResult.GetErrorValue());
	}

	return Future;
}

/** LANSessionManager methods */

FSessionsLAN::FHostSessionResult FSessionsLAN::TryHostLANSession()
{
	// The LAN Beacon can only broadcast one session at a time
	if (LANSessionManager->GetBeaconState() == ELanBeaconState::NotUsingLanBeacon)
	{
		FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateThreadSafeSP(this, &FSessionsLAN::OnValidQueryPacketReceived);

		if (LANSessionManager->Host(QueryPacketDelegate))
		{
			UE_LOGF(LogOnlineServices, VeryVerbose, "[FSessionsLAN::TryHostLANSession] LAN Beacon hosting...");

			return MakeValue();
		}
		else
		{
			UE_LOGF(LogOnlineServices, VeryVerbose, "[FSessionsLAN::TryHostLANSession] LAN Beacon failed to host!");

			LANSessionManager->StopLANSession();

			return MakeError(Errors::RequestFailure());
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsLAN::TryHostLANSession] LAN Beacon already in use!");
		return MakeError(Errors::AlreadyPending());
	}
}

TSharedRef<FSessionLAN> FSessionsLAN::AddLANSession(const FCreateSession::Params& Params)
{
	TSharedRef<FSessionLAN> NewSessionLANRef = MakeShared<FSessionLAN>();
	NewSessionLANRef->OwnerAccountId = Params.LocalAccountId;
	NewSessionLANRef->SessionInfo.bAllowSanctionedPlayers = Params.bAllowSanctionedPlayers;
	NewSessionLANRef->SessionInfo.bAntiCheatProtected = Params.bAntiCheatProtected;
	NewSessionLANRef->SessionInfo.bIsDedicatedServerSession = IsRunningDedicatedServer();
	NewSessionLANRef->SessionInfo.bIsLANSession = true;
	NewSessionLANRef->SessionInfo.SessionId = FOnlineSessionIdRegistryLAN::GetChecked(Services.GetServicesProvider()).GetNextSessionId();
	NewSessionLANRef->SessionInfo.SessionIdOverride = Params.SessionIdOverride;
	NewSessionLANRef->SessionSettings = Params.SessionSettings;

	// We save the local object for the session, and set up the appropriate references
	AddSessionWithReferences(NewSessionLANRef, Params.SessionName, Params.LocalAccountId, Params.bPresenceEnabled);

	return NewSessionLANRef;
}

void FSessionsLAN::FindLANSessions(const FAccountId& LocalAccountId)
{
	// Recreate the unique identifier for this client
	GenerateNonce((uint8*)&LANSessionManager->LanNonce, 8);

	// Bind delegates
	FOnValidResponsePacketDelegate ResponseDelegate = FOnValidResponsePacketDelegate::CreateThreadSafeSP(this, &FSessionsLAN::OnValidResponsePacketReceived, LocalAccountId);
	FOnSearchingTimeoutDelegate TimeoutDelegate = FOnSearchingTimeoutDelegate::CreateThreadSafeSP(this, &FSessionsLAN::OnLANSearchTimeout, LocalAccountId);

	FNboSerializeToBuffer Packet(LAN_BEACON_MAX_PACKET_SIZE);
	LANSessionManager->CreateClientQueryPacket(Packet, LANSessionManager->LanNonce);
	if (LANSessionManager->Search(Packet, ResponseDelegate, TimeoutDelegate) == false)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FSessionsLAN::FindLANSessions] Search failed!");

		if (LANSessionManager->GetBeaconState() == ELanBeaconState::Searching)
		{
			LANSessionManager->StopLANSession();
		}

		// Trigger the delegate as having failed
		CurrentSessionSearchPromisesUserMap.FindAndRemoveChecked(LocalAccountId).EmplaceValue(Errors::RequestFailure());

		// If we were hosting public sessions before the search, we'll return the beacon to that state
		if (HostedLANSession)
		{
			TryHostLANSession();
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Verbose, "[]FSessionsLAN::FindLANSessions] Searching....");
	}
}

void FSessionsLAN::StopLANSession()
{
	check(HostedLANSession);
	HostedLANSession.Reset();
	LANSessionManager->StopLANSession();
}

void FSessionsLAN::OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce)
{
	// Iterate through all registered sessions and respond for each one that can be joinable
	for (const TPair<FOnlineSessionId, TSharedRef<FSessionCommon>>& Entry : AllSessionsById)
	{
		const TSharedRef<FSessionCommon>& Session = Entry.Value;

		if (Session->SessionSettings.JoinPolicy == ESessionJoinPolicy::Public)
		{
			FNboSerializeToBuffer Packet(LAN_BEACON_MAX_PACKET_SIZE);
			// Create the basic header before appending additional information
			LANSessionManager->CreateHostResponsePacket(Packet, ClientNonce);

			// Add all the session details
			AppendSessionToPacket(Packet, FSessionLAN::Cast(*Session));

			// Broadcast this response so the client can see us
			if (!Packet.HasOverflow())
			{
				LANSessionManager->BroadcastPacket(Packet, Packet.GetByteCount());
			}
			else
			{
				UE_LOGF(LogOnlineServices, Warning, "[FSessionsLAN::OnValidQueryPacketReceived] LAN broadcast packet overflow, cannot broadcast on LAN");
			}
		}
	}
}

void FSessionsLAN::OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength, const FAccountId LocalAccountId)
{
	TSharedRef<FSessionLAN> Session = MakeShared<FSessionLAN>();

	FNboSerializeFromBuffer Packet(PacketData, PacketLength);
	ReadSessionFromPacket(Packet, *Session);

	AddSearchResult(Session, LocalAccountId);
}

void FSessionsLAN::OnLANSearchTimeout(const FAccountId LocalAccountId)
{
	if (LANSessionManager->GetBeaconState() == ELanBeaconState::Searching)
	{
		LANSessionManager->StopLANSession();
	}

	const TArray<FOnlineSessionId>& FoundSessionIds = SearchResultsUserMap.FindChecked(LocalAccountId);
	UE_LOGF(LogOnlineServices, Verbose, "[FSessionsLAN::OnLANSearchTimeout] %d sessions found!", FoundSessionIds.Num());

	CurrentSessionSearchPromisesUserMap.FindAndRemoveChecked(LocalAccountId).EmplaceValue(FFindSessions::Result{ FoundSessionIds });

	// If we were hosting public sessions before the search, we'll return the beacon to that state
	if (HostedLANSession)
	{
		TryHostLANSession();
	}
}

/* UE::Online */ }
