// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FName;

namespace UE::Online {

ONLINESERVICESCOMMONENGINEUTILS_API int32 GetPortFromNetDriver(FName InstanceName);

/* UE::Online */ }
