// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/SessionsCommon.h"
#include "Templates/ValueOrError.h"

#define UE_API ONLINESERVICESCOMMONENGINEUTILS_API

class FInternetAddr;
class FLANSession;
class FNboSerializeFromBuffer;
class FNboSerializeToBuffer;

namespace UE::Online {

class FOnlineServicesCommon;

class FOnlineSessionIdRegistryLAN : public FOnlineSessionIdStringRegistry
{
public:
	static UE_API FOnlineSessionIdRegistryLAN& GetChecked(EOnlineServices ServicesType);

	UE_API FOnlineSessionId GetNextSessionId();

protected:
	UE_API FOnlineSessionIdRegistryLAN(EOnlineServices ServicesType);
};

class FSessionLAN : public FSessionCommon
{
public:
	UE_API FSessionLAN();

	static UE_API FSessionLAN& Cast(FSessionCommon& InSession);
	static UE_API const FSessionLAN& Cast(const ISession& InSession);

	UE_API virtual void DumpState() const override;

private:
	void Initialize();

public:
	/** The IP address for the session owner */
	TSharedPtr<FInternetAddr> OwnerInternetAddr;
};

namespace NboSerializerLANSvc {

void ONLINESERVICESCOMMONENGINEUTILS_API SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionLAN& Session);
void ONLINESERVICESCOMMONENGINEUTILS_API SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionLAN& Session);

/* NboSerializerLANSvc */ }

class FSessionsLAN : public FSessionsCommon
{
public:
	using Super = FSessionsCommon;

	UE_API FSessionsLAN(FOnlineServicesCommon& InServices);
	UE_API virtual ~FSessionsLAN() override;

	UE_API virtual void Tick(float DeltaSeconds) override;

	// FSessionsCommon
	UE_API virtual TFuture<TOnlineResult<FCreateSession>> CreateSessionImpl(const FCreateSession::Params& Params) override;
	UE_API virtual TFuture<TOnlineResult<FUpdateSessionSettings>> UpdateSessionSettingsImpl(const FUpdateSessionSettings::Params& Params) override;
	UE_API virtual TFuture<TOnlineResult<FLeaveSession>> LeaveSessionImpl(const FLeaveSession::Params& Params) override;
	UE_API virtual TFuture<TOnlineResult<FFindSessions>> FindSessionsImpl(const FFindSessions::Params& Params) override;
	UE_API virtual TFuture<TOnlineResult<FJoinSession>> JoinSessionImpl(const FJoinSession::Params& Params) override;

protected:
	using FHostSessionResult = TValueOrError<void, FOnlineError>;

	UE_API FHostSessionResult TryHostLANSession();
	UE_API TSharedRef<FSessionLAN> AddLANSession(const FCreateSession::Params& Params);
	UE_API void FindLANSessions(const FAccountId& LocalAccountId);
	UE_API void StopLANSession();
	UE_API void OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce);
	UE_API void OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength, const FAccountId LocalAccountId);
	UE_API void OnLANSearchTimeout(const FAccountId LocalAccountId);
	virtual void AppendSessionToPacket(FNboSerializeToBuffer& Packet, const FSessionLAN& Session) = 0;
	virtual void ReadSessionFromPacket(FNboSerializeFromBuffer& Packet, FSessionLAN& Session) = 0;

	TSharedPtr<FSessionLAN> HostedLANSession = nullptr;
	TSharedRef<FLANSession> LANSessionManager;
};

/* UE::Online */ }

#undef UE_API
