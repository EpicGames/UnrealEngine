// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
#include "Engine/Engine.h"
#include "GameDelegates.h"
#include "Online/OnlineServicesLog.h"
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
#include "Templates/SharedPointer.h"

/**
 * Object cache keyed by world context name that verifies a world context exists before creating a new entry.
 * It also automatically cleans up objects when PIE worlds end after the world contexts have been destroyed.
 */
template<typename ObjectType>
class FWorldContextScopedObjectCache
{
public:
	FWorldContextScopedObjectCache()
	{
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
		EndPlayMapDelegateHandle = FGameDelegates::Get().GetEndPlayMapDelegate().AddLambda(
			[this]()
			{
				TArray<FName> ContextNames;
				Objects.GenerateKeyArray(ContextNames);
				for (FName ContextName : ContextNames)
				{
					if (ContextName != NAME_None && (!GEngine || !GEngine->GetWorldContextFromHandle(ContextName)))
					{
						Objects.Remove(ContextName);
					}
				}
			});
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
	}

	~FWorldContextScopedObjectCache()
	{
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
		FGameDelegates::Get().GetEndPlayMapDelegate().Remove(EndPlayMapDelegateHandle);
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
	}

	TSharedPtr<ObjectType> FindOrAdd(FName ContextName, const TFunctionRef<TSharedRef<ObjectType>()>& Create, bool bSkipWorldContextCheck = false)
	{
		if (TSharedRef<ObjectType>* Object = Objects.Find(ContextName))
		{
			return *Object;
		}

		// Only create new objects if context is none (non-PIE) or the context exists (PIE).
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
		if (ContextName == NAME_None || bSkipWorldContextCheck || GEngine->GetWorldContextFromHandle(ContextName))
		{
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
			return Objects.Emplace(ContextName, Create());
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
		}

		UE_LOGF(LogOnlineServices, Log, "[FWorldContextScopedObjectCache::FindOrAdd] Trying to create object for context that does not exist: %ls", *ContextName.ToString());
		return nullptr;
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
	}

	/** Explicitly clear an object for the given context name. Expected to only be used in strictly controlled scenarios, such as automated tests. */
	void Clear(FName ContextName)
	{
		Objects.Remove(ContextName);
	}

private:
	TMap<FName, TSharedRef<ObjectType>> Objects;
#if UE_EDITOR && !UE_IS_COOKED_EDITOR
	FDelegateHandle EndPlayMapDelegateHandle;
#endif // UE_EDITOR && !UE_IS_COOKED_EDITOR
};
