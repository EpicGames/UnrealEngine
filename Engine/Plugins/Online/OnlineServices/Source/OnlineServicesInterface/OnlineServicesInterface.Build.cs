// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineServicesInterface : ModuleRules
{
	public OnlineServicesInterface(ReadOnlyTargetRules Target) : base(Target)
    {
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"CoreOnline",
				"OnlineBase"
			}
		);

		// OnlineService cannot depend on Engine!
		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
			}
		);
	}
}
