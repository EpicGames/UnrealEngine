<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="UE::Online::TResult&lt;*,*&gt;">
    <DisplayString Condition="Storage.TypeIndex == 0">Success {*($T1*)(&amp;Storage.Storage)}</DisplayString>
    <DisplayString Condition="Storage.TypeIndex == 1">Failure {*($T2*)(&amp;Storage.Storage)}</DisplayString>
    <Expand>
      <ExpandedItem Condition="Storage.TypeIndex == 0">*($T1*)(&amp;Storage.Storage)</ExpandedItem>
      <ExpandedItem Condition="Storage.TypeIndex == 1">*($T2*)(&amp;Storage.Storage)</ExpandedItem>
    </Expand>
  </Type>

</AutoVisualizer>