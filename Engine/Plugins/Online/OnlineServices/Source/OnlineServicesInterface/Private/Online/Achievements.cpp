// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Achievements.h"

namespace UE::Online {

FString ToLogString(const FAchievementStatDefinition& Definition)
{
	return FString::Printf(TEXT("StatId=\"%s\" UnlockThreshold=%d"),
		*Definition.StatId, Definition.UnlockThreshold);
}

FString ToLogString(const FAchievementDefinition& Definition)
{
	static constexpr TCHAR FormatString[] = TEXT(
		"AchievementId=\"%s\" "
		"UnlockedDisplayName=\"%s\" "
		"UnlockedDescription=\"%s\" "
		"LockedDisplayName=\"%s\" "
		"LockedDescription=\"%s\" "
		"FlavorText=\"%s\" "
		"UnlockedIconUrl=\"%s\" "
		"LockedIconUrl=\"%s\" "
		"bIsHidden=%s "
		"StatDefinitions=[%s]");

	return FString::Printf(FormatString,
		*Definition.AchievementId,
		*Definition.UnlockedDisplayName.ToString(),
		*Definition.UnlockedDescription.ToString(),
		*Definition.LockedDisplayName.ToString(),
		*Definition.LockedDescription.ToString(),
		*Definition.FlavorText.ToString(),
		*Definition.UnlockedIconUrl,
		*Definition.LockedIconUrl,
		*::LexToString(Definition.bIsHidden),
		*FString::JoinBy(Definition.StatDefinitions, TEXT(", "), [](const FAchievementStatDefinition& Definition) { return ToLogString(Definition); })
	);
}

FString ToLogString(const FAchievementState& State)
{
	return FString::Printf(TEXT("AchievementId=\"%s\" Progress=%.2f UnlockTime=%s"),
		*State.AchievementId, State.Progress, *State.UnlockTime.ToString());
}

bool operator==(const FAchievementStateUpdated& A, const FAchievementStateUpdated& B)
{
	return A.LocalAccountId == B.LocalAccountId
		&& A.AchievementIds == B.AchievementIds;
}


/* UE::Online */ }
