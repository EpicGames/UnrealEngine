// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Auth.h"

namespace UE::Online {

namespace LoginCredentialsType
{
const FName Auto = TEXT("Auto");
const FName Password = TEXT("Password");
const FName ExchangeCode = TEXT("ExchangeCode");
const FName PersistentAuth = TEXT("PersistentAuth");
const FName Developer = TEXT("Developer");
const FName RefreshToken = TEXT("RefreshToken");
const FName AccountPortal = TEXT("AccountPortal");
const FName ExternalAuth = TEXT("ExternalAuth");
}

namespace ExternalLoginType
{
const FName Epic = TEXT("Epic");
const FName SteamSessionTicket = TEXT("SteamSessionTicket");
const FName PsnIdToken = TEXT("PsnIdToken");
const FName XblXstsToken = TEXT("XblXstsToken");
const FName DiscordAccessToken = TEXT("DiscordAccessToken");
const FName GogSessionTicket = TEXT("GogSessionTicket");
const FName NintendoIdToken = TEXT("NintendoIdToken");
const FName NintendoNsaIdToken = TEXT("NintendoNsaIdToken");
const FName UplayAccessToken = TEXT("UplayAccessToken");
const FName OpenIdAccessToken = TEXT("OpenIdAccessToken");
const FName DeviceIdAccessToken = TEXT("DeviceIdAccessToken");
const FName AppleIdToken = TEXT("AppleIdToken");
const FName GoogleIdToken = TEXT("GoogleIdToken");
const FName OculusUserIdNonce = TEXT("OculusUserIdNonce");
const FName ItchioJwt = TEXT("ItchioJwt");
const FName ItchioKey = TEXT("ItchioKey");
const FName EpicIdToken = TEXT("EpicIdToken");
const FName AmazonAccessToken = TEXT("AmazonAccessToken");
}

namespace ExternalServerAuthTicketType
{
const FName PsnAuthCode = TEXT("PsnAuthCode");
const FName XblXstsToken = TEXT("XblXstsToken");
}

namespace AccountAttributeData
{
const FSchemaAttributeId DisplayName = TEXT("DisplayName");
}

const TCHAR* LexToString(ELoginStatus Value)
{
	switch (Value)
	{
	case ELoginStatus::UsingLocalProfile:	return TEXT("UsingLocalProfile");
	case ELoginStatus::LoggedIn:			return TEXT("LoggedIn");
	case ELoginStatus::LoggedInReducedFunctionality:	return TEXT("LoggedInReducedFunctionality");
	default:								checkNoEntry(); // Intentional fallthrough
	case ELoginStatus::NotLoggedIn:			return TEXT("NotLoggedIn");
	}
}

bool LexTryParseString(ELoginStatus& OutValue, const TCHAR* InStr)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(InStr, TEXT(#Enum)) == 0) { OutValue = ELoginStatus::Enum; return true; }
	ENUM_CASE_FROM_STRING(LoggedIn);
	ENUM_CASE_FROM_STRING(UsingLocalProfile);
	ENUM_CASE_FROM_STRING(LoggedInReducedFunctionality);
	ENUM_CASE_FROM_STRING(NotLoggedIn);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(ELoginStatus& OutValue, const TCHAR* InStr)
{
	if (!ensureAlwaysMsgf(LexTryParseString(OutValue, InStr), TEXT("Unable to parse ELoginStatus value: %s"), InStr))
	{
		OutValue = ELoginStatus::NotLoggedIn;
	}
}

const TCHAR* LexToString(ERemoteAuthTicketAudience Value)
{
	switch (Value)
	{
	case ERemoteAuthTicketAudience::DedicatedServer:	return TEXT("DedicatedServer");
	default:											checkNoEntry(); // Intentional fallthrough
	case ERemoteAuthTicketAudience::Peer:				return TEXT("Peer");
	}
}

bool LexTryParseString(ERemoteAuthTicketAudience& OutValue, const TCHAR* InStr)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(InStr, TEXT(#Enum)) == 0) { OutValue = ERemoteAuthTicketAudience::Enum; return true; }
	ENUM_CASE_FROM_STRING(Peer);
	ENUM_CASE_FROM_STRING(DedicatedServer);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(ERemoteAuthTicketAudience& OutValue, const TCHAR* InStr)
{
	if (!ensureAlwaysMsgf(LexTryParseString(OutValue, InStr), TEXT("Unable to parse EExternalAuthTokenMethod value: %s"), InStr))
	{
		OutValue = ERemoteAuthTicketAudience::Peer;
	}
}

const TCHAR* LexToString(EExternalAuthTokenMethod Value)
{
	switch (Value)
	{
	case EExternalAuthTokenMethod::Primary:		return TEXT("Primary");
	default:									checkNoEntry(); // Intentional fallthrough
	case EExternalAuthTokenMethod::Secondary:	return TEXT("Secondary");
	}
}

bool LexTryParseString(EExternalAuthTokenMethod& OutValue, const TCHAR* InStr)
{
#define ENUM_CASE_FROM_STRING(Enum) if (FCString::Stricmp(InStr, TEXT(#Enum)) == 0) { OutValue = EExternalAuthTokenMethod::Enum; return true; }
	ENUM_CASE_FROM_STRING(Primary);
	ENUM_CASE_FROM_STRING(Secondary);
#undef ENUM_CASE_FROM_STRING
	return false;
}

void LexFromString(EExternalAuthTokenMethod& OutValue, const TCHAR* InStr)
{
	if (!ensureAlwaysMsgf(LexTryParseString(OutValue, InStr), TEXT("Unable to parse EExternalAuthTokenMethod value: %s"), InStr))
	{
		OutValue = EExternalAuthTokenMethod::Primary;
	}
}

/* UE::Online */ }
