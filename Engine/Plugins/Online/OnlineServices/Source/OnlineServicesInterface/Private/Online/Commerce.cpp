// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Commerce.h"

namespace UE::Online {

const TCHAR* LexToString(EOfferType OfferType)
{
	switch (OfferType)
	{
	case EOfferType::Nonconsumable: return TEXT("Nonconsumable");
	case EOfferType::Consumable:return TEXT("Consumable");
	case EOfferType::Subscription: return TEXT("Subscription");
	default: checkNoEntry();
	case EOfferType::Unknown: return TEXT("Unknown");
	}
}

void LexFromString(EOfferType& OutOfferType, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Nonconsumable")) == 0)
	{
		OutOfferType = EOfferType::Nonconsumable;
	}
	else if (FCString::Stricmp(InStr, TEXT("Consumable")) == 0)
	{
		OutOfferType = EOfferType::Consumable;
	}
	else if (FCString::Stricmp(InStr, TEXT("Subscription")) == 0)
	{
		OutOfferType = EOfferType::Subscription;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unknown")) == 0)
	{
		OutOfferType = EOfferType::Unknown;
	}
	else
	{
		checkNoEntry();
		OutOfferType = EOfferType::Unknown;
	}
}

} // namespace UE::Online