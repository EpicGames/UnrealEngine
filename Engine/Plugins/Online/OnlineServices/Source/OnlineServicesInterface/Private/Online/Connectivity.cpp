// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Connectivity.h"
#include "CoreGlobals.h"

namespace UE::Online {

const TCHAR* LexToString(EOnlineServicesConnectionStatus Status)
{
	switch (Status)
	{
	case EOnlineServicesConnectionStatus::Connected:	return TEXT("Connected");
	default:											checkNoEntry(); // Intentional fallthrough
	case EOnlineServicesConnectionStatus::NotConnected:	return TEXT("NotConnected");
	}
}

void LexFromString(EOnlineServicesConnectionStatus& OutStatus, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Connected")) == 0)
	{
		OutStatus = EOnlineServicesConnectionStatus::Connected;
	}
	else if (FCString::Stricmp(InStr, TEXT("NotConnected")) == 0)
	{
		OutStatus = EOnlineServicesConnectionStatus::NotConnected;
	}
	else
	{
		checkNoEntry();
		OutStatus = EOnlineServicesConnectionStatus::NotConnected;
	}
}

/* UE::Online */}
