// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Lobbies.h"

namespace UE::Online {

const TCHAR* LexToString(ELobbyJoinPolicy Policy)
{
	switch (Policy)
	{
	case ELobbyJoinPolicy::PublicAdvertised:	return TEXT("PublicAdvertised");
	case ELobbyJoinPolicy::PublicNotAdvertised:	return TEXT("PublicNotAdvertised");
	default:									checkNoEntry(); // Intentional fallthrough
	case ELobbyJoinPolicy::InvitationOnly:		return TEXT("InvitationOnly");
	}
}

void LexFromString(ELobbyJoinPolicy& OutPolicy, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("PublicAdvertised")) == 0)
	{
		OutPolicy = ELobbyJoinPolicy::PublicAdvertised;
	}
	else if (FCString::Stricmp(InStr, TEXT("PublicNotAdvertised")) == 0)
	{
		OutPolicy = ELobbyJoinPolicy::PublicNotAdvertised;
	}
	else if (FCString::Stricmp(InStr, TEXT("InvitationOnly")) == 0)
	{
		OutPolicy = ELobbyJoinPolicy::InvitationOnly;
	}
	else
	{
		checkNoEntry();
		OutPolicy = ELobbyJoinPolicy::InvitationOnly;
	}
}

const TCHAR* LexToString(ELobbyMemberLeaveReason LeaveReason)
{
	switch (LeaveReason)
	{
	case ELobbyMemberLeaveReason::Left:			return TEXT("Left");
	case ELobbyMemberLeaveReason::Kicked:		return TEXT("Kicked");
	case ELobbyMemberLeaveReason::Disconnected:	return TEXT("Disconnected");
	default:									checkNoEntry(); // Intentional fallthrough
	case ELobbyMemberLeaveReason::Closed:		return TEXT("Closed");
	}
}

void LexFromString(ELobbyMemberLeaveReason& OutLeaveReason, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Left")) == 0)
	{
		OutLeaveReason = ELobbyMemberLeaveReason::Left;
	}
	else if (FCString::Stricmp(InStr, TEXT("Kicked")) == 0)
	{
		OutLeaveReason = ELobbyMemberLeaveReason::Kicked;
	}
	else if (FCString::Stricmp(InStr, TEXT("Disconnected")) == 0)
	{
		OutLeaveReason = ELobbyMemberLeaveReason::Disconnected;
	}
	else if (FCString::Stricmp(InStr, TEXT("Closed")) == 0)
	{
		OutLeaveReason = ELobbyMemberLeaveReason::Closed;
	}
	else
	{
		checkNoEntry();
		OutLeaveReason = ELobbyMemberLeaveReason::Closed;
	}
}

const TCHAR* LexToString(EUILobbyJoinRequestedSource UILobbyJoinRequestedSource)
{
	switch (UILobbyJoinRequestedSource)
	{
	case EUILobbyJoinRequestedSource::FromInvitation:	return TEXT("FromInvitation");
	default:											checkNoEntry(); // Intentional fallthrough
	case EUILobbyJoinRequestedSource::Unspecified:		return TEXT("Unspecified");
	}
}

void LexFromString(EUILobbyJoinRequestedSource& OutUILobbyJoinRequestedSource, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("FromInvitation")) == 0)
	{
		OutUILobbyJoinRequestedSource = EUILobbyJoinRequestedSource::FromInvitation;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unspecified")) == 0)
	{
		OutUILobbyJoinRequestedSource = EUILobbyJoinRequestedSource::Unspecified;
	}
	else
	{
		checkNoEntry();
		OutUILobbyJoinRequestedSource = EUILobbyJoinRequestedSource::Unspecified;
	}
}

void SortLobbies(const TArray<FFindLobbySearchFilter>& Filters, TArray<TSharedRef<const FLobby>>& Lobbies)
{
	// todo
}

/* UE::Online */ }
