// Copyright Epic Games, Inc. All Rights Reserved.
#include "Online/OnlineError.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "Containers/StringView.h"
#include "Misc/Optional.h"
#include "Misc/Parse.h"
#include "Online/OnlineErrorCode.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Serialization/CompactBinaryWriter.h"
#include "String/ParseTokens.h"

namespace UE::Online{
	namespace Errors {
		uint64 ErrorCodeSystem(ErrorCodeType ErrorCode) { return (ErrorCode >> 60 & 0xfull); }
		uint64 ErrorCodeCategory(ErrorCodeType ErrorCode) { return (ErrorCode >> 32 & 0x0fffffffull); }
		uint64 ErrorCodeValue(ErrorCodeType ErrorCode) { return ErrorCode & 0xffffffffull; }

		namespace ErrorCode
		{
			FString ToString(ErrorCodeType ErrorCode)
			{
				const uint64 System = ErrorCodeSystem(ErrorCode);
				const uint64 Category = ErrorCodeCategory(ErrorCode); 
				const uint64 Code = ErrorCodeValue(ErrorCode); 
				return FString::Printf(TEXT("%llx.%llx.%llx"), System, Category, Code);
			}

			TOptional<ErrorCodeType> FromString(FStringView ErrorCode)
			{
				int32 Count = 0;
				FStringView Fields[3];
				UE::String::ParseTokens(ErrorCode, TCHAR('.'), [&Fields, &Count](FStringView Token)
					{ 
						if (Count < 3)
						{
							if (Algo::AllOf(Token, &FChar::IsHexDigit))
							{
								Fields[Count] = Token;
							}
						}
						Count++;
					});
				
				if (Count != 3 || Algo::AnyOf(Fields, &FStringView::IsEmpty))
				{
					return NullOpt;
				}

				const uint64 System = FParse::HexNumber64(Fields[0]);
				const uint64 Category = FParse::HexNumber64(Fields[1]);
				const uint64 Code = FParse::HexNumber64(Fields[2]);
				return Create(System, Category, Code);
			}
		} /*namespace ErrorCode */
	} /* namespace Errors */

	bool operator==(const FOnlineError& Lhs, const FOnlineError& Rhs)
	{
		const FOnlineError * Error = &Lhs;
		while(Error)
		{
			if(Rhs == Error->GetErrorCode())
			{
				return true;
			}
			Error = Error->GetInner();
		}
		return false;
	}

	bool operator==(const FOnlineError& Lhs, ErrorCodeType OtherErrorCode)
	{
		if (Lhs.GetErrorCode() == OtherErrorCode)
		{
			return true;
		}

		const FOnlineError* LoopInner = Lhs.GetInner();
		while (LoopInner != nullptr)
		{
			if (LoopInner->GetErrorCode() == OtherErrorCode)
			{
				return true;
			}
			LoopInner = LoopInner->GetInner();
		}

		return false;
	}



	void SerializeForLog(FCbWriter& Writer, const FOnlineError& OnlineError)
	{
		
		Writer.BeginObject();
		Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("OnlineError"));


		FText ErrorMessage;
		FFormatNamedArguments ErrorMessageArgs;

		bool bHasDetails = false;
		bool bHasInner = false;

		FString ErrorCodeString = OnlineError.GetErrorId();
		ErrorMessageArgs.Add(TEXT("ErrorCode"), FText::FromString(ErrorCodeString));
		Writer.AddString(ANSITEXTVIEW("ErrorCode"), ErrorCodeString);

		if (OnlineError.Details)
		{
			FText ErrorDetails = OnlineError.Details->GetText(OnlineError);
			ErrorMessageArgs.Add(TEXT("ErrorDetails"), ErrorDetails);
			Writer.AddString(ANSITEXTVIEW("ErrorDetails"), ErrorDetails.ToString());
			bHasDetails = true;
			ErrorMessageArgs.Add(TEXT("FriendlyErrorCode"), FText::FromString(OnlineError.Details->GetFriendlyErrorCode(OnlineError)));
			Writer.AddString(ANSITEXTVIEW("FriendlyErrorCode"), OnlineError.Details->GetFriendlyErrorCode(OnlineError));
		}
		
		if (OnlineError.GetInner() != nullptr)
		{
			FText InnerText = OnlineError.GetInner()->GetText();
			ErrorMessageArgs.Add(TEXT("InnerError"), InnerText);
			Writer.AddString(ANSITEXTVIEW("InnerError"), InnerText.ToString());
			bHasInner = true;
		}

		if (bHasInner && bHasDetails)
		{
			Writer.AddString("$text", FText::Format(NSLOCTEXT("OnlineError", "ErrorDetailsErrorInner", "{ErrorCode} ({FriendlyErrorCode}), {ErrorDetails}, InnerError:{InnerError}"), ErrorMessageArgs).ToString());
		}
		else if (bHasDetails)
		{
			Writer.AddString("$text", FText::Format(NSLOCTEXT("OnlineError", "ErrorDetails", "{ErrorCode} ({FriendlyErrorCode}), {ErrorDetails}"), ErrorMessageArgs).ToString());
		}
		else if (bHasInner)
		{
			Writer.AddString("$text", FText::Format(NSLOCTEXT("OnlineError", "ErrorInner", "{ErrorCode}, InnerError:{InnerError}"), ErrorMessageArgs).ToString());
		}
		else
		{
			Writer.AddString("$text", FText::Format(NSLOCTEXT("OnlineError", "ErrorCode", "{ErrorCode}"), ErrorMessageArgs).ToString());
		}
		

		// Writer.AddString(ANSITEXTVIEW("errorNamespace"), OnlineError.ErrorNamespace);
		Writer.EndObject();
	}

	bool FOnlineError::IsSuccess() const
	{
		return ErrorCode == Errors::ErrorCode::Common::Success;
	}

} /* namespace UE::Online */

