// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServices.h"
#include "Online/OnlineBase.h"

#include "Online/OnlineServicesLog.h"
#include "Online/OnlineServicesRegistry.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

DEFINE_LOG_CATEGORY(LogOnlineServices);

namespace UE::Online {

const TCHAR* LexToString(EOnlinePlatformType EnumVal)
{
	switch (EnumVal)
	{
	case EOnlinePlatformType::Epic:			return TEXT("Epic");
	case EOnlinePlatformType::Steam:			return TEXT("Steam");
	case EOnlinePlatformType::PSN:			return TEXT("PSN");
	case EOnlinePlatformType::Nintendo:		return TEXT("Nintendo");
	case EOnlinePlatformType::XBL:			return TEXT("XBL");
	default:										checkNoEntry(); // Intentional fall-through
	case EOnlinePlatformType::Unknown:		return TEXT("Unknown");
	}
}

void LexFromString(EOnlinePlatformType& OutPlatformType, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Epic")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::Epic;
	}
	else if (FCString::Stricmp(InStr, TEXT("Steam")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::Steam;
	}
	else if (FCString::Stricmp(InStr, TEXT("PSN")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::PSN;
	}
	else if (FCString::Stricmp(InStr, TEXT("Nintendo")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::Nintendo;
	}
	else if (FCString::Stricmp(InStr, TEXT("XBL")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::XBL;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unknown")) == 0)
	{
		OutPlatformType = EOnlinePlatformType::Unknown;
	}
	else
	{
		checkNoEntry();
		OutPlatformType = EOnlinePlatformType::Unknown;
	}
}

int32 GetBuildUniqueId()
{
	static bool bStaticCheck = false;
	static int32 BuildId = 0;
	static bool bUseBuildIdOverride = false;
	static int32 BuildIdOverride = 0;

	if (!bStaticCheck)
	{
		bStaticCheck = true;
		FString BuildIdOverrideCommandLineString;
		if (FParse::Value(FCommandLine::Get(), TEXT("BuildIdOverride="), BuildIdOverrideCommandLineString))
		{
			BuildIdOverride = FCString::Atoi(*BuildIdOverrideCommandLineString);
		}
		if (BuildIdOverride != 0)
		{
			bUseBuildIdOverride = true;
		}
		else
		{
			if (!GConfig->GetBool(TEXT("OnlineServices"), TEXT("bUseBuildIdOverride"), bUseBuildIdOverride, GEngineIni))
			{
				UE_LOGF(LogOnlineServices, Warning, "Missing bUseBuildIdOverride= in [OnlineServices] of DefaultEngine.ini");
			}

			if (!GConfig->GetInt(TEXT("OnlineServices"), TEXT("BuildIdOverride"), BuildIdOverride, GEngineIni))
			{
				UE_LOGF(LogOnlineServices, Warning, "Missing BuildIdOverride= in [OnlineServices] of DefaultEngine.ini");
			}
		}

		if (bUseBuildIdOverride == false)
		{
			// Removed old hashing code to use something more predictable and easier to override for when
			// it's necessary to force compatibility with an older build
			BuildId = FNetworkVersion::GetNetworkCompatibleChangelist();
		}
		else
		{
			BuildId = BuildIdOverride;
		}
		
		// use a cvar so it can be modified at runtime
		TAutoConsoleVariable<int32>& CVarBuildIdOverride = GetBuildIdOverrideCVar();
		CVarBuildIdOverride->Set(BuildId);
	}

	return GetBuildIdOverrideCVar()->GetInt();
}

bool IsLoaded(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	return FOnlineServicesRegistry::Get().IsLoaded(OnlineServices, InstanceName, InstanceConfigName);
}

TSharedPtr<IOnlineServices> GetServices(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	return FOnlineServicesRegistry::Get().GetNamedServicesInstance(OnlineServices, InstanceName, InstanceConfigName);
}

void DestroyService(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	FOnlineServicesRegistry::Get().DestroyNamedServicesInstance(OnlineServices, InstanceName, InstanceConfigName);
}

void DestroyAllNamedServices(EOnlineServices OnlineServices)
{
	FOnlineServicesRegistry::Get().DestroyAllNamedServicesInstances(OnlineServices);
}

void DestroyAllServicesWithName(FName InstanceName)
{
	FOnlineServicesRegistry::Get().DestroyAllServicesInstancesWithName(InstanceName);
}

/* UE::Online */ }
