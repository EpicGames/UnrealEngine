// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Online/OnlineServicesRegistry.h"

class FOnlineServicesInterfaceModule : public IModuleInterface
{
public:
private:
	virtual void ShutdownModule() override
	{
		UE::Online::FOnlineServicesRegistry::TearDown();
	}
};

IMPLEMENT_MODULE(FOnlineServicesInterfaceModule, OnlineServicesInterface);
