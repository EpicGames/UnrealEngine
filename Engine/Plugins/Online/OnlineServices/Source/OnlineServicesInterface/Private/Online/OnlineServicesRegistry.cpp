// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesRegistry.h"
#include "Online/OnlineServices.h"
#include "Online/OnlineServicesDelegates.h"
#include "Online/OnlineServicesLog.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/LazySingleton.h"

namespace UE::Online {

FOnlineServicesRegistry& FOnlineServicesRegistry::Get()
{
	return TLazySingleton<FOnlineServicesRegistry>::Get();
}

void FOnlineServicesRegistry::TearDown()
{
	return TLazySingleton<FOnlineServicesRegistry>::TearDown();
}

EOnlineServices FOnlineServicesRegistry::ResolveServiceName(EOnlineServices OnlineServices) const
{
	if (OnlineServices == EOnlineServices::Default)
	{
		if (DefaultServiceOverride != EOnlineServices::Default)
		{
			OnlineServices = DefaultServiceOverride;
		}
		else
		{
			FString Value;

			if (GConfig->GetString(TEXT("OnlineServices"), TEXT("DefaultServices"), Value, GEngineIni))
			{
				LexFromString(OnlineServices, *Value);
			}
		};
	}
	else if (OnlineServices == EOnlineServices::Platform)
	{
		FString Value;

		if (GConfig->GetString(TEXT("OnlineServices"), TEXT("PlatformServices"), Value, GEngineIni))
		{
			LexFromString(OnlineServices, *Value);
		}
	}

	return OnlineServices;
}

void FOnlineServicesRegistry::RegisterServicesFactory(EOnlineServices OnlineServices, TUniquePtr<IOnlineServicesFactory>&& Factory, int32 Priority)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	FFactoryAndPriority* ExistingFactoryAndPriority = ServicesFactories.Find(OnlineServices);
	if (ExistingFactoryAndPriority == nullptr || ExistingFactoryAndPriority->Priority < Priority)
	{
		UE_LOGF(LogOnlineServices, Log, "%s Type=%ls Priority=%d",
			__FUNCTION__, LexToString(OnlineServices), Priority);
		ServicesFactories.Add(OnlineServices, FFactoryAndPriority(MoveTemp(Factory), Priority));
	}
}

void FOnlineServicesRegistry::UnregisterServicesFactory(EOnlineServices OnlineServices, int32 Priority)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	FFactoryAndPriority* ExistingFactoryAndPriority = ServicesFactories.Find(OnlineServices);
	if (ExistingFactoryAndPriority != nullptr && ExistingFactoryAndPriority->Priority == Priority)
	{
		UE_LOGF(LogOnlineServices, Log, "%s Type=%ls Priority=%d",
			__FUNCTION__, LexToString(OnlineServices), Priority);
		ServicesFactories.Remove(OnlineServices);
	}

	DestroyAllNamedServicesInstances(OnlineServices);
}

bool FOnlineServicesRegistry::IsLoaded(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName) const
{
	OnlineServices = ResolveServiceName(OnlineServices);

	bool bExists = false;
	if (const TMap<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>* OnlineServicesInstances = NamedServiceInstances.Find(OnlineServices))
	{
		bExists = OnlineServicesInstances->Find({ InstanceName, InstanceConfigName }) != nullptr;
	}
	return bExists;
}

TSharedPtr<IOnlineServices> FOnlineServicesRegistry::GetNamedServicesInstance(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	TSharedPtr<IOnlineServices> Services;

	if (OnlineServices < EOnlineServices::None)
	{
		if (TSharedRef<IOnlineServices>* ServicesPtr = NamedServiceInstances.FindOrAdd(OnlineServices).Find({ InstanceName, InstanceConfigName }))
		{
			Services = *ServicesPtr;
		}
		else
		{
			Services = CreateServices(OnlineServices, InstanceName, InstanceConfigName);
			if (Services.IsValid())
			{
				NamedServiceInstances.FindOrAdd(OnlineServices).Add({ InstanceName, InstanceConfigName }, Services.ToSharedRef());
				OnOnlineServicesCreated.Broadcast(Services.ToSharedRef());
			}
		}
	}

	return Services;
}

#if WITH_DEV_AUTOMATION_TESTS
void FOnlineServicesRegistry::SetDefaultServiceOverride(EOnlineServices DefaultService)
{
	// No need to call ResolveServiceName here as a generic services name can be used as a Default Service Override
	DefaultServiceOverride = DefaultService;
}

void FOnlineServicesRegistry::ClearDefaultServiceOverride()
{
	DefaultServiceOverride = EOnlineServices::Default;
}
#endif //WITH_DEV_AUTOMATION_TESTS

void FOnlineServicesRegistry::DestroyNamedServicesInstance(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	if (TSharedRef<IOnlineServices>* ServicesPtr = NamedServiceInstances.FindOrAdd(OnlineServices).Find({ InstanceName, InstanceConfigName }))
	{
		(*ServicesPtr)->Destroy();

		UE_CLOGF(!ServicesPtr->IsUnique(), LogOnlineServices, Error, "%ls online services is still been referenced after shutting down", LexToString((*ServicesPtr)->GetServicesProvider()));
		NamedServiceInstances.FindOrAdd(OnlineServices).Remove({ InstanceName, InstanceConfigName });
	}
}

void FOnlineServicesRegistry::DestroyAllNamedServicesInstances(EOnlineServices OnlineServices)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	if (TMap<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>* ServicesMapPtr = NamedServiceInstances.Find(OnlineServices))
	{
		for (const TPair<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>& ServicesEntryRef : *ServicesMapPtr)
		{
			ServicesEntryRef.Value->Destroy();
			UE_CLOGF(!ServicesEntryRef.Value.IsUnique(), LogOnlineServices, Error, "%ls online services is still been referenced after shutting down", LexToString(ServicesEntryRef.Value->GetServicesProvider()));
		}

		NamedServiceInstances.Remove(OnlineServices);
	}
}

void FOnlineServicesRegistry::DestroyAllServicesInstancesWithName(FName InstanceName)
{
	for (auto NamedServiceIterator = NamedServiceInstances.CreateIterator(); NamedServiceIterator; ++NamedServiceIterator)
	{
		for (auto InstanceIterator = NamedServiceIterator->Value.CreateIterator(); InstanceIterator; ++InstanceIterator)
		{
			if (InstanceIterator->Key.Key == InstanceName)
			{
				TSharedRef<IOnlineServices> Service = InstanceIterator->Value;
				Service->Destroy();
				InstanceIterator.RemoveCurrent();
				UE_CLOGF(!Service.IsUnique(), LogOnlineServices, Error, "%ls online services is still been referenced after shutting down", LexToString(Service->GetServicesProvider()));
			}
		}

		if (NamedServiceIterator->Value.IsEmpty())
		{
			NamedServiceIterator.RemoveCurrent();
		}
	}
}

TSharedPtr<IOnlineServices> FOnlineServicesRegistry::CreateServices(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName)
{
	OnlineServices = ResolveServiceName(OnlineServices);

	// Ensure if modular plugins are enabled (EpicAccount + EpicGame) that nothing asks for Epic.
	const bool bIsMonolithicPlugin = OnlineServices == EOnlineServices::Epic;
	ensure(!(TEMP_ShouldUseEpicModularPlugins() && bIsMonolithicPlugin));

	TSharedPtr<IOnlineServices> Services;

	if (FFactoryAndPriority* FactoryAndPriority = ServicesFactories.Find(OnlineServices))
	{
		Services = FactoryAndPriority->Factory->Create(InstanceName, InstanceConfigName);
		if (Services)
		{
			UE_LOGF(LogOnlineServices, Verbose, "[service_creation_succeeded] OnlineServices=[%ls], InstanceName=[%ls], InstanceConfigName=[%ls]", LexToString(OnlineServices), *InstanceName.ToString(), *InstanceConfigName.ToString());
			Services->Init();
		}
		else
		{
			UE_LOGF(LogOnlineServices, Verbose, "[service_creation_failed] OnlineServices=[%ls], InstanceName=[%ls], InstanceConfigName=[%ls]", LexToString(OnlineServices), *InstanceName.ToString(), *InstanceConfigName.ToString());
		}
	}

	return Services;
}

void FOnlineServicesRegistry::GetAllServicesInstances(TArray<TSharedRef<IOnlineServices>>& OutOnlineServices) const
{
	for (const TPair<EOnlineServices, TMap<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>>& OnlineServiceTypesMaps : NamedServiceInstances)
	{
		for (const TPair<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>& NamedInstance : OnlineServiceTypesMaps.Value)
		{
			OutOnlineServices.Emplace(NamedInstance.Value);
		}
	}
}

FOnlineServicesRegistry::~FOnlineServicesRegistry()
{
	for (TPair<EOnlineServices, TMap<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>>& ServiceInstances : NamedServiceInstances)
	{
		for (TPair<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>& ServiceInstance : ServiceInstances.Value)
		{
			ServiceInstance.Value->Destroy();
		}
	}
}

/* UE::Online */ }
