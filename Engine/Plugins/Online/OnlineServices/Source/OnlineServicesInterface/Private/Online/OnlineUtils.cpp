// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineUtils.h"
#include "Misc/DateTime.h"

namespace UE::Online
{

FString ToLogString(const FDateTime& Time)
{
	return Time.ToString();
}

}
