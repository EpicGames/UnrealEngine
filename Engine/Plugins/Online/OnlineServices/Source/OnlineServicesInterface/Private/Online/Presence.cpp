// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Presence.h"

namespace UE::Online {

const TCHAR* LexToString(EUserPresenceStatus EnumVal)
{
	switch (EnumVal)
	{
	case EUserPresenceStatus::Offline:		return TEXT("Offline");
	case EUserPresenceStatus::Online:		return TEXT("Online");
	case EUserPresenceStatus::Away:			return TEXT("Away");
	case EUserPresenceStatus::ExtendedAway:	return TEXT("ExtendedAway");
	case EUserPresenceStatus::DoNotDisturb:	return TEXT("DoNotDisturb");
	default:								checkNoEntry(); // Intentional fall-through
	case EUserPresenceStatus::Unknown:		return TEXT("Unknown");
	}
}

void LexFromString(EUserPresenceStatus& OutStatus, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Offline")) == 0)
	{
		OutStatus = EUserPresenceStatus::Offline;
	}
	else if (FCString::Stricmp(InStr, TEXT("Online")) == 0)
	{
		OutStatus = EUserPresenceStatus::Online;
	}
	else if (FCString::Stricmp(InStr, TEXT("Away")) == 0)
	{
		OutStatus = EUserPresenceStatus::Away;
	}
	else if (FCString::Stricmp(InStr, TEXT("ExtendedAway")) == 0)
	{
		OutStatus = EUserPresenceStatus::ExtendedAway;
	}
	else if (FCString::Stricmp(InStr, TEXT("DoNotDisturb")) == 0)
	{
		OutStatus = EUserPresenceStatus::DoNotDisturb;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unknown")) == 0)
	{
		OutStatus = EUserPresenceStatus::Unknown;
	}
	else
	{
		checkNoEntry();
		OutStatus = EUserPresenceStatus::Unknown;
	}
}

FText ToLocText(EUserPresenceStatus EnumVal)
{
	static FText OnlineText =  NSLOCTEXT("OnlinePresence", "Online", "Online");
	static FText OfflineText =  NSLOCTEXT("OnlinePresence", "Offline", "Offline");
	static FText AwayText =  NSLOCTEXT("OnlinePresence", "Away", "Away");
	static FText DoNotDisturbText =  NSLOCTEXT("OnlinePresence", "DoNotDisturb", "Do Not Disturb");
	static FText UnknownText =  NSLOCTEXT("OnlinePresence", "Unknown", "Unknown");

	switch (EnumVal)
	{
	case EUserPresenceStatus::Online:
		return OnlineText;
	case EUserPresenceStatus::Offline:
		return OfflineText;
	case EUserPresenceStatus::ExtendedAway:
		// falls through to return away text
	case EUserPresenceStatus::Away:
		return AwayText;
	case EUserPresenceStatus::DoNotDisturb:
		return DoNotDisturbText;
	case EUserPresenceStatus::Unknown:
		return UnknownText;
	}

	return FText::GetEmpty();
}

const TCHAR* LexToString(EUserPresenceJoinability EnumVal)
{
	switch (EnumVal)
	{
	case EUserPresenceJoinability::Public:		return TEXT("Public");
	case EUserPresenceJoinability::FriendsOnly:	return TEXT("FriendsOnly");
	case EUserPresenceJoinability::InviteOnly:	return TEXT("InviteOnly");
	case EUserPresenceJoinability::Private:		return TEXT("Private");
	default:									checkNoEntry(); // Intentional fall-through
	case EUserPresenceJoinability::Unknown:		return TEXT("Unknown");
	}
}

void LexFromString(EUserPresenceJoinability& OutJoinability, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Public")) == 0)
	{
		OutJoinability = EUserPresenceJoinability::Public;
	}
	else if (FCString::Stricmp(InStr, TEXT("FriendsOnly")) == 0)
	{
		OutJoinability = EUserPresenceJoinability::FriendsOnly;
	}
	else if (FCString::Stricmp(InStr, TEXT("InviteOnly")) == 0)
	{
		OutJoinability = EUserPresenceJoinability::InviteOnly;
	}
	else if (FCString::Stricmp(InStr, TEXT("Private")) == 0)
	{
		OutJoinability = EUserPresenceJoinability::Private;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unknown")) == 0)
	{
		OutJoinability = EUserPresenceJoinability::Unknown;
	}
	else
	{
		checkNoEntry();
		OutJoinability = EUserPresenceJoinability::Unknown;
	}
}

const TCHAR* LexToString(EUserPresenceGameStatus EnumVal)
{
	switch (EnumVal)
	{
	case EUserPresenceGameStatus::PlayingThisGame:	return TEXT("PlayingThisGame");
	case EUserPresenceGameStatus::PlayingOtherGame:	return TEXT("PlayingOtherGame");
	default:										checkNoEntry(); // Intentional fall-through
	case EUserPresenceGameStatus::Unknown:			return TEXT("Unknown");
	}
}

void LexFromString(EUserPresenceGameStatus& OutGameStatus, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("PlayingThisGame")) == 0)
	{
		OutGameStatus = EUserPresenceGameStatus::PlayingThisGame;
	}
	else if (FCString::Stricmp(InStr, TEXT("PlayingOtherGame")) == 0)
	{
		OutGameStatus = EUserPresenceGameStatus::PlayingOtherGame;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unknown")) == 0)
	{
		OutGameStatus = EUserPresenceGameStatus::Unknown;
	}
	else
	{
		checkNoEntry();
		OutGameStatus = EUserPresenceGameStatus::Unknown;
	}
}

/* UE::Online */ }
