// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Privileges.h"

namespace UE::Online {

const TCHAR* LexToString(EUserPrivileges Type)
{
	switch (Type)
	{

	case EUserPrivileges::CanPlayOnline:				return TEXT("CanPlayOnline");
	case EUserPrivileges::CanCommunicateViaTextOnline:	return TEXT("CanCommunicateViaTextOnline");
	case EUserPrivileges::CanCommunicateViaVoiceOnline:	return TEXT("CanCommunicateViaVoiceOnline");
	case EUserPrivileges::CanUseUserGeneratedContent:	return TEXT("CanUseUserGeneratedContent");
	case EUserPrivileges::CanCrossPlay:					return TEXT("CanCrossPlay");
	default:											checkNoEntry(); // Intentional fallthrough
	case EUserPrivileges::CanPlay:						return TEXT("CanPlay");
	}
}

void LexFromString(EUserPrivileges& OutType, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("CanPlay")) == 0)
	{
		OutType = EUserPrivileges::CanPlay;
	}
	else if (FCString::Stricmp(InStr, TEXT("CanPlayOnline")) == 0)
	{
		OutType = EUserPrivileges::CanPlayOnline;
	}
	else if (FCString::Stricmp(InStr, TEXT("CanCommunicateViaTextOnline")) == 0)
	{
		OutType = EUserPrivileges::CanCommunicateViaTextOnline;
	}
	else if (FCString::Stricmp(InStr, TEXT("CanCommunicateViaVoiceOnline")) == 0)
	{
		OutType = EUserPrivileges::CanCommunicateViaVoiceOnline;
	}
	else if (FCString::Stricmp(InStr, TEXT("CanUseUserGeneratedContent")) == 0)
	{
		OutType = EUserPrivileges::CanUseUserGeneratedContent;
	}
	else if (FCString::Stricmp(InStr, TEXT("CanUserCrossPlay")) == 0)
	{
		OutType = EUserPrivileges::CanCrossPlay;
	}
	else
	{
		checkNoEntry();
		OutType = EUserPrivileges::CanPlay;
	}
}

FString LexToString(EPrivilegeResults Type)
{
	if (Type == EPrivilegeResults::NoFailures)
	{
		return TEXT("NoFailures");
	}
	else
	{
		TArray<FString> ResultNames;

		if (EnumHasAllFlags(Type, EPrivilegeResults::RequiredPatchAvailable))		ResultNames.Emplace(TEXT("RequiredPatchAvailable"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::RequiredSystemUpdate))			ResultNames.Emplace(TEXT("RequiredSystemUpdate"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::AgeRestrictionFailure))		ResultNames.Emplace(TEXT("AgeRestrictionFailure"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::AccountTypeFailure))			ResultNames.Emplace(TEXT("AccountTypeFailure"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::UserNotFound))					ResultNames.Emplace(TEXT("UserNotFound"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::UserNotLoggedIn))				ResultNames.Emplace(TEXT("UserNotLoggedIn"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::ChatRestriction))				ResultNames.Emplace(TEXT("ChatRestriction"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::UGCRestriction))				ResultNames.Emplace(TEXT("UGCRestriction"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::GenericFailure))				ResultNames.Emplace(TEXT("GenericFailure"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::OnlinePlayRestricted))			ResultNames.Emplace(TEXT("OnlinePlayRestricted"));
		if (EnumHasAllFlags(Type, EPrivilegeResults::NetworkConnectionUnavailable))	ResultNames.Emplace(TEXT("NetworkConnectionUnavailable"));

		return FString::Join(ResultNames, TEXT(" | "));
	}
}

void LexFromString(EPrivilegeResults& OutType, const FString& InStr)
{
	if (FCString::Stricmp(*InStr, TEXT("NoFailures")) == 0)
	{
		OutType = EPrivilegeResults::NoFailures;
	}
	else
	{
		TArray<FString> PrivilegeNames;
		InStr.ParseIntoArray(PrivilegeNames, TEXT(" | "));

		for (const FString& PrivilegeName : PrivilegeNames)
		{
			if (FCString::Stricmp(*PrivilegeName, TEXT("RequiredPatchAvailable")) == 0)
			{
				OutType |= EPrivilegeResults::RequiredPatchAvailable;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("RequiredSystemUpdate")) == 0)
			{
				OutType |= EPrivilegeResults::RequiredSystemUpdate;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("AgeRestrictionFailure")) == 0)
			{
				OutType |= EPrivilegeResults::AgeRestrictionFailure;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("AccountTypeFailure")) == 0)
			{
				OutType |= EPrivilegeResults::AccountTypeFailure;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("UserNotFound")) == 0)
			{
				OutType |= EPrivilegeResults::UserNotFound;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("UserNotLoggedIn")) == 0)
			{
				OutType |= EPrivilegeResults::UserNotLoggedIn;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("ChatRestriction")) == 0)
			{
				OutType |= EPrivilegeResults::ChatRestriction;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("UGCRestriction")) == 0)
			{
				OutType |= EPrivilegeResults::UGCRestriction;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("GenericFailure")) == 0)
			{
				OutType |= EPrivilegeResults::GenericFailure;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("OnlinePlayRestricted")) == 0)
			{
				OutType |= EPrivilegeResults::OnlinePlayRestricted;
			}
			else if (FCString::Stricmp(*PrivilegeName, TEXT("NetworkConnectionUnavailable")) == 0)
			{
				OutType |= EPrivilegeResults::NetworkConnectionUnavailable;
			}
		}
	}
}

FString LexToString(EShowPrivilegeResolutionUI Type)
{
	if (EnumHasAllFlags(Type, EShowPrivilegeResolutionUI::Default))		return TEXT("Default");
	if (EnumHasAllFlags(Type, EShowPrivilegeResolutionUI::Show))			return TEXT("Show");
	if (EnumHasAllFlags(Type, EShowPrivilegeResolutionUI::NotShow))		return TEXT("NotShow");
	return TEXT("Invalid value");
}

void LexFromString(EShowPrivilegeResolutionUI& OutType, const FString& InStr)
{
	if (FCString::Stricmp(*InStr, TEXT("Show")) == 0)
	{
		OutType = EShowPrivilegeResolutionUI::Show;
		return;
	}
	if (FCString::Stricmp(*InStr, TEXT("NotShow")) == 0)
	{
		OutType = EShowPrivilegeResolutionUI::NotShow;
		return;
	}	
	//fallback to default
	OutType = EShowPrivilegeResolutionUI::Default;
	return;
}

/* UE::Online */}