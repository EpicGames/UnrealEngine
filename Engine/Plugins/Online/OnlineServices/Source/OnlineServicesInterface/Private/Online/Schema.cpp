// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Schema.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Misc/EnumRange.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineResult.h"

DEFINE_LOG_CATEGORY_STATIC(LogOnlineSchema, Log, All);

namespace UE::Online {

namespace
{

template <typename T>
T BuildFlagsFromArray(const TArray<T>& Input)
{
	T OutValue = T::None;
	Algo::ForEach(Input, [&](T FlagValue){ OutValue |= FlagValue; });
	return OutValue;
}

static uint32 SchemaNameCrc32(const FName& Name, uint32 CRC=0)
{
	const FString NameString = Name.ToString();
	return FCrc::StrCrc32<TCHAR>(*NameString, CRC);
}

ESchemaServiceAttributeSupportedTypeFlags TranslateAttributeType(ESchemaAttributeType InType)
{
	switch (InType)
	{
	case ESchemaAttributeType::Bool:
		return ESchemaServiceAttributeSupportedTypeFlags::Bool;
	case ESchemaAttributeType::Int64:
		return ESchemaServiceAttributeSupportedTypeFlags::Int64;
	case ESchemaAttributeType::Double:
		return ESchemaServiceAttributeSupportedTypeFlags::Double;
	case ESchemaAttributeType::String:
		return ESchemaServiceAttributeSupportedTypeFlags::String;
	default:
		checkNoEntry();
		// Intentional fall-through.
	case ESchemaAttributeType::None:
		return ESchemaServiceAttributeSupportedTypeFlags::None;
	}
}

ESchemaServiceAttributeFlags TranslateAttributeFlags(ESchemaAttributeFlags InFlags)
{
	ESchemaServiceAttributeFlags OutFlags = ESchemaServiceAttributeFlags::None;
	OutFlags |= EnumHasAnyFlags(InFlags, ESchemaAttributeFlags::Public) ? ESchemaServiceAttributeFlags::Public : ESchemaServiceAttributeFlags::None;
	OutFlags |= EnumHasAnyFlags(InFlags, ESchemaAttributeFlags::Private) ? ESchemaServiceAttributeFlags::Private : ESchemaServiceAttributeFlags::None;
	OutFlags |= EnumHasAnyFlags(InFlags, ESchemaAttributeFlags::Searchable) ? ESchemaServiceAttributeFlags::Searchable : ESchemaServiceAttributeFlags::None;
	return OutFlags;
}

void LogSchemaDefinition(const FSchemaDefinition& Definition)
{
	if (UE_LOG_ACTIVE(LogOnlineSchema, Verbose))
	{
		auto GetParentString = [](const FSchemaDefinition& Definition) -> FString {
			TArray<FString> ParentNames;
			Algo::Transform(Definition.ParentSchemaIds, ParentNames, [](const FSchemaId& ParentSchemaId){ return ParentSchemaId.ToString(); });
			return FString::Join(ParentNames, TEXT(","));
		};

		auto GetCategoryString = [](const FSchemaDefinition& Definition) -> FString {
			TArray<FString> CategoryNames;
			Algo::Transform(Definition.Categories, CategoryNames, [](const TPair<FSchemaCategoryId, FSchemaCategoryDefinition>& CategoryPair){ return CategoryPair.Key.ToString(); });
			return FString::Join(CategoryNames, TEXT(","));
		};

		auto GetFlagsString = [](ESchemaAttributeFlags Flags) -> FString {
			return FString::JoinBy(MakeFlagsRange(Flags), TEXT("|"), [](ESchemaAttributeFlags Flag){ return LexToString(Flag); });
		};

		const FString SchemaName = Definition.Id.ToString();
		const FString ParentSchema = GetParentString(Definition);
		UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition] SchemaName: [%ls]", *SchemaName);
		UE_LOG(LogOnlineSchema, Verbose, TEXT("[SchemaDefinition][%s] CompatabilityId: [0x%08" INT64_X_FMT "]"), *SchemaName, Definition.CompatibilityId);
		if (!ParentSchema.IsEmpty())
		{
			UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition][%ls] Parent: [%ls]", *SchemaName, *ParentSchema);
		}
		UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition][%ls] Categories: [%ls]", *SchemaName, *GetCategoryString(Definition));
		for (const TPair<FSchemaCategoryId, FSchemaCategoryDefinition>& CategoryPair : Definition.Categories)
		{
			const FSchemaCategoryDefinition& Category = CategoryPair.Value;
			const FString CategoryName = Category.Id.ToString();
			if (Category.SchemaCompatibilityAttributeId != FSchemaId())
			{
				UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition][%ls][%ls] SchemaCompatibilityAttributeId: [%ls]", *SchemaName, *CategoryName, *Category.SchemaCompatibilityAttributeId.ToString());
			}
			if (Category.SchemaCompatibilityServiceAttributeId != FSchemaId())
			{
				UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition][%ls][%ls] SchemaCompatibilityServiceAttributeId: [%ls]", *SchemaName, *CategoryName, *Category.SchemaCompatibilityServiceAttributeId.ToString());
			}

			for (const TPair<FSchemaAttributeId, FSchemaAttributeDefinition>& AttributeDefinitionPair : Category.SchemaAttributeDefinitions)
			{
				const FSchemaAttributeDefinition& AttributeDefinition = AttributeDefinitionPair.Value;
				UE_LOGF(LogOnlineSchema, Verbose, "[SchemaDefinition][%ls][%ls][%ls] Type: [%ls], Flags: [%ls], MaxSize: [%d], ServiceAttributeId: [%ls]",
				*SchemaName, *CategoryName, *AttributeDefinition.Id.ToString(), LexToString(AttributeDefinition.Type), *GetFlagsString(AttributeDefinition.Flags), AttributeDefinition.MaxSize, *AttributeDefinition.ServiceAttributeId.ToString());
			}
		}
	}
}

/* anonymous */ }

bool FSchemaRegistry::ParseConfig(const FSchemaRegistryDescriptorConfig& Config)
{
	bool ParsedSuccessfully = true;

	// Populate known service attributes
	TMap<FSchemaServiceAttributeId, const FSchemaServiceAttributeDescriptor*> KnownSchemaServiceAttributeDescriptors;
	for (const FSchemaServiceAttributeDescriptor& SchemaServiceAttributeDescriptor : Config.ServiceAttributeDescriptors)
	{
		// Check that schema descriptor id has not already been used.
		if (KnownSchemaServiceAttributeDescriptors.Contains(SchemaServiceAttributeDescriptor.Id))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema service attribute descriptor found: [%ls]", *SchemaServiceAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
			continue;
		}

		KnownSchemaServiceAttributeDescriptors.Add(SchemaServiceAttributeDescriptor.Id, &SchemaServiceAttributeDescriptor);
	}

	// Verify and populate known service descriptors.
	TMap<FSchemaId, const FSchemaServiceDescriptor*> KnownSchemaServiceDescriptors;
	for (const FSchemaServiceDescriptor& SchemaServiceDescriptor : Config.ServiceDescriptors)
	{
		// Check that schema service descriptor id has not already been used.
		if (KnownSchemaServiceDescriptors.Contains(SchemaServiceDescriptor.Id))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema service descriptor found: [%ls]", *SchemaServiceDescriptor.Id.ToString());
			ParsedSuccessfully = false;
			continue;
		}

		// Verify attributes
		TSet<FSchemaServiceAttributeId> SeenAttributes;
		for (const FSchemaServiceAttributeId& SchemaServiceAttributeId : SchemaServiceDescriptor.AttributeIds)
		{
			// Check that attribute ids are not reused.
			if (SeenAttributes.Contains(SchemaServiceAttributeId))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema service attribute %ls.%ls: Attribute id has already been used.",
					*SchemaServiceDescriptor.Id.ToString(),
					*SchemaServiceAttributeId.ToString());
				ParsedSuccessfully = false;
				continue;
			}
			SeenAttributes.Add(SchemaServiceAttributeId);

			const FSchemaServiceAttributeDescriptor** SchemaServiceAttributeDescriptorPtr = KnownSchemaServiceAttributeDescriptors.Find(SchemaServiceAttributeId);
			if (!SchemaServiceAttributeDescriptorPtr)
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema service attribute %ls.%ls: Attribute does not exist.",
					*SchemaServiceDescriptor.Id.ToString(),
					*SchemaServiceAttributeId.ToString());
				ParsedSuccessfully = false;
				continue;
			}
			const FSchemaServiceAttributeDescriptor& SchemaServiceAttributeDescriptor = **SchemaServiceAttributeDescriptorPtr;
			const ESchemaServiceAttributeSupportedTypeFlags SupportedTypes = BuildFlagsFromArray(SchemaServiceAttributeDescriptor.SupportedTypes);

			// Check that supported type has been set.
			if (SupportedTypes == ESchemaServiceAttributeSupportedTypeFlags::None)
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema service attribute %ls.%ls: A valid supported type must be selected.",
					*SchemaServiceDescriptor.Id.ToString(),
					*SchemaServiceAttributeDescriptor.Id.ToString());
				ParsedSuccessfully = false;
			}

			// Check that variable length data has a set maximum length.
			if (EnumHasAnyFlags(SupportedTypes, ESchemaServiceAttributeSupportedTypeFlags::String) && SchemaServiceAttributeDescriptor.MaxSize <= 0)
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema service attribute %ls.%ls: A valid max size must be set for variable length data.",
					*SchemaServiceDescriptor.Id.ToString(),
					*SchemaServiceAttributeDescriptor.Id.ToString());
				ParsedSuccessfully = false;
			}

			// Check for valid attribute flags
			const ESchemaServiceAttributeFlags ServiceAttributeFlags = BuildFlagsFromArray(SchemaServiceAttributeDescriptor.Flags);
			if (!EnumHasAnyFlags(ServiceAttributeFlags, ESchemaServiceAttributeFlags::Public | ESchemaServiceAttributeFlags::Private))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema service attribute %ls.%ls: Either Public or Private visibility must be set.",
					*SchemaServiceDescriptor.Id.ToString(),
					*SchemaServiceAttributeDescriptor.Id.ToString());
				ParsedSuccessfully = false;
			}
		}

		KnownSchemaServiceDescriptors.Add(SchemaServiceDescriptor.Id, &SchemaServiceDescriptor);
	}

	// Populate known schema categories
	TMap<FSchemaCategoryId, const FSchemaCategoryDescriptor*> KnownSchemaCategoryDescriptors;
	for (const FSchemaCategoryDescriptor& SchemaCategoryDescriptor : Config.SchemaCategoryDescriptors)
	{
		// Check that schema descriptor id has not already been used.
		if (KnownSchemaCategoryDescriptors.Contains(SchemaCategoryDescriptor.Id))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema category descriptor found: %ls", *SchemaCategoryDescriptor.Id.ToString());
			ParsedSuccessfully = false;
			continue;
		}
		KnownSchemaCategoryDescriptors.Add(SchemaCategoryDescriptor.Id, &SchemaCategoryDescriptor);

		// Check that schema service descriptor exists
		if (SchemaCategoryDescriptor.ServiceDescriptorId != FSchemaServiceDescriptorId()
			&& !KnownSchemaServiceDescriptors.Contains(SchemaCategoryDescriptor.ServiceDescriptorId))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema category %ls: Service descriptor id %ls not found.",
				*SchemaCategoryDescriptor.Id.ToString(),
				*SchemaCategoryDescriptor.ServiceDescriptorId.ToString());
			ParsedSuccessfully = false;
		}
	}

	// Populate known schema. Verify what can be verified statically. Hierarchy validation is performed below.
	TMap<FSchemaId, const FSchemaDescriptor*> KnownSchemaDescriptors;
	for (const FSchemaDescriptor& SchemaDescriptor : Config.SchemaDescriptors)
	{
		// Check that schema descriptor id has not already been used.
		if (KnownSchemaDescriptors.Contains(SchemaDescriptor.Id))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema descriptor found: %ls", *SchemaDescriptor.Id.ToString());
			ParsedSuccessfully = false;
			continue;
		}
		KnownSchemaDescriptors.Add(SchemaDescriptor.Id, &SchemaDescriptor);

		// Parent must declare at least one category
		if (SchemaDescriptor.ParentId == FSchemaId() && SchemaDescriptor.CategoryIds.IsEmpty())
		{
			// Error, base type must define at least one category
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema %ls: Base schema must define at least one category.",
				*SchemaDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}

		// Children must not declare categories
		if (SchemaDescriptor.ParentId != FSchemaId() && !SchemaDescriptor.CategoryIds.IsEmpty())
		{
			// Error child cannot define categories
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema %ls: Child schema cannot define categories.",
				*SchemaDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}

		TSet<FSchemaCategoryId> SeenCategoryIds;
		for (const FSchemaCategoryId& CategoryId : SchemaDescriptor.CategoryIds)
		{
			if (SeenCategoryIds.Contains(CategoryId))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema category %ls.%ls: Category id has already been used.",
					*SchemaDescriptor.Id.ToString(),
					*CategoryId.ToString());
				ParsedSuccessfully = false;
				continue;
			}
			SeenCategoryIds.Emplace(CategoryId);

			if (!KnownSchemaCategoryDescriptors.Contains(CategoryId))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema category %ls.%ls: Category does not exist.",
					*SchemaDescriptor.Id.ToString(),
					*CategoryId.ToString());
				ParsedSuccessfully = false;
			}
		}
	}

	// Populate known schema attributes
	TMap<FSchemaAttributeId, const FSchemaAttributeDescriptor*> KnownSchemaAttributeDescriptors;
	for (const FSchemaAttributeDescriptor& SchemaAttributeDescriptor : Config.SchemaAttributeDescriptors)
	{
		// Check that schema descriptor id has not already been used.
		if (KnownSchemaAttributeDescriptors.Contains(SchemaAttributeDescriptor.Id))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema attribute descriptor found: %ls", *SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
			continue;
		}
		KnownSchemaAttributeDescriptors.Emplace(SchemaAttributeDescriptor.Id, &SchemaAttributeDescriptor);
		
		// Check that a type has been set
		if (SchemaAttributeDescriptor.Type == ESchemaAttributeType::None)
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: A valid type must be set.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}

		// Check that variable length data has a set maximum length.
		if (SchemaAttributeDescriptor.Type == ESchemaAttributeType::String && SchemaAttributeDescriptor.MaxSize <= 0)
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: A valid max size must be set for variable length data.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}

		// Check that group id is valid.
		if (SchemaAttributeDescriptor.UpdateGroupId < 0)
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: UpdateGroupId must be set to either 0 or a positive integer value.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}
		
		const ESchemaAttributeFlags AttributeFlags = BuildFlagsFromArray(SchemaAttributeDescriptor.Flags);

		if (EnumHasAllFlags(AttributeFlags, ESchemaAttributeFlags::SchemaCompatibilityId))
		{
			if (SchemaAttributeDescriptor.Type != ESchemaAttributeType::Int64)
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: SchemaCompatibilityId field may only be set to Int64.",
					*SchemaAttributeDescriptor.Id.ToString());
				ParsedSuccessfully = false;
			}
		
			if (EnumHasAnyFlags(AttributeFlags, ESchemaAttributeFlags::Private))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: SchemaCompatibilityId attribute may not be private.",
					*SchemaAttributeDescriptor.Id.ToString());
				ParsedSuccessfully = false;
			}
		}
		
		// Check that public or private visibility has been set and is valid.
		if (EnumHasAllFlags(AttributeFlags, ESchemaAttributeFlags::Public | ESchemaAttributeFlags::Private))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: Public and Private visibility are mutually exclusive.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}
		if (!EnumHasAnyFlags(AttributeFlags, ESchemaAttributeFlags::Public | ESchemaAttributeFlags::Private))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: Either Public or Private visibility must be set.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}
		if (EnumHasAllFlags(AttributeFlags, ESchemaAttributeFlags::Searchable | ESchemaAttributeFlags::Private))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls: Searchable attributes may not be private.",
				*SchemaAttributeDescriptor.Id.ToString());
			ParsedSuccessfully = false;
		}
	}

	// Populate known schema category attributes 
	using FCategoryToAttributesMap = TMap<FSchemaCategoryId, TArray<FSchemaAttributeId>>;
	using FSchemaToCategoryAttributesMap = TMap<FSchemaId, FCategoryToAttributesMap>;
	FSchemaToCategoryAttributesMap KnownSchemaCategoryAttributes;
	for (const FSchemaCategoryAttributesDescriptor& SchemaCategoryAttributesDescriptor : Config.SchemaCategoryAttributeDescriptors)
	{
		// Check that the id has not already been used.
		FCategoryToAttributesMap* FoundSchema = KnownSchemaCategoryAttributes.Find(SchemaCategoryAttributesDescriptor.SchemaId);
		if (FoundSchema && FoundSchema->Contains(SchemaCategoryAttributesDescriptor.CategoryId))
		{
			UE_LOGF(LogOnlineSchema, Error, "Duplicate schema additional category attributes descriptor found: %ls.%ls",
				*SchemaCategoryAttributesDescriptor.SchemaId.ToString(),
				*SchemaCategoryAttributesDescriptor.CategoryId.ToString());
			ParsedSuccessfully = false;
			continue;
		}

		// Check that the schema exists
		if (!KnownSchemaDescriptors.Contains(SchemaCategoryAttributesDescriptor.SchemaId))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid additional category attributes %ls.%ls: Schema does not exist",
				*SchemaCategoryAttributesDescriptor.SchemaId.ToString(),
				*SchemaCategoryAttributesDescriptor.CategoryId.ToString());
			ParsedSuccessfully = false;
			continue;
		}

		// Check that the category exists
		if (!KnownSchemaCategoryDescriptors.Contains(SchemaCategoryAttributesDescriptor.CategoryId))
		{
			UE_LOGF(LogOnlineSchema, Error, "Invalid additional category attributes %ls.%ls: Category does not exist",
				*SchemaCategoryAttributesDescriptor.SchemaId.ToString(),
				*SchemaCategoryAttributesDescriptor.CategoryId.ToString());
			ParsedSuccessfully = false;
			continue;
		}

		bool bAttributesParsedSuccessfully = true;
		TSet<FSchemaAttributeId> SeenAttributeIds;
		for (const FSchemaAttributeId& AttributeId : SchemaCategoryAttributesDescriptor.AttributeIds)
		{
			if (SeenAttributeIds.Contains(AttributeId))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid category attributes %ls.%ls: Attribute %ls duplicated",
					*SchemaCategoryAttributesDescriptor.SchemaId.ToString(),
					*SchemaCategoryAttributesDescriptor.CategoryId.ToString(),
					*AttributeId.ToString());
				bAttributesParsedSuccessfully = false;
				continue;
			}
			SeenAttributeIds.Emplace(AttributeId);

			if (!KnownSchemaAttributeDescriptors.Contains(AttributeId))
			{
				UE_LOGF(LogOnlineSchema, Error, "Invalid category attributes %ls.%ls: Attribute %ls does not exist",
					*SchemaCategoryAttributesDescriptor.SchemaId.ToString(),
					*SchemaCategoryAttributesDescriptor.CategoryId.ToString(),
					*AttributeId.ToString());
				bAttributesParsedSuccessfully = false;
				continue;
			}
		}
		if (!bAttributesParsedSuccessfully)
		{
			ParsedSuccessfully = false;
			continue;
		}

		if (!FoundSchema)
		{
			FoundSchema = &KnownSchemaCategoryAttributes.Emplace(SchemaCategoryAttributesDescriptor.SchemaId);
		}
		FoundSchema->Emplace(SchemaCategoryAttributesDescriptor.CategoryId, SchemaCategoryAttributesDescriptor.AttributeIds);
	}

	// Verify schema data
	for (const TPair<FSchemaId, const FSchemaDescriptor*>& SchemaDescriptorPair : KnownSchemaDescriptors)
	{
		const FSchemaDescriptor& SchemaDescriptorToVerify = *SchemaDescriptorPair.Value;

		// Stack of schemas, will push in tail->root order and then reverse to root->tail order.
		TArray<FSchemaId> SchemaHierarchy;

		// Verify hierarchy.
		const FSchemaDescriptor* TestSchemaDescriptor = &SchemaDescriptorToVerify;
		while (TestSchemaDescriptor)
		{
			// Check for circular dependencies in schema.
			if (SchemaHierarchy.Contains(TestSchemaDescriptor->Id))
			{
				UE_LOGF(LogOnlineSchema, Error, "Circular parent dependency found in schema: %ls", *SchemaDescriptorToVerify.Id.ToString());
				ParsedSuccessfully = false;
				break;
			}
			SchemaHierarchy.Push(TestSchemaDescriptor->Id);

			if (TestSchemaDescriptor->ParentId != FSchemaId())
			{
				// Move onto next layer in the hierarchy
				if (const FSchemaDescriptor** ParentDescriptor = KnownSchemaDescriptors.Find(TestSchemaDescriptor->ParentId))
				{
					TestSchemaDescriptor = *ParentDescriptor;
				}
				else
				{
					UE_LOGF(LogOnlineSchema, Error, "Invalid schema %ls: Parent schema %ls does not exist.",
						*SchemaDescriptorToVerify.Id.ToString(),
						*TestSchemaDescriptor->ParentId.ToString());
					ParsedSuccessfully = false;
					break;
				}
			}
			else
			{
				TestSchemaDescriptor = nullptr;
			}
		}

		// Reverse the stack so it's in root->tail order
		Algo::Reverse(SchemaHierarchy);
		check(SchemaHierarchy.Num() > 0);
		const FSchemaId RootSchemaId = SchemaHierarchy[0];
		const FSchemaDescriptor& RootSchemaDescriptor = *KnownSchemaDescriptors.FindChecked(RootSchemaId);

		// Verify categories
		TSet<FSchemaServiceDescriptorId> SeenSchemaServiceDescriptorIds;
		for (const FSchemaCategoryId& SchemaCategoryId : RootSchemaDescriptor.CategoryIds)
		{
			const FSchemaCategoryDescriptor** SchemaCategoryDescriptorPtr = KnownSchemaCategoryDescriptors.Find(SchemaCategoryId);
			if (!SchemaCategoryDescriptorPtr)
			{
				// Already reported in schema population checks above.
				continue;
			}
			const FSchemaCategoryDescriptor& SchemaCategoryDescriptor = **SchemaCategoryDescriptorPtr;

			// Verify that service descriptor is valid.
			if (SchemaCategoryDescriptor.ServiceDescriptorId != FSchemaServiceDescriptorId())
			{
				// Verify that the service descriptor is not reused between categories.
				if (SeenSchemaServiceDescriptorIds.Find(SchemaCategoryDescriptor.ServiceDescriptorId) != nullptr)
				{
					UE_LOGF(LogOnlineSchema, Error, "Invalid schema category %ls.%ls: Service descriptor id %ls has already been used by another category.",
						*SchemaDescriptorToVerify.Id.ToString(),
						*SchemaCategoryDescriptor.Id.ToString(),
						*SchemaCategoryDescriptor.ServiceDescriptorId.ToString());
					ParsedSuccessfully = false;
				}

				SeenSchemaServiceDescriptorIds.Add(SchemaCategoryDescriptor.ServiceDescriptorId);
			}

			// Verify category attributes.
			TSet<FSchemaServiceAttributeId> SeenAttributes;
			FSchemaAttributeId SeenSchemaCompatibilityAttributeId;
			TSet<FSchemaAttributeId> SchemaCategoryAttributeIds;
			// Iterate down the stack, checking attributes as we go
			for (const FSchemaId SchemaHierarchyLayerId : SchemaHierarchy)
			{
				const FCategoryToAttributesMap* FoundSchemaCategories = KnownSchemaCategoryAttributes.Find(SchemaHierarchyLayerId);
				const TArray<FSchemaAttributeId>* FoundCategoryAttributes = FoundSchemaCategories ? FoundSchemaCategories->Find(SchemaCategoryId) : nullptr;
				if (!FoundCategoryAttributes)
				{
					// This layer doesn't add any attribs for this category.
					continue;
				}

				for (const FSchemaAttributeId& SchemaAttributeId : *FoundCategoryAttributes)
				{
					// Check that attribute ids are not reused.
					if (SeenAttributes.Find(SchemaAttributeId))
					{
						UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls.%ls.%ls: Attribute id has already been used.",
							*SchemaDescriptorToVerify.Id.ToString(),
							*SchemaCategoryDescriptor.Id.ToString(),
							*SchemaAttributeId.ToString());
						ParsedSuccessfully = false;
						continue;
					}
					SeenAttributes.Add(SchemaAttributeId);

					const FSchemaAttributeDescriptor** SchemaAttributeDescriptorPtr = KnownSchemaAttributeDescriptors.Find(SchemaAttributeId);
					if (!SchemaAttributeDescriptorPtr)
					{
						// Already reported in the category attribute population checks above.
						continue;
					}
					const FSchemaAttributeDescriptor& SchemaAttributeDescriptor = **SchemaAttributeDescriptorPtr;
					const ESchemaAttributeFlags AttributeFlags = BuildFlagsFromArray(SchemaAttributeDescriptor.Flags);

					// Check that searchable attributes only exist in a base schema.
					if (EnumHasAllFlags(AttributeFlags, ESchemaAttributeFlags::Searchable) && SchemaHierarchyLayerId != RootSchemaId)
					{
						UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls.%ls.%ls: Searchable fields may only exist in the base schema.",
							*SchemaDescriptorToVerify.Id.ToString(),
							*SchemaCategoryDescriptor.Id.ToString(),
							*SchemaAttributeDescriptor.Id.ToString());
						ParsedSuccessfully = false;
					}

					// Check that schema compatibility attribute only exists in a base schema and used only once and that it is the Int64 type.
					if (EnumHasAllFlags(AttributeFlags, ESchemaAttributeFlags::SchemaCompatibilityId))
					{
						if (SchemaHierarchyLayerId != RootSchemaId)
						{
							UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls.%ls.%ls: SchemaCompatibilityId field may only exist in the base schema.",
								*SchemaDescriptorToVerify.Id.ToString(),
								*SchemaCategoryDescriptor.Id.ToString(),
								*SchemaAttributeDescriptor.Id.ToString());
							ParsedSuccessfully = false;
						}

						if (SeenSchemaCompatibilityAttributeId != FSchemaId())
						{
							UE_LOGF(LogOnlineSchema, Error, "Invalid schema attribute %ls.%ls.%ls: SchemaCompatibilityId field has already been used by attribute %ls.",
								*SchemaDescriptorToVerify.Id.ToString(),
								*SchemaCategoryDescriptor.Id.ToString(),
								*SchemaAttributeDescriptor.Id.ToString(),
								*SeenSchemaCompatibilityAttributeId.ToString());
							ParsedSuccessfully = false;
						}
						else
						{
							SeenSchemaCompatibilityAttributeId = SchemaAttributeDescriptor.Id;
						}
					}
				}
			}
		}
	}

	// Build definitions only when there are no parsing errors.
	TMap<FSchemaId, TSharedRef<const FSchemaDefinition>> ParsedSchemaDefinitions;
	TMap<int64, TSharedRef<const FSchemaDefinition>> ParsedSchemaDefinitionsByCompatibilityId;
	if (ParsedSuccessfully)
	{
		// Build schema definitions
		TSet<uint32> SeenSchemaCrcs;
		for (const TPair<FSchemaId, const FSchemaDescriptor*>& SchemaDescriptorPair : KnownSchemaDescriptors)
		{
			uint32 SchemaDataCrc = 0;
			const FSchemaDescriptor& SchemaDescriptor = *SchemaDescriptorPair.Get<1>();

			TSharedRef<FSchemaDefinition> SchemaDefinition = MakeShared<FSchemaDefinition>();
			ParsedSchemaDefinitions.Add(SchemaDescriptor.Id, SchemaDefinition);
			SchemaDefinition->Id = SchemaDescriptor.Id;

			// Get the hierarchy from root->leaf
			TArray<FSchemaId> SchemaHierarchy;
			SchemaHierarchy.Emplace(SchemaDefinition->Id);
			for (const FSchemaDescriptor** IterSchemaDescriptor = KnownSchemaDescriptors.Find(SchemaDescriptor.ParentId);
				IterSchemaDescriptor != nullptr;
				IterSchemaDescriptor = KnownSchemaDescriptors.Find((*IterSchemaDescriptor)->ParentId))
			{
				const FSchemaId& IterSchemaId = (*IterSchemaDescriptor)->Id;
				SchemaHierarchy.Emplace(IterSchemaId);

				// Populate parent info.
				SchemaDefinition->ParentSchemaIds.Add(IterSchemaId);
				SchemaDataCrc = SchemaNameCrc32(IterSchemaId, SchemaDataCrc);
			}
			Algo::Reverse(SchemaHierarchy);
			check(SchemaHierarchy.Num() > 0);
			const FSchemaId RootSchemaId = SchemaHierarchy[0];
			const FSchemaDescriptor& RootSchemaDescriptor = *KnownSchemaDescriptors.FindChecked(RootSchemaId);

			// Add the category definitions.
			for (const FSchemaCategoryId& CategoryId : RootSchemaDescriptor.CategoryIds)
			{
				FSchemaCategoryDefinition& SchemaCategoryDefinition = SchemaDefinition->Categories.Emplace(CategoryId);
				SchemaCategoryDefinition.Id = CategoryId;

				SchemaDataCrc = SchemaNameCrc32(CategoryId, SchemaDataCrc);
			}

			// Iterate the hierarchy from root -> leaf
			for (const FSchemaId& SchemaHierarchyLayerId : SchemaHierarchy)
			{
				const FSchemaDescriptor& CurrentSchemaDescriptor = *KnownSchemaDescriptors.FindChecked(SchemaHierarchyLayerId);

				// Populate attribute definitions for each category
				for (const FSchemaCategoryId& CurrentSchemaCategoryId : RootSchemaDescriptor.CategoryIds)
				{
					if (const FCategoryToAttributesMap* FoundSchemaCategoryAttributes = KnownSchemaCategoryAttributes.Find(CurrentSchemaDescriptor.Id))
					{
						if (const TArray<FSchemaAttributeId>* FoundCategoryAttributeIds = FoundSchemaCategoryAttributes->Find(CurrentSchemaCategoryId))
						{
							const FSchemaCategoryDescriptor& CurrentSchemaCategoryDescriptor = *KnownSchemaCategoryDescriptors.FindChecked(CurrentSchemaCategoryId);
							FSchemaCategoryDefinition& SchemaCategoryDefinition = SchemaDefinition->Categories.FindChecked(CurrentSchemaCategoryDescriptor.Id);

							for (const FSchemaAttributeId& CurrentSchemaAttributeId : *FoundCategoryAttributeIds)
							{
								check(!SchemaCategoryDefinition.SchemaAttributeDefinitions.Contains(CurrentSchemaAttributeId));

								const FSchemaAttributeDescriptor& CurrentSchemaAttributeDescriptor = *KnownSchemaAttributeDescriptors.FindChecked(CurrentSchemaAttributeId);

								FSchemaAttributeDefinition& AttributeDefinition = SchemaCategoryDefinition.SchemaAttributeDefinitions.Emplace(CurrentSchemaAttributeDescriptor.Id);
								AttributeDefinition.Id = CurrentSchemaAttributeDescriptor.Id;
								AttributeDefinition.Flags = BuildFlagsFromArray(CurrentSchemaAttributeDescriptor.Flags);
								AttributeDefinition.Type = CurrentSchemaAttributeDescriptor.Type;
								AttributeDefinition.MaxSize = CurrentSchemaAttributeDescriptor.MaxSize;

								if (EnumHasAnyFlags(AttributeDefinition.Flags, ESchemaAttributeFlags::SchemaCompatibilityId))
								{
									SchemaCategoryDefinition.SchemaCompatibilityAttributeId = AttributeDefinition.Id;
								}

								SchemaDataCrc = SchemaNameCrc32(AttributeDefinition.Id, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(AttributeDefinition.Flags, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(AttributeDefinition.Type, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(AttributeDefinition.MaxSize, SchemaDataCrc);
							}
						}
					}
				}
			}

			// Assign attributes to service attributes for each category
			for (const FSchemaCategoryId& CategoryId : RootSchemaDescriptor.CategoryIds)
			{
				FSchemaCategoryDefinition& SchemaCategoryDefinition = SchemaDefinition->Categories.FindChecked(CategoryId);

				const FSchemaCategoryDescriptor& SchemaCategoryDescriptor = *KnownSchemaCategoryDescriptors.FindChecked(CategoryId);
				if (SchemaCategoryDescriptor.ServiceDescriptorId != FSchemaServiceDescriptorId())
				{
					const FSchemaServiceDescriptor& SchemaServiceDescriptor = *KnownSchemaServiceDescriptors.FindChecked(SchemaCategoryDescriptor.ServiceDescriptorId);

					// todo: pack multiple attributes into a service attribute.
					// For the time being schema attributes and service attributes have a 1:1 relationship.

					// Bucket service attributes for assignment
					TMap<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags> PublicAttributes;
					TMap<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags> PrivateAttributes;
					TMap<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags> SearchableAttributes;

					// Build out buckets.
					for (const FSchemaServiceAttributeId& SchemaServiceAttributeId : SchemaServiceDescriptor.AttributeIds)
					{
						const FSchemaServiceAttributeDescriptor& SchemaServiceAttributeDescriptor = *KnownSchemaServiceAttributeDescriptors.FindChecked(SchemaServiceAttributeId);
						const ESchemaServiceAttributeSupportedTypeFlags SupportedTypes = BuildFlagsFromArray(SchemaServiceAttributeDescriptor.SupportedTypes);
						const ESchemaServiceAttributeFlags ServiceAttributeFlags = BuildFlagsFromArray(SchemaServiceAttributeDescriptor.Flags);
						if (EnumHasAnyFlags(ServiceAttributeFlags, ESchemaServiceAttributeFlags::Public))
						{
							PublicAttributes.Add(&SchemaServiceAttributeDescriptor, SupportedTypes);
						}
						if (EnumHasAnyFlags(ServiceAttributeFlags, ESchemaServiceAttributeFlags::Private))
						{
							PrivateAttributes.Add(&SchemaServiceAttributeDescriptor, SupportedTypes);
						}
						if (EnumHasAnyFlags(ServiceAttributeFlags, ESchemaServiceAttributeFlags::Searchable))
						{
							SearchableAttributes.Add(&SchemaServiceAttributeDescriptor, SupportedTypes);
						}
					}

					// Assign schema attributes based on buckets.
					for (const FSchemaId& SchemaHierarchyLayerId : SchemaHierarchy)
					{
						const FCategoryToAttributesMap* FoundSchemaCategoryAttributes = KnownSchemaCategoryAttributes.Find(SchemaHierarchyLayerId);
						const TArray<FSchemaAttributeId>* FoundCategoryAttributeIds = FoundSchemaCategoryAttributes ? FoundSchemaCategoryAttributes->Find(CategoryId) : nullptr;
						if (!FoundCategoryAttributeIds)
						{
							continue;
						}

						for (const FSchemaAttributeId& SchemaAttributeId : *FoundCategoryAttributeIds)
						{
							FSchemaAttributeDefinition& SchemaAttributeDefinition = SchemaCategoryDefinition.SchemaAttributeDefinitions.FindChecked(SchemaAttributeId);

							// Find a service attribute which can handle the schema attribute.
							TMap<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags>* Bucket = nullptr;
							if (EnumHasAnyFlags(SchemaAttributeDefinition.Flags, ESchemaAttributeFlags::Public))
							{
								Bucket = EnumHasAnyFlags(SchemaAttributeDefinition.Flags, ESchemaAttributeFlags::Searchable) ? &SearchableAttributes : &PublicAttributes;
							}
							else
							{
								Bucket = &PrivateAttributes;
							}

							const TPair<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags>* SchemaServiceAttributeDescriptorPair = Algo::FindByPredicate(*Bucket,
								[SchemaAttributeDefinition](const TPair<const FSchemaServiceAttributeDescriptor*, ESchemaServiceAttributeSupportedTypeFlags>& ServiceAttributePair)
							{
								const FSchemaServiceAttributeDescriptor* SchemaServiceAttributeDescriptor = ServiceAttributePair.Get<0>();
								const ESchemaServiceAttributeSupportedTypeFlags SchemaServiceAttributeSupportedTypeFlags = ServiceAttributePair.Get<1>();
								return EnumHasAnyFlags(SchemaServiceAttributeSupportedTypeFlags, TranslateAttributeType(SchemaAttributeDefinition.Type)) &&
									SchemaAttributeDefinition.MaxSize <= SchemaServiceAttributeDescriptor->MaxSize;
							});
							const FSchemaServiceAttributeDescriptor* FoundServiceAttributeDescriptor = SchemaServiceAttributeDescriptorPair ? SchemaServiceAttributeDescriptorPair->Get<0>() : nullptr;

							if (FoundServiceAttributeDescriptor)
							{
								// Todo: Reuse descriptor for assignment until it is full.

								// Remove found descriptor from buckets.
								// Stable remove to preserve ordering with compact tset enabled.
								// Automated tests (eg System.Engine.Online.ClientLobbyDataTest) depend on this order.
								PublicAttributes.RemoveStable(FoundServiceAttributeDescriptor);
								PrivateAttributes.RemoveStable(FoundServiceAttributeDescriptor);
								SearchableAttributes.RemoveStable(FoundServiceAttributeDescriptor);

								// Create service attribute definition.
								FSchemaServiceAttributeDefinition& SchemaServiceAttributeDefinition = SchemaCategoryDefinition.ServiceAttributeDefinitions.Add(FoundServiceAttributeDescriptor->Id);
								SchemaServiceAttributeDefinition.Id = FoundServiceAttributeDescriptor->Id;
								SchemaServiceAttributeDefinition.Type = TranslateAttributeType(SchemaAttributeDefinition.Type);
								SchemaServiceAttributeDefinition.Flags = TranslateAttributeFlags(SchemaAttributeDefinition.Flags);
								SchemaServiceAttributeDefinition.MaxSize = FoundServiceAttributeDescriptor->MaxSize;
								SchemaServiceAttributeDefinition.SchemaAttributeIds.Add(SchemaAttributeDefinition.Id);
								SchemaAttributeDefinition.ServiceAttributeId = SchemaServiceAttributeDefinition.Id;

								if (EnumHasAnyFlags(SchemaAttributeDefinition.Flags, ESchemaAttributeFlags::SchemaCompatibilityId))
								{
									SchemaCategoryDefinition.SchemaCompatibilityServiceAttributeId = SchemaServiceAttributeDefinition.Id;
								}

								// Add service attribute definition to data crc.
								SchemaDataCrc = SchemaNameCrc32(SchemaServiceAttributeDefinition.Id, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(SchemaServiceAttributeDefinition.Type, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(SchemaServiceAttributeDefinition.Flags, SchemaDataCrc);
								SchemaDataCrc = FCrc::TypeCrc32(SchemaServiceAttributeDefinition.MaxSize, SchemaDataCrc);

								Algo::ForEach(SchemaServiceAttributeDefinition.SchemaAttributeIds, [&SchemaDataCrc](const FSchemaAttributeId& Id) { SchemaDataCrc = SchemaNameCrc32(Id, SchemaDataCrc); });
							}
							else
							{
								UE_LOGF(LogOnlineSchema, Error, "Failed to find service attribute to fit schema attribute %ls.%ls.%ls.",
									*SchemaDefinition->Id.ToString(),
									*SchemaCategoryDefinition.Id.ToString(),
									*SchemaAttributeDefinition.Id.ToString());
								ParsedSuccessfully = false;
							}
						}
					}

					SchemaDataCrc = SchemaNameCrc32(SchemaCategoryDefinition.SchemaCompatibilityAttributeId, SchemaDataCrc);
					SchemaDataCrc = SchemaNameCrc32(SchemaCategoryDefinition.SchemaCompatibilityServiceAttributeId, SchemaDataCrc);
				}
			}

			// Build compatibility id from the schema and data CRCs.
			{
				const uint32 SchemaIdCrc = SchemaNameCrc32(SchemaDefinition->Id);
				if (SeenSchemaCrcs.Find(SchemaIdCrc) != nullptr)
				{
					UE_LOGF(LogOnlineSchema, Error, "Invalid schema %ls: CRC collision processing schema name, please rename the schema to avoid collision.",
						*SchemaDefinition->Id.ToString());
					ParsedSuccessfully = false;
				}
				SeenSchemaCrcs.Add(SchemaIdCrc);

				const uint64 CompatibilityIdUint = (static_cast<uint64>(SchemaIdCrc) << 32) | SchemaDataCrc;
				SchemaDefinition->CompatibilityId = *reinterpret_cast<const int64*>(&CompatibilityIdUint);

				UE_LOGF(LogOnlineSchema, Verbose, "Building compatability id: SchemaId: [%ls], SchemaIdCrc: [0x%04X], SchemaDataCrc [0x%04X].",
					*SchemaDefinition->Id.ToString(), SchemaIdCrc, SchemaDataCrc);
			}

			ParsedSchemaDefinitionsByCompatibilityId.Add(SchemaDefinition->CompatibilityId, SchemaDefinition);

			UE_LOG(LogOnlineSchema, Log, TEXT("Parsed Schema definition: SchemaId: [%s], SchemaCompatabilityId: [0x%08" INT64_X_FMT "]"),
				*SchemaDefinition->Id.ToString(), SchemaDefinition->CompatibilityId);

			LogSchemaDefinition(*SchemaDefinition);
		}
	}

	if (ParsedSuccessfully)
	{
		SchemaDefinitionsById = MoveTemp(ParsedSchemaDefinitions);
		SchemaDefinitionsByCompatibilityId = MoveTemp(ParsedSchemaDefinitionsByCompatibilityId);
		return true;
	}
	else
	{
		return false;
	}
}

namespace Private {

FSchemaCategoryInstanceBase::FSchemaCategoryInstanceBase(
	const FSchemaId& DerivedSchemaId,
	const FSchemaId& BaseSchemaId,
	const FSchemaCategoryId& CategoryId,
	const TSharedRef<const FSchemaRegistry>& SchemaRegistry)
	: SchemaRegistry(SchemaRegistry)
{
	bool bInitSuccess = true;

	if (InitializeSchemaDefinition(BaseSchemaId, nullptr, CategoryId, &BaseSchemaDefinition, &BaseSchemaCategoryDefinition))
	{
		if (DerivedSchemaId == FSchemaId())
		{
			// Derived schema id is unknown. One of the below is true:
			// 1. All attributes are defined in the base schema.
			// 2. An attribute in the base schema is flagged as SchemaCompatibilityId to allow
			//    detection of the derived schema based on incoming service attribute data.
		}
		else
		{
			if (!InitializeSchemaDefinition(DerivedSchemaId, BaseSchemaDefinition.Get(), CategoryId, &DerivedSchemaDefinition, &DerivedSchemaCategoryDefinition))
			{
				// Unable to find derived schema when it was expected to be valid.
				bInitSuccess = false;
			}
		}
	}

	if (!bInitSuccess)
	{
		UE_LOGF(LogOnlineSchema, Error, "[FSchemaCategoryInstance] init failed. DerivedSchema[%ls], BaseSchema[%ls], CategoryId[%ls]",
			*DerivedSchemaId.ToString(),
			*BaseSchemaId.ToString(),
			*CategoryId.ToString());

		// Invalidate schema to fail initialization.
		BaseSchemaDefinition.Reset();
		BaseSchemaCategoryDefinition = nullptr;
		DerivedSchemaDefinition.Reset();
		DerivedSchemaCategoryDefinition = nullptr;
	}
}

TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> FSchemaCategoryInstanceBase::PrepareClientChanges(FSchemaCategoryInstancePrepareClientChanges::Params&& Params) const
{
	// todo: handle packing multiple attributes into a service attribute.

	// Start by clearing any previous prepared state.
	ResetPreparedChanges();

	const FSchemaDefinition* SchemaDefinition = nullptr;
	const FSchemaCategoryDefinition* SchemaCategoryDefinition = nullptr;
	if (!GetSerializationSchema(&SchemaDefinition, &SchemaCategoryDefinition))
	{
		UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] Serialization schema category definition not found.");
		return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidState());
	}

	FSchemaClientServiceChanges ServiceChanges;
	FPreparedClientChanges NewPreparedClientChanges;

	// Handle schema swap.
	if (Params.ClientChanges.SchemaId)
	{
		if (InitializeSchemaDefinition(
			*Params.ClientChanges.SchemaId,
			BaseSchemaDefinition.Get(),
			BaseSchemaCategoryDefinition->Id,
			&NewPreparedClientChanges.DerivedSchemaDefinition,
			&NewPreparedClientChanges.DerivedSchemaCategoryDefinition))
		{
			if (DerivedSchemaDefinition)
			{
				// todo: handle schema change when previously set.
				return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::NotImplemented());
			}
			else
			{
				// Derived schema init succeeded. Switch to using it for preparing the service changes.
				SchemaDefinition = NewPreparedClientChanges.DerivedSchemaDefinition.Get();
				SchemaCategoryDefinition = NewPreparedClientChanges.DerivedSchemaCategoryDefinition;

				NewPreparedClientChanges.ClientChanges.SchemaId.Emplace(SchemaDefinition->Id);
			}
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] Schema change failed: NewDerivedSchema[%ls], BaseSchema[%ls]",
				*Params.ClientChanges.SchemaId->ToString(),
				*BaseSchemaCategoryDefinition->Id.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}
	}

	// Translate mutated attributes.
	for (TPair<FSchemaAttributeId, FSchemaVariant>& UpdatedAttributeData : Params.ClientChanges.UpdatedAttributes)
	{
		const FSchemaAttributeId& UpdatedAttributeId = UpdatedAttributeData.Key;
		FSchemaVariant& UpdatedAttributeValue = UpdatedAttributeData.Value;

		const FSchemaAttributeDefinition* AttributeDefinition = SchemaCategoryDefinition->SchemaAttributeDefinitions.Find(UpdatedAttributeId);
		if (AttributeDefinition == nullptr)
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Attribute definition does not exist in schema category.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*UpdatedAttributeId.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}

		// Schema compatibility id cannot be modified by client code.
		if (EnumHasAnyFlags(AttributeDefinition->Flags, ESchemaAttributeFlags::SchemaCompatibilityId))
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: It is not valid to set SchemaCompatibilityId in an attribute update",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*AttributeDefinition->Id.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}

		// Check that type matches.
		if (AttributeDefinition->Type != UpdatedAttributeValue.GetType())
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Attribute type %ls does not match the expected type %ls.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*AttributeDefinition->Id.ToString(),
				LexToString(UpdatedAttributeValue.GetType()),
				LexToString(AttributeDefinition->Type));
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}

		// Check that variable sized data fits within the service attribute.
		if (AttributeDefinition->Type == ESchemaAttributeType::String && UpdatedAttributeValue.GetString().Len() > AttributeDefinition->MaxSize)
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Variably sized attribute exceeds maximum defined length. %d > %d.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*AttributeDefinition->Id.ToString(),
				UpdatedAttributeValue.GetString().Len(),
				AttributeDefinition->MaxSize);
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}

		// Compare attribute to previously known client data.
		if (const FSchemaVariant* PreviousValue = GetClientSnapshot().Find(UpdatedAttributeId))
		{
			// Ignore change to the same value.
			if (*PreviousValue != UpdatedAttributeValue)
			{
				// Store change to client data.
				NewPreparedClientChanges.ClientChanges.ChangedAttributes.Add(UpdatedAttributeId, { *PreviousValue, UpdatedAttributeValue });
			}
		}
		else
		{
			// Store change to client data.
			NewPreparedClientChanges.ClientChanges.AddedAttributes.Add(UpdatedAttributeId, UpdatedAttributeValue);
		}

		if (AttributeDefinition->ServiceAttributeId != FSchemaServiceAttributeId())
		{
			const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = SchemaCategoryDefinition->ServiceAttributeDefinitions.Find(AttributeDefinition->ServiceAttributeId);
			if (ensure(ServiceAttributeDefinition))
			{
				// Add modified attribute to output.
				ServiceChanges.UpdatedAttributes.Add(AttributeDefinition->ServiceAttributeId, { AttributeDefinition->ServiceAttributeId, ServiceAttributeDefinition->Flags, MoveTemp(UpdatedAttributeValue) });
			}
			else
			{
				UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Unable to find service attribute definition for service attribute %ls.",
					*SchemaDefinition->Id.ToString(),
					*SchemaCategoryDefinition->Id.ToString(),
					*AttributeDefinition->Id.ToString(),
					*AttributeDefinition->ServiceAttributeId.ToString());
				return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidState());
			}
		}
	}

	// Translate removed attributes.
	for (FSchemaAttributeId& RemovedAttributeId : Params.ClientChanges.RemovedAttributes)
	{
		if (const FSchemaAttributeDefinition* AttributeDefinition = SchemaCategoryDefinition->SchemaAttributeDefinitions.Find(RemovedAttributeId))
		{
			// Schema compatibility id cannot be modified by client code.
			if (EnumHasAnyFlags(AttributeDefinition->Flags, ESchemaAttributeFlags::SchemaCompatibilityId))
			{
				UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: It is not valid to set SchemaCompatibilityId in an attribute update",
					*SchemaDefinition->Id.ToString(),
					*SchemaCategoryDefinition->Id.ToString(),
					*AttributeDefinition->Id.ToString());
				return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
			}
			else
			{
				// Store change to client data.
				NewPreparedClientChanges.ClientChanges.RemovedAttributes.Add(RemovedAttributeId);

				if (AttributeDefinition->ServiceAttributeId != FSchemaServiceAttributeId())
				{
					// Add removed attribute to service output.
					ServiceChanges.RemovedAttributes.Add(AttributeDefinition->ServiceAttributeId);
				}
			}
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Attribute definition does not exist in schema category.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*RemovedAttributeId.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidParams());
		}
	}

	// Check if SchemaCompatibilityId needs to be set.
	if (SchemaCategoryDefinition->SchemaCompatibilityAttributeId != FSchemaAttributeId() && LastSentSchemaCompatibilityId == 0)
	{
		const FSchemaAttributeDefinition* AttributeDefinition = SchemaCategoryDefinition->SchemaAttributeDefinitions.Find(SchemaCategoryDefinition->SchemaCompatibilityAttributeId);
		if (ensure(AttributeDefinition))
		{
			if (AttributeDefinition->ServiceAttributeId != FSchemaServiceAttributeId())
			{
				const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = SchemaCategoryDefinition->ServiceAttributeDefinitions.Find(AttributeDefinition->ServiceAttributeId);
				if (ensure(ServiceAttributeDefinition))
				{
					ServiceChanges.UpdatedAttributes.Add(AttributeDefinition->ServiceAttributeId, { AttributeDefinition->ServiceAttributeId, ServiceAttributeDefinition->Flags, SchemaDefinition->CompatibilityId });
				}
				else
				{
					UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Unable to find service attribute definition for service attribute %ls.",
						*SchemaDefinition->Id.ToString(),
						*SchemaCategoryDefinition->Id.ToString(),
						*AttributeDefinition->Id.ToString(),
						*AttributeDefinition->ServiceAttributeId.ToString());
					return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidState());
				}
			}

			NewPreparedClientChanges.SchemaCompatibilityId = SchemaDefinition->CompatibilityId;
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareClientChanges] %ls.%ls.%ls: Failed to find attribute definition for SchemaCompatibilityId field.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*SchemaCategoryDefinition->SchemaCompatibilityAttributeId.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>(Errors::InvalidState());
		}
	}

	// Store pending changes to be applied on commit.
	PreparedClientChanges = MoveTemp(NewPreparedClientChanges);

	return TOnlineResult<FSchemaCategoryInstancePrepareClientChanges>({MoveTemp(ServiceChanges)});
}

FSchemaCategoryInstanceCommitClientChanges::Result FSchemaCategoryInstanceBase::CommitClientChanges()
{
	FSchemaServiceClientChanges ClientChanges;

	// Check that there is a waiting delta state to be applied.
	if (!PreparedClientChanges.IsSet())
	{
		return FSchemaCategoryInstanceCommitClientChanges::Result({ MoveTemp(ClientChanges) });
	}

	// Apply delta to state snapshot.
	TMap<FSchemaAttributeId, FSchemaVariant>& ClientDataSnapshot = GetMutableClientSnapshot();

	// Handle added attributes.
	Algo::ForEach(PreparedClientChanges->ClientChanges.AddedAttributes,
		[&ClientDataSnapshot](const TPair<FSchemaAttributeId, FSchemaVariant>& AddedPair) -> void
		{
			ClientDataSnapshot.Add(AddedPair.Key, AddedPair.Value);
		});

	// Handle changed attributes.
	Algo::ForEach(PreparedClientChanges->ClientChanges.ChangedAttributes,
		[&ClientDataSnapshot](const TPair<FSchemaAttributeId, TPair<FSchemaVariant, FSchemaVariant>>& ChangedPair) -> void
		{
			ClientDataSnapshot.Add(ChangedPair.Key, ChangedPair.Value.Value);
		});

	// Handle removed attributes.
	Algo::ForEach(PreparedClientChanges->ClientChanges.RemovedAttributes,
		[&ClientDataSnapshot](FSchemaAttributeId RemovedAttributeId) -> void
		{
			ClientDataSnapshot.Remove(RemovedAttributeId);
		});

	// Set last sent schema compatibility id.
	if (PreparedClientChanges->SchemaCompatibilityId != 0)
	{
		LastSentSchemaCompatibilityId = PreparedClientChanges->SchemaCompatibilityId;
	}

	// Set schema definitions from a schema swap.
	if (PreparedClientChanges->DerivedSchemaCategoryDefinition)
	{
		DerivedSchemaDefinition = PreparedClientChanges->DerivedSchemaDefinition;
		DerivedSchemaCategoryDefinition = PreparedClientChanges->DerivedSchemaCategoryDefinition;
	}

	// Move out client changes for returning to the client.
	ClientChanges = MoveTemp(PreparedClientChanges->ClientChanges);

	// Reset pending state.
	ResetPreparedChanges();

	// Return client changes.
	return FSchemaCategoryInstanceCommitClientChanges::Result({MoveTemp(ClientChanges)});
}

TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> FSchemaCategoryInstanceBase::PrepareServiceSnapshot(FSchemaCategoryInstancePrepareServiceSnapshot::Params&& Params) const
{
	// todo: handle unpacking a service attribute into multiple attributes.

	// Start by clearing any previous prepared state.
	ResetPreparedChanges();
	FPreparedServiceChanges NewPreparedServiceChanges;
	FSchemaCategoryInstancePrepareServiceSnapshot::Result ApplyResult;

	const FSchemaDefinition* SchemaDefinition = nullptr;
	const FSchemaCategoryDefinition* SchemaCategoryDefinition = nullptr;
	if (!GetSerializationSchema(&SchemaDefinition, &SchemaCategoryDefinition))
	{
		UE_LOGF(LogOnlineSchema, Warning, "[CommitServiceSnapshot] Serialization schema category definition not found.");
		return TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot>(Errors::InvalidState());
	}

	// Check for a schema change.
	TPair<FSchemaServiceAttributeId, FSchemaVariant>* SchemaCompatibilityAttributeData = Algo::FindByPredicate(Params.ServiceSnapshot.Attributes,
		[&SchemaCategoryDefinition](const TPair<FSchemaServiceAttributeId, FSchemaVariant>& AttributeData)
		{
			return AttributeData.Key == SchemaCategoryDefinition->SchemaCompatibilityServiceAttributeId;
		});

	// Handle incoming schema change from service attribute.
	if (SchemaCompatibilityAttributeData)
	{
		const int64 SchemaCompatibilityId = SchemaCompatibilityAttributeData->Value.GetInt64();

		if (InitializeSchemaDefinition(
			SchemaCompatibilityId,
			BaseSchemaDefinition.Get(),
			SchemaCategoryDefinition->Id,
			&NewPreparedServiceChanges.DerivedSchemaDefinition,
			&NewPreparedServiceChanges.DerivedSchemaCategoryDefinition))
		{
			// Check that schema has actually changed.
			if (DerivedSchemaDefinition == nullptr || DerivedSchemaDefinition->Id != NewPreparedServiceChanges.DerivedSchemaDefinition->Id)
			{
				// Derived schema init succeeded. Switch to using it for preparing the client changes.
				SchemaDefinition = NewPreparedServiceChanges.DerivedSchemaDefinition.Get();
				SchemaCategoryDefinition = NewPreparedServiceChanges.DerivedSchemaCategoryDefinition;

				NewPreparedServiceChanges.ClientChanges.SchemaId.Emplace(SchemaDefinition->Id);
				ApplyResult.DerivedSchemaId.Emplace(SchemaDefinition->Id);
			}
		}
		else
		{
			UE_LOG(LogOnlineSchema, Warning, TEXT("[PrepareServiceSnapshot] %s.%s.%s: Failed to find valid definition for SchemaCompatibilityId 0x%08" INT64_X_FMT "."),
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*SchemaCategoryDefinition->SchemaCompatibilityServiceAttributeId.ToString(),
				SchemaCompatibilityId);
			return TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot>(Errors::InvalidParams());
		}
	}

	// Handle incoming schema change from user parameter.
	if (Params.SchemaId)
	{
		// Manual setting of schema id is only allowed when it cannot be detected from the service.
		if (SchemaCompatibilityAttributeData)
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareServiceSnapshot] %ls.%ls: Unable to handle user schema swap when service defines a schema compatibility id.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot>(Errors::InvalidParams());
		}
	
		if (InitializeSchemaDefinition(
			*Params.SchemaId,
			BaseSchemaDefinition.Get(),
			SchemaCategoryDefinition->Id,
			&NewPreparedServiceChanges.DerivedSchemaDefinition,
			&NewPreparedServiceChanges.DerivedSchemaCategoryDefinition))
		{
			// Check that schema has actually changed.
			if (DerivedSchemaDefinition == nullptr || DerivedSchemaDefinition->Id != NewPreparedServiceChanges.DerivedSchemaDefinition->Id)
			{
				// Derived schema init succeeded. Switch to using it for preparing the client changes.
				SchemaDefinition = NewPreparedServiceChanges.DerivedSchemaDefinition.Get();
				SchemaCategoryDefinition = NewPreparedServiceChanges.DerivedSchemaCategoryDefinition;

				NewPreparedServiceChanges.ClientChanges.SchemaId.Emplace(SchemaDefinition->Id);
				ApplyResult.DerivedSchemaId.Emplace(SchemaDefinition->Id);
			}
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareServiceSnapshot] %ls.%ls: Failed to find valid definition for SchemaId %ls.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*Params.SchemaId->ToString());
			return TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot>(Errors::InvalidParams());
		}
	}

	// Unpack service attributes.
	for (TPair<FSchemaServiceAttributeId, FSchemaVariant>& SchemaServiceAttributeData : Params.ServiceSnapshot.Attributes)
	{
		const FSchemaServiceAttributeId& SchemaServiceAttributeId = SchemaServiceAttributeData.Key;
		FSchemaVariant& SchemaServiceAttributeValue = SchemaServiceAttributeData.Value;

		if (const FSchemaServiceAttributeDefinition* SchemaServiceAttributeDefinition =
			SchemaCategoryDefinition->ServiceAttributeDefinitions.Find(SchemaServiceAttributeId))
		{
			for (const FSchemaAttributeId& SchemaAttributeId : SchemaServiceAttributeDefinition->SchemaAttributeIds)
			{
				// todo: unpack attributes from service attribute.
				check(SchemaServiceAttributeDefinition->SchemaAttributeIds.Num() == 1);

				const FSchemaAttributeDefinition* SchemaAttributeDefinition = SchemaCategoryDefinition->SchemaAttributeDefinitions.Find(SchemaAttributeId);

				// Attribute definition must be valid due to successfully processing config.
				check(SchemaAttributeDefinition);

				// Schema changes are handled by surfacing a new schema id in the result object. Don't return SchemaCompatibilityId as an attribute to the client.
				if (EnumHasAnyFlags(SchemaAttributeDefinition->Flags, ESchemaAttributeFlags::SchemaCompatibilityId))
				{
					UE_LOG(LogOnlineSchema, Verbose, TEXT("[PrepareServiceSnapshot] %s.%s.%s: Consuming SchemaCompatibility attribute: 0x%08" INT64_X_FMT "."),
						*SchemaDefinition->Id.ToString(),
						*SchemaCategoryDefinition->Id.ToString(),
						*SchemaAttributeDefinition->Id.ToString(),
						SchemaServiceAttributeValue.GetInt64());
				}
				else
				{
					NewPreparedServiceChanges.ClientDataSnapshot.Emplace(SchemaAttributeDefinition->Id, MoveTemp(SchemaServiceAttributeValue));
				}
			}
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Warning, "[PrepareServiceSnapshot] %ls.%ls.%ls: Service attribute definition does not exist in schema category.",
				*SchemaDefinition->Id.ToString(),
				*SchemaCategoryDefinition->Id.ToString(),
				*SchemaServiceAttributeId.ToString());
			// Allow code to continue when an unexpected attribute is received.
			continue;
		}
	}

	// Process new snapshot against old snapshot to build client data changes.
	{
		// Find attributes that were added or changed.
		for (const TPair<FSchemaAttributeId, FSchemaVariant>& NewSnapshotAttributeData : NewPreparedServiceChanges.ClientDataSnapshot)
		{
			const FSchemaServiceAttributeId& NewSnapshotAttributeId = NewSnapshotAttributeData.Key;
			const FSchemaVariant& NewSnapshotAttributeValue = NewSnapshotAttributeData.Value;

			if (const FSchemaVariant* OldSnapshotAttributeValue = GetClientSnapshot().Find(NewSnapshotAttributeId))
			{
				if (*OldSnapshotAttributeValue != NewSnapshotAttributeValue)
				{
					NewPreparedServiceChanges.ClientChanges.ChangedAttributes.Add(NewSnapshotAttributeId, { *OldSnapshotAttributeValue, NewSnapshotAttributeValue });
				}
			}
			else
			{
				NewPreparedServiceChanges.ClientChanges.AddedAttributes.Add(NewSnapshotAttributeId, NewSnapshotAttributeValue);
			}
		}

		// Add removed attribute ids to changes.
		Algo::TransformIf(GetClientSnapshot(), NewPreparedServiceChanges.ClientChanges.RemovedAttributes,
			[&NewPreparedServiceChanges](const TPair<FSchemaAttributeId, FSchemaVariant>& OldSnapshotAttributeData) -> bool
			{
				const FSchemaServiceAttributeId& OldSnapshotAttributeId = OldSnapshotAttributeData.Key;
				return !NewPreparedServiceChanges.ClientDataSnapshot.Contains(OldSnapshotAttributeId);
			},
			[](const TPair<FSchemaAttributeId, FSchemaVariant>& OldSnapshotAttributeData) -> FSchemaAttributeId
			{
				const FSchemaServiceAttributeId& OldSnapshotAttributeId = OldSnapshotAttributeData.Key;
				return OldSnapshotAttributeId;
			});
	}

	// Store pending changes to be applied on commit.
	PreparedServiceChanges = MoveTemp(NewPreparedServiceChanges);
	return TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot>(MoveTemp(ApplyResult));
}

FSchemaCategoryInstanceCommitServiceSnapshot::Result FSchemaCategoryInstanceBase::CommitServiceSnapshot()
{
	FSchemaServiceClientChanges ClientChanges;

	// Check that there is a waiting snapshot state to be applied.
	if (!PreparedServiceChanges.IsSet())
	{
		return FSchemaCategoryInstanceCommitServiceSnapshot::Result({ MoveTemp(ClientChanges) });
	}

	// Apply attribute changes.
	GetMutableClientSnapshot() = MoveTemp(PreparedServiceChanges->ClientDataSnapshot);

	// Set schema definitions from a schema swap.
	if (PreparedServiceChanges->DerivedSchemaCategoryDefinition)
	{
		DerivedSchemaDefinition = PreparedServiceChanges->DerivedSchemaDefinition;
		DerivedSchemaCategoryDefinition = PreparedServiceChanges->DerivedSchemaCategoryDefinition;
	}

	// Move out client changes for returning to the client.
	ClientChanges = MoveTemp(PreparedServiceChanges->ClientChanges);

	// Reset pending state.
	ResetPreparedChanges();

	// Return client changes.
	return FSchemaCategoryInstanceCommitServiceSnapshot::Result({ MoveTemp(ClientChanges) });
}

bool FSchemaCategoryInstanceBase::IsValid() const
{
	return BaseSchemaCategoryDefinition != nullptr;
}

bool FSchemaCategoryInstanceBase::VerifyBaseAttributeData(
	const FSchemaAttributeId& Id,
	const FSchemaVariant& Data,
	FSchemaServiceAttributeId& OutSchemaServiceAttributeId,
	ESchemaServiceAttributeFlags& OutSchemaServiceAttributeFlags)
{
	if (ensure(IsValid()))
	{
		if (const FSchemaAttributeDefinition* AttributeDefinition = BaseSchemaCategoryDefinition->SchemaAttributeDefinitions.Find(Id))
		{
			if (AttributeDefinition->Type == Data.GetType())
			{
				const bool bIsDataSizeValid = AttributeDefinition->Type != ESchemaAttributeType::String || Data.GetString().Len() <= AttributeDefinition->MaxSize;
				if (bIsDataSizeValid)
				{
					OutSchemaServiceAttributeId = AttributeDefinition->ServiceAttributeId;

					if (AttributeDefinition->ServiceAttributeId != FSchemaServiceAttributeId())
					{
						const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = BaseSchemaCategoryDefinition->ServiceAttributeDefinitions.Find(AttributeDefinition->ServiceAttributeId);
						if (ensure(ServiceAttributeDefinition))
						{
							OutSchemaServiceAttributeFlags = ServiceAttributeDefinition->Flags;
						}
						else
						{
							UE_LOGF(LogOnlineSchema, Warning, "[VerifyBaseAttributeData] %ls.%ls.%ls: Unable to find service attribute definition for service attribute %ls.",
								*BaseSchemaCategoryDefinition->Id.ToString(),
								*BaseSchemaCategoryDefinition->Id.ToString(),
								*AttributeDefinition->Id.ToString(),
								*AttributeDefinition->ServiceAttributeId.ToString());
							return false;
						}
					}

					return true;
				}
				else
				{
					return false;
				}
			}
			else
			{
				UE_LOGF(LogOnlineSchema, Verbose, "[VerifyBaseAttributeData] Schema attribute %ls.%ls.%ls set with invalid type %ls. Expected type %ls",
					*BaseSchemaDefinition->Id.ToString(),
					*BaseSchemaCategoryDefinition->Id.ToString(),
					*AttributeDefinition->Id.ToString(),
					LexToString(Data.GetType()),
					LexToString(AttributeDefinition->Type));
				return false;
			}
		}
		else
		{
			UE_LOGF(LogOnlineSchema, Verbose, "[VerifyBaseAttributeData] Attribute %ls not found in schema definition %ls.%ls",
				*Id.ToString(),
				*BaseSchemaDefinition->Id.ToString(),
				*BaseSchemaCategoryDefinition->Id.ToString());
			return false;
		}
	}
	else
	{
		UE_LOGF(LogOnlineSchema, Verbose, "[VerifyBaseAttributeData] Unable to set attribute %ls. Schema is not valid",
			*Id.ToString());
		return false;
	}
}

const TMap<FSchemaAttributeId, FSchemaVariant>& FSchemaCategoryInstanceBase::GetClientSnapshot() const
{
	return const_cast<FSchemaCategoryInstanceBase*>(this)->GetMutableClientSnapshot();
}

bool FSchemaCategoryInstanceBase::InitializeSchemaDefinition(
	int64 SchemaCompatibilityId,
	const FSchemaDefinition* OptionalBaseDefinition,
	const FSchemaCategoryId& CategoryId,
	TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
	const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const
{
	if (TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry->GetDefinition(SchemaCompatibilityId))
	{
		return InitializeSchemaDefinition(SchemaDefinition.ToSharedRef(), OptionalBaseDefinition, CategoryId, OutSchemaDefinition, OutSchemaCategoryDefinition);
	}
	else
	{
		UE_LOG(LogOnlineSchema, Error, TEXT("[InitializeSchemaDefinition] init error: Unable to find definition for SchemaCompatibilityId 0x%08" INT64_X_FMT "."),
			SchemaCompatibilityId);
		return false;
	}
}

bool FSchemaCategoryInstanceBase::InitializeSchemaDefinition(
	const FSchemaId& SchemaId,
	const FSchemaDefinition* OptionalBaseDefinition,
	const FSchemaCategoryId& CategoryId,
	TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
	const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const
{
	if (TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry->GetDefinition(SchemaId))
	{
		return InitializeSchemaDefinition(SchemaDefinition.ToSharedRef(), OptionalBaseDefinition, CategoryId, OutSchemaDefinition, OutSchemaCategoryDefinition);
	}
	else
	{
		UE_LOGF(LogOnlineSchema, Error, "[InitializeSchemaDefinition] init error: Unable to find definition for schema %ls",
			*SchemaId.ToString());
		return false;
	}
}

bool FSchemaCategoryInstanceBase::InitializeSchemaDefinition(
	const TSharedRef<const FSchemaDefinition>& NewDefinition,
	const FSchemaDefinition* OptionalBaseDefinition,
	const FSchemaCategoryId& CategoryId,
	TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
	const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const
{
	check(OutSchemaDefinition);
	check(OutSchemaCategoryDefinition);

	// Enforce parent <-> child relationship if needed.
	if (OptionalBaseDefinition && !SchemaRegistry->IsSchemaChildOf(NewDefinition->Id, OptionalBaseDefinition->Id))
	{
		UE_LOGF(LogOnlineSchema, Warning, "[InitializeSchemaDefinition] Invalid schema definition. New schema %ls is not a child of %ls.",
			*NewDefinition->Id.ToString(),
			*OptionalBaseDefinition->Id.ToString());
		return false;
	}

	// Check that category exists in incoming schema.
	const FSchemaCategoryDefinition* SchemaCategoryDefinition = NewDefinition->Categories.Find(CategoryId);
	if (SchemaCategoryDefinition == nullptr)
	{
		UE_LOGF(LogOnlineSchema, Error, "[InitializeSchemaDefinition] init error: Unable to find category %ls in schema %ls",
			*CategoryId.ToString(),
			*NewDefinition->Id.ToString());
		return false;
	}

	*OutSchemaDefinition = NewDefinition;
	*OutSchemaCategoryDefinition = SchemaCategoryDefinition;
	return true;
}

bool FSchemaCategoryInstanceBase::GetSerializationSchema(
	const FSchemaDefinition** OutSchemaDefinition,
	const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const
{
	check(OutSchemaDefinition);
	check(OutSchemaCategoryDefinition);

	if (IsValid())
	{
		if (DerivedSchemaCategoryDefinition)
		{
			*OutSchemaDefinition = DerivedSchemaDefinition.Get();
			*OutSchemaCategoryDefinition = DerivedSchemaCategoryDefinition;
		}
		else
		{
			*OutSchemaDefinition = BaseSchemaDefinition.Get();
			*OutSchemaCategoryDefinition = BaseSchemaCategoryDefinition;
		}

		return true;
	}
	else
	{
		return false;
	}
}

void FSchemaCategoryInstanceBase::ResetPreparedChanges() const
{
	PreparedClientChanges.Reset();
	PreparedServiceChanges.Reset();
}

/* Private */ }

TSharedPtr<const FSchemaDefinition> FSchemaRegistry::GetDefinition(const FSchemaId& SchemaId) const
{
	const TSharedRef<const FSchemaDefinition>* Definition = SchemaDefinitionsById.Find(SchemaId);
	return Definition ? *Definition : TSharedPtr<const FSchemaDefinition>();
}

TSharedPtr<const FSchemaDefinition> FSchemaRegistry::GetDefinition(int64 CompatibilityId) const
{
	const TSharedRef<const FSchemaDefinition>* Definition = SchemaDefinitionsByCompatibilityId.Find(CompatibilityId);
	return Definition ? *Definition : TSharedPtr<const FSchemaDefinition>();
}

bool FSchemaRegistry::IsSchemaChildOf(const FSchemaId& SchemaId, const FSchemaId& ParentSchemaId) const
{
	const FSchemaId* FoundSchemaId = nullptr;
	if (const TSharedRef<const FSchemaDefinition>* Definition = SchemaDefinitionsById.Find(SchemaId))
	{
		FoundSchemaId = (* Definition)->ParentSchemaIds.Find(ParentSchemaId);
	}
	return FoundSchemaId != nullptr;
}

/* UE::Online */ }