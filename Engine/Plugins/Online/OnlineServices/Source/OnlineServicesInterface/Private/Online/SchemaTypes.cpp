// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SchemaTypes.h"
#include "CoreGlobals.h"

namespace UE::Online {

const TCHAR* LexToString(ESchemaAttributeComparisonOp Comparison)
{
	switch (Comparison)
	{
	case ESchemaAttributeComparisonOp::Equals:			return TEXT("Equals");
	case ESchemaAttributeComparisonOp::NotEquals:			return TEXT("NotEquals");
	case ESchemaAttributeComparisonOp::GreaterThan:		return TEXT("GreaterThan");
	case ESchemaAttributeComparisonOp::GreaterThanEquals:	return TEXT("GreaterThanEquals");
	case ESchemaAttributeComparisonOp::LessThan:			return TEXT("LessThan");
	case ESchemaAttributeComparisonOp::LessThanEquals:	return TEXT("LessThanEquals");
	case ESchemaAttributeComparisonOp::Near:				return TEXT("Near");
	case ESchemaAttributeComparisonOp::In:				return TEXT("In");
	default:									checkNoEntry(); // Intentional fallthrough
	case ESchemaAttributeComparisonOp::NotIn:				return TEXT("NotIn");
	}
}

void LexFromString(ESchemaAttributeComparisonOp& OutComparison, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Equals")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::Equals;
	}
	else if (FCString::Stricmp(InStr, TEXT("NotEquals")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::NotEquals;
	}
	else if (FCString::Stricmp(InStr, TEXT("GreaterThan")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::GreaterThan;
	}
	else if (FCString::Stricmp(InStr, TEXT("GreaterThanEquals")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::GreaterThanEquals;
	}
	else if (FCString::Stricmp(InStr, TEXT("LessThan")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::LessThan;
	}
	else if (FCString::Stricmp(InStr, TEXT("LessThanEquals")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::LessThanEquals;
	}
	else if (FCString::Stricmp(InStr, TEXT("Near")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::Near;
	}
	else if (FCString::Stricmp(InStr, TEXT("In")) == 0)
	{
		OutComparison = ESchemaAttributeComparisonOp::In;
	}
	else
	{
		checkNoEntry();
		OutComparison = ESchemaAttributeComparisonOp::In;
	}
}

const TCHAR* LexToString(ESchemaAttributeVisibility Visibility)
{
	switch (Visibility)
	{
	case ESchemaAttributeVisibility::Public:		return TEXT("Public");
	default:									checkNoEntry(); // Intentional fallthrough
	case ESchemaAttributeVisibility::Private:	return TEXT("Private");
	}
}

void LexFromString(ESchemaAttributeVisibility& OutVisibility, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Public")) == 0)
	{
		OutVisibility = ESchemaAttributeVisibility::Public;
	}
	else if (FCString::Stricmp(InStr, TEXT("Private")) == 0)
	{
		OutVisibility = ESchemaAttributeVisibility::Private;
	}
	else
	{
		checkNoEntry();
		OutVisibility = ESchemaAttributeVisibility::Private;
	}
}

const TCHAR* LexToString(ESchemaAttributeFlags SchemaAttributeFlags)
{
	switch (SchemaAttributeFlags)
	{
	default:											checkNoEntry(); // Intentional fallthrough
	case ESchemaAttributeFlags::None:					return TEXT("None");
	case ESchemaAttributeFlags::Searchable:				return TEXT("Searchable");
	case ESchemaAttributeFlags::Public:					return TEXT("Public");
	case ESchemaAttributeFlags::Private:				return TEXT("Private");
	case ESchemaAttributeFlags::SchemaCompatibilityId:	return TEXT("SchemaCompatibilityId");
	}
}

void LexFromString(ESchemaAttributeFlags& OutSchemaAttributeFlags, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("None")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaAttributeFlags::None;
	}
	else if (FCString::Stricmp(InStr, TEXT("Searchable")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaAttributeFlags::Searchable;
	}
	else if (FCString::Stricmp(InStr, TEXT("Public")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaAttributeFlags::Public;
	}
	else if (FCString::Stricmp(InStr, TEXT("Private")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaAttributeFlags::Private;
	}
	else if (FCString::Stricmp(InStr, TEXT("SchemaCompatibilityId")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaAttributeFlags::SchemaCompatibilityId;
	}
	else
	{
		checkNoEntry();
		OutSchemaAttributeFlags = ESchemaAttributeFlags::None;
	}
}

const TCHAR* LexToString(ESchemaAttributeType SchemaAttributeType)
{
	switch (SchemaAttributeType)
	{
	default:							checkNoEntry(); // Intentional fallthrough
	case ESchemaAttributeType::Bool:	return TEXT("Bool");
	case ESchemaAttributeType::Int64:	return TEXT("Int64");
	case ESchemaAttributeType::Double:	return TEXT("Double");
	case ESchemaAttributeType::String:	return TEXT("String");
	}
}

void LexFromString(ESchemaAttributeType& OutSchemaAttributeType, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Bool")) == 0)
	{
		OutSchemaAttributeType = ESchemaAttributeType::Bool;
	}
	else if (FCString::Stricmp(InStr, TEXT("Int64")) == 0)
	{
		OutSchemaAttributeType = ESchemaAttributeType::Int64;
	}
	else if (FCString::Stricmp(InStr, TEXT("Double")) == 0)
	{
		OutSchemaAttributeType = ESchemaAttributeType::Double;
	}
	else if (FCString::Stricmp(InStr, TEXT("String")) == 0)
	{
		OutSchemaAttributeType = ESchemaAttributeType::String;
	}

	else
	{
		checkNoEntry();
		OutSchemaAttributeType = ESchemaAttributeType::Bool;
	}
}

const TCHAR* LexToString(ESchemaServiceAttributeFlags SchemaAttributeFlags)
{
	switch (SchemaAttributeFlags)
	{
	default:										checkNoEntry(); // Intentional fallthrough
	case ESchemaServiceAttributeFlags::None:		return TEXT("None");
	case ESchemaServiceAttributeFlags::Searchable:	return TEXT("Searchable");
	case ESchemaServiceAttributeFlags::Public:		return TEXT("Public");
	case ESchemaServiceAttributeFlags::Private:		return TEXT("Private");
	}
}

void LexFromString(ESchemaServiceAttributeFlags& OutSchemaAttributeFlags, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("None")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaServiceAttributeFlags::None;
	}
	else if (FCString::Stricmp(InStr, TEXT("Searchable")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaServiceAttributeFlags::Searchable;
	}
	else if (FCString::Stricmp(InStr, TEXT("Public")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaServiceAttributeFlags::Public;
	}
	else if (FCString::Stricmp(InStr, TEXT("Private")) == 0)
	{
		OutSchemaAttributeFlags = ESchemaServiceAttributeFlags::Private;
	}
	else
	{
		checkNoEntry();
		OutSchemaAttributeFlags = ESchemaServiceAttributeFlags::None;
	}
}

const TCHAR* LexToString(ESchemaServiceAttributeSupportedTypeFlags SchemaServiceAttributeTypeFlags)
{
	switch (SchemaServiceAttributeTypeFlags)
	{
	default:												checkNoEntry(); // Intentional fallthrough
	case ESchemaServiceAttributeSupportedTypeFlags::Bool:	return TEXT("Bool");
	case ESchemaServiceAttributeSupportedTypeFlags::Int64:	return TEXT("Int64");
	case ESchemaServiceAttributeSupportedTypeFlags::Double:	return TEXT("Double");
	case ESchemaServiceAttributeSupportedTypeFlags::String:	return TEXT("String");
	}
}

void LexFromString(ESchemaServiceAttributeSupportedTypeFlags& OutSchemaServiceAttributeTypeFlags, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Bool")) == 0)
	{
		OutSchemaServiceAttributeTypeFlags = ESchemaServiceAttributeSupportedTypeFlags::Bool;
	}
	else if (FCString::Stricmp(InStr, TEXT("Int64")) == 0)
	{
		OutSchemaServiceAttributeTypeFlags = ESchemaServiceAttributeSupportedTypeFlags::Int64;
	}
	else if (FCString::Stricmp(InStr, TEXT("Double")) == 0)
	{
		OutSchemaServiceAttributeTypeFlags = ESchemaServiceAttributeSupportedTypeFlags::Double;
	}
	else if (FCString::Stricmp(InStr, TEXT("String")) == 0)
	{
		OutSchemaServiceAttributeTypeFlags = ESchemaServiceAttributeSupportedTypeFlags::String;
	}
	else
	{
		checkNoEntry();
		OutSchemaServiceAttributeTypeFlags = ESchemaServiceAttributeSupportedTypeFlags::Bool;
	}
}

FSchemaVariant::FSchemaVariant(FSchemaVariant&& InOther)
{
	VariantData = MoveTemp(InOther.VariantData);
	VariantType = InOther.VariantType;
	InOther.VariantType = ESchemaAttributeType::None;
}

FSchemaVariant& FSchemaVariant::operator=(FSchemaVariant&& InOther)
{
	VariantData = MoveTemp(InOther.VariantData);
	VariantType = InOther.VariantType;
	InOther.VariantType = ESchemaAttributeType::None;
	return *this;
}

void FSchemaVariant::Set(const TCHAR* AsString)
{
	VariantData.Emplace<FString>(AsString);
	VariantType = ESchemaAttributeType::String;
}

void FSchemaVariant::Set(const FString& AsString)
{
	VariantData.Emplace<FString>(AsString);
	VariantType = ESchemaAttributeType::String;
}

void FSchemaVariant::Set(FString&& AsString)
{
	VariantData.Emplace<FString>(MoveTemp(AsString));
	VariantType = ESchemaAttributeType::String;
}

void FSchemaVariant::Set(int64 AsInt)
{
	VariantData.Emplace<int64>(AsInt);
	VariantType = ESchemaAttributeType::Int64;
}

void FSchemaVariant::Set(double AsDouble)
{
	VariantData.Emplace<double>(AsDouble);
	VariantType = ESchemaAttributeType::Double;
}

void FSchemaVariant::Set(bool bAsBool)
{
	VariantData.Emplace<bool>(bAsBool);
	VariantType = ESchemaAttributeType::Bool;
}

int64 FSchemaVariant::GetInt64() const
{
	if (ensure(VariantType == ESchemaAttributeType::Int64))
	{
		return VariantData.Get<int64>();
	}
	else
	{
		return 0;
	}
}

double FSchemaVariant::GetDouble() const
{
	if (ensure(VariantType == ESchemaAttributeType::Double))
	{
		return VariantData.Get<double>();
	}
	else
	{
		return 0;
	}
}

bool FSchemaVariant::GetBoolean() const
{
	if (ensure(VariantType == ESchemaAttributeType::Bool))
	{
		return VariantData.Get<bool>();
	}
	else
	{
		return 0;
	}
}

const FString& FSchemaVariant::GetString() const
{
	if (ensure(VariantType == ESchemaAttributeType::String))
	{
		return VariantData.Get<FString>();
	}
	else
	{
		static FString EmptyString;
		return EmptyString;
	}
}

FString FSchemaVariant::ToLogString() const
{
	FString ValueString;

	if (VariantData.IsType<FString>())
	{
		ValueString = VariantData.Get<FString>();
	}
	else if (VariantData.IsType<int64>())
	{
		ValueString = FString::Printf(TEXT("%" INT64_FMT), VariantData.Get<int64>());
	}
	else if (VariantData.IsType<bool>())
	{
		ValueString = ::LexToString(VariantData.Get<bool>());
	}
	else if (VariantData.IsType<double>())
	{
		ValueString = FString::Printf(TEXT("%.3f"), VariantData.Get<double>());
	}

	return FString::Printf(TEXT("%s:%s"), LexToString(VariantType), *ValueString);
}

bool FSchemaVariant::operator==(const FSchemaVariant& Other) const
{
	if (VariantData.GetIndex() != Other.VariantData.GetIndex())
	{
		return false;
	}

	// Type should always match due to the variant data.
	check(VariantType == Other.VariantType);

	switch (VariantData.GetIndex())
	{
	case FVariantType::IndexOfType<FString>():	return VariantData.Get<FString>() == Other.VariantData.Get<FString>();
	case FVariantType::IndexOfType<int64>():	return VariantData.Get<int64>() == Other.VariantData.Get<int64>();
	case FVariantType::IndexOfType<double>():	return VariantData.Get<double>() == Other.VariantData.Get<double>();

	default:									checkNoEntry(); // Intentional fallthrough
	case FVariantType::IndexOfType<bool>():		return VariantData.Get<bool>() == Other.VariantData.Get<bool>();
	}
}

const FString LexToString(const FLexToStringAdaptor<FSchemaVariant>& Adaptor)
{
	return Adaptor.SchemaVariant.ToLogString();
}

void LexFromString(FSchemaVariant& OutSchemaVariant, const TCHAR* InStr)
{
	if (const TCHAR* ValueStr = FCString::Strchr(InStr, ':'))
	{
		const int32 TypeLen = UE_PTRDIFF_TO_INT32(ValueStr - InStr);
		const FString TypeStr = FString::ConstructFromPtrSize(InStr, TypeLen);
		ValueStr++;

		LexFromString(OutSchemaVariant.VariantType, *TypeStr);
		switch (OutSchemaVariant.VariantType)
		{
		case ESchemaAttributeType::None:
			OutSchemaVariant.VariantData = FSchemaVariant::FVariantType();
			break;
		case ESchemaAttributeType::Bool:
			OutSchemaVariant.VariantData.Emplace<bool>(FCString::ToBool(ValueStr));
			break;
		case ESchemaAttributeType::Int64:
			OutSchemaVariant.VariantData.Emplace<int64>(FCString::Strtoi64(ValueStr, nullptr, 10));
			break;
		case ESchemaAttributeType::Double:
			OutSchemaVariant.VariantData.Emplace<double>(FCString::Atod(ValueStr));
			break;
		case ESchemaAttributeType::String:
			OutSchemaVariant.VariantData.Emplace<FString>(ValueStr);
			break;
		}
	}
}

/* UE::Online */ }
