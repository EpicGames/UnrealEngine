// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/Sessions.h"

namespace UE::Online {

const TCHAR* LexToString(ESessionJoinPolicy Value)
{
	switch (Value)
	{
	case ESessionJoinPolicy::Public:		return TEXT("Public");
	case ESessionJoinPolicy::FriendsOnly:	return TEXT("FriendsOnly");
	default:								checkNoEntry(); // Intentional fallthrough
	case ESessionJoinPolicy::InviteOnly:	return TEXT("InviteOnly");
	}
}

void LexFromString(ESessionJoinPolicy& Value, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Public")) == 0)
	{
		Value = ESessionJoinPolicy::Public;
	}
	else if (FCString::Stricmp(InStr, TEXT("FriendsOnly")) == 0)
	{
		Value = ESessionJoinPolicy::FriendsOnly;
	}
	else if (FCString::Stricmp(InStr, TEXT("InviteOnly")) == 0)
	{
		Value = ESessionJoinPolicy::InviteOnly;
	}
	else
	{
		checkNoEntry();
		Value = ESessionJoinPolicy::InviteOnly;
	}
}

#define MOVE_TOPTIONAL_IF_SET(Value) \
	if (UpdatedValue.Value.IsSet()) \
	{ \
		Value = MoveTemp(UpdatedValue.Value); \
	} \

FSessionSettingsUpdate& FSessionSettingsUpdate::operator+=(FSessionSettingsUpdate&& UpdatedValue)
{
	MOVE_TOPTIONAL_IF_SET(SchemaName)
	MOVE_TOPTIONAL_IF_SET(NumMaxConnections)
	MOVE_TOPTIONAL_IF_SET(JoinPolicy)
	MOVE_TOPTIONAL_IF_SET(bAllowNewMembers)

	for (const TPair<FSchemaAttributeId, FCustomSessionSetting>& UpdatedCustomSetting : UpdatedValue.UpdatedCustomSettings)
	{
		// If an update adds a modification to a setting that had previously been marked for removal, we'll keep the latest change
		RemovedCustomSettings.Remove(UpdatedCustomSetting.Key);
	}

	UpdatedCustomSettings.Append(MoveTemp(UpdatedValue.UpdatedCustomSettings));

	for (FSchemaAttributeId& Key : UpdatedValue.RemovedCustomSettings)
	{
		// If an update removes a setting that had previously been modified, we'll keep the latest change
		UpdatedCustomSettings.Remove(Key);
		RemovedCustomSettings.AddUnique(MoveTemp(Key));
	}

	return *this;
}

#undef MOVE_TOPTIONAL_IF_SET

#define COPY_TOPTIONAL_VALUE_IF_SET(Value) \
	if (SettingsChanges.Value.IsSet()) \
	{ \
		Value = SettingsChanges.Value.GetValue(); \
	} \

FSessionSettings& FSessionSettings::operator+=(const FSessionSettingsChanges& SettingsChanges)
{
	// TODO: Full support for Schema functionality still pending
	COPY_TOPTIONAL_VALUE_IF_SET(SchemaName)
	COPY_TOPTIONAL_VALUE_IF_SET(NumMaxConnections)
	COPY_TOPTIONAL_VALUE_IF_SET(JoinPolicy)
	COPY_TOPTIONAL_VALUE_IF_SET(bAllowNewMembers)

	for (const FSchemaAttributeId& Key : SettingsChanges.RemovedCustomSettings)
	{
		CustomSettings.Remove(Key);
	}

	CustomSettings.Append(SettingsChanges.AddedCustomSettings);

	for (const TPair<FSchemaAttributeId, FCustomSessionSettingUpdate>& SettingEntry : SettingsChanges.ChangedCustomSettings)
	{
		if (FCustomSessionSetting* CustomSetting = CustomSettings.Find(SettingEntry.Key))
		{
			(*CustomSetting) = SettingEntry.Value.NewValue;
		}
	}

	return *this;
}

#undef COPY_TOPTIONAL_VALUE_IF_SET

const FString ToLogString(const ISession& Session)
{
	return Session.ToLogString();
}

const FString ToLogString(const ISessionInvite& SessionInvite)
{
	return SessionInvite.ToLogString();
}

const TCHAR* LexToString(EUISessionJoinRequestedSource UISessionJoinRequestedSource)
{
	switch (UISessionJoinRequestedSource)
	{
	case EUISessionJoinRequestedSource::FromInvitation:	return TEXT("FromInvitation");
	default:											checkNoEntry(); // Intentional fallthrough
	case EUISessionJoinRequestedSource::Unspecified:	return TEXT("Unspecified");
	}
}

void LexFromString(EUISessionJoinRequestedSource& OutUISessionJoinRequestedSource, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("FromInvitation")) == 0)
	{
		OutUISessionJoinRequestedSource = EUISessionJoinRequestedSource::FromInvitation;
	}
	else if (FCString::Stricmp(InStr, TEXT("Unspecified")) == 0)
	{
		OutUISessionJoinRequestedSource = EUISessionJoinRequestedSource::Unspecified;
	}
	else
	{
		checkNoEntry();
		OutUISessionJoinRequestedSource = EUISessionJoinRequestedSource::Unspecified;
	}
}

/* UE::Online */ }