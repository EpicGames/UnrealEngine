// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"
#include "Misc/DateTime.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

struct FQueryAchievementDefinitions
{
	static constexpr TCHAR Name[] = TEXT("QueryAchievementDefinitions");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FGetAchievementIds
{
	static constexpr TCHAR Name[] = TEXT("GetAchievementIds");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Available achievements */
		TArray<FString> AchievementIds;
	};
};

struct FAchievementStatDefinition
{
	/** Unique Id of the stat */
	FString StatId;
	/** Threshold value a user must meet for the achievement to auto unlock */
	uint32 UnlockThreshold = 0;
};

FString ONLINESERVICESINTERFACE_API ToLogString(const FAchievementStatDefinition&);

struct FAchievementDefinition
{
	/** Unique Id for this achievement */
	FString AchievementId;
	/** Localized display name of this achievement, once unlocked */
	FText UnlockedDisplayName;
	/** Localized description of this achievement, once unlocked */
	FText UnlockedDescription;
	/** Localized display name of this achievement, while locked */
	FText LockedDisplayName;
	/** Localized description of this achievement, while locked */
	FText LockedDescription;
	/** Localized flavor text */
	FText FlavorText;
	/** URL of the icon for this achievement, once unlocked */
	FString UnlockedIconUrl;
	/** URL of the icon for this achievement, while locked */
	FString LockedIconUrl;
	/** Is this achievement "hidden" until unlocked */
	bool bIsHidden = false;
	/** The stats that relate to this achievement */
	TArray<FAchievementStatDefinition> StatDefinitions;
};

FString ONLINESERVICESINTERFACE_API ToLogString(const FAchievementDefinition&);

struct FGetAchievementDefinition
{
	static constexpr TCHAR Name[] = TEXT("GetAchievementDefinition");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** Achievement to get definition of */
		FString AchievementId;
	};

	struct Result
	{
		/** Definition of the requested achievement */
		FAchievementDefinition AchievementDefinition;
	};
};

struct FQueryAchievementStates
{
	static constexpr TCHAR Name[] = TEXT("QueryAchievementStates");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FAchievementState
{
	/** Achievement that this state relates to */
	FString AchievementId;
	/** Progress towards unlocking this achievement, as a percentage in the range [0.0, 1.0]. A value of 1.0 means the achievement is unlocked */
	float Progress = 0.0f;
	/** If unlocked, the time this achievement was unlocked */
	FDateTime UnlockTime;
};

FString ONLINESERVICESINTERFACE_API ToLogString(const FAchievementState&);

struct FGetAchievementState
{
	static constexpr TCHAR Name[] = TEXT("GetAchievementState");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** Achievement to get state of */
		FString AchievementId;
	};

	struct Result
	{
		/** The state of the achievement for the local user */
		FAchievementState AchievementState;
	};
};

struct FUnlockAchievements
{
	static constexpr TCHAR Name[] = TEXT("UnlockAchievements");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** Achievements to unlock */
		TArray<FString> AchievementIds;
	};

	struct Result
	{
	};
};

struct FDisplayAchievementUI
{
	static constexpr TCHAR Name[] = TEXT("DisplayAchievementUI");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** Achievement to display */
		FString AchievementId;
	};

	struct Result
	{
	};
};

/** Struct for AchievementStateUpdated event */
struct FAchievementStateUpdated
{
	/** User whose achievement states have updated */
	FAccountId LocalAccountId;
	/** Achievements which have updated */
	TArray<FString> AchievementIds;
};

bool ONLINESERVICESINTERFACE_API operator==(const FAchievementStateUpdated&, const FAchievementStateUpdated&);

class IAchievements
{
public:
	virtual ~IAchievements() = default;

	/**
	 * Query all achievement definitions.
	 */
	virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) = 0;

	/**
	 * Gets the achievement id's.
	 * Requires first calling QueryAchievementDefinitions.
	 */
	virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) = 0;

	/**
	 * Gets an achievement definition by id.
	 * Requires first calling QueryAchievementDefinitions.
	 */
	virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) = 0;

	/**
	 * Query the state of all achievements for the given player.
	 * Requires first calling QueryAchievementDefinitions.
	 */
	virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) = 0;

	/**
	 * Gets the state of an achievement by id for the given player.
	 * Requires first calling QueryAchievementStates.
	 */
	virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const = 0;

	/**
	 * Manually unlock achievements
	 * Requires first calling QueryAchievementStates.
	 */
	virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) = 0;

	/**
	 * Launch the platform UI for a particular achievement
	 */
	virtual TOnlineResult<FDisplayAchievementUI> DisplayAchievementUI(FDisplayAchievementUI::Params&& Params) = 0;

	/**
	 * Event triggered when a player's achievement state changes
	 */
	virtual TOnlineEvent<void(const FAchievementStateUpdated&)> OnAchievementStateUpdated() = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FQueryAchievementDefinitions::Params)
	ONLINE_STRUCT_FIELD(FQueryAchievementDefinitions::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryAchievementDefinitions::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementIds::Params)
	ONLINE_STRUCT_FIELD(FGetAchievementIds::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementIds::Result)
	ONLINE_STRUCT_FIELD(FGetAchievementIds::Result, AchievementIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementStatDefinition)
	ONLINE_STRUCT_FIELD(FAchievementStatDefinition, StatId),
	ONLINE_STRUCT_FIELD(FAchievementStatDefinition, UnlockThreshold)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementDefinition)
	ONLINE_STRUCT_FIELD(FAchievementDefinition, AchievementId),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, UnlockedDisplayName),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, UnlockedDescription),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, LockedDisplayName),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, LockedDescription),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, FlavorText),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, UnlockedIconUrl),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, LockedIconUrl),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, bIsHidden),
	ONLINE_STRUCT_FIELD(FAchievementDefinition, StatDefinitions)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementDefinition::Params)
	ONLINE_STRUCT_FIELD(FGetAchievementDefinition::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetAchievementDefinition::Params, AchievementId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementDefinition::Result)
	ONLINE_STRUCT_FIELD(FGetAchievementDefinition::Result, AchievementDefinition)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryAchievementStates::Params)
	ONLINE_STRUCT_FIELD(FQueryAchievementStates::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryAchievementStates::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementState)
	ONLINE_STRUCT_FIELD(FAchievementState, AchievementId),
	ONLINE_STRUCT_FIELD(FAchievementState, Progress),
	ONLINE_STRUCT_FIELD(FAchievementState, UnlockTime)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementState::Params)
	ONLINE_STRUCT_FIELD(FGetAchievementState::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetAchievementState::Params, AchievementId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAchievementState::Result)
	ONLINE_STRUCT_FIELD(FGetAchievementState::Result, AchievementState)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUnlockAchievements::Params)
	ONLINE_STRUCT_FIELD(FUnlockAchievements::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUnlockAchievements::Params, AchievementIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUnlockAchievements::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FDisplayAchievementUI::Params)
	ONLINE_STRUCT_FIELD(FDisplayAchievementUI::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FDisplayAchievementUI::Params, AchievementId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FDisplayAchievementUI::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAchievementStateUpdated)
	ONLINE_STRUCT_FIELD(FAchievementStateUpdated, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAchievementStateUpdated, AchievementIds)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
