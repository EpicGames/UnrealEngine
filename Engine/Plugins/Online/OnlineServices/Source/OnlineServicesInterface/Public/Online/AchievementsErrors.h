// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineError.h"
#include "Online/OnlineErrorCode.h"

#define LOCTEXT_NAMESPACE "AchievementsErrors"

namespace UE::Online::Errors {
	UE_ONLINE_ERROR_CATEGORY(Achievements, Engine, 0x5, "Achievements")
	UE_ONLINE_ERROR(Achievements, AlreadyUnlocked, 1, TEXT("AlreadyUnlocked"), LOCTEXT("AlreadyUnlocked", "That achievement is already unlocked"))

/* UE::Online::Errors */ }

#undef LOCTEXT_NAMESPACE
