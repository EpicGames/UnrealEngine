// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Online/CoreOnline.h"
#include "Online/SchemaTypes.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }

namespace UE::Online {

class FOnlineError;

namespace LoginCredentialsType
{
ONLINESERVICESINTERFACE_API extern const FName Auto;
ONLINESERVICESINTERFACE_API extern const FName Password;
ONLINESERVICESINTERFACE_API extern const FName ExchangeCode;
ONLINESERVICESINTERFACE_API extern const FName PersistentAuth;
ONLINESERVICESINTERFACE_API extern const FName Developer;
ONLINESERVICESINTERFACE_API extern const FName RefreshToken;
ONLINESERVICESINTERFACE_API extern const FName AccountPortal;
ONLINESERVICESINTERFACE_API extern const FName ExternalAuth;
}

namespace ExternalLoginType
{
ONLINESERVICESINTERFACE_API extern const FName Epic;
ONLINESERVICESINTERFACE_API extern const FName SteamSessionTicket;
ONLINESERVICESINTERFACE_API extern const FName PsnIdToken;
ONLINESERVICESINTERFACE_API extern const FName XblXstsToken;
ONLINESERVICESINTERFACE_API extern const FName DiscordAccessToken;
ONLINESERVICESINTERFACE_API extern const FName GogSessionTicket;
ONLINESERVICESINTERFACE_API extern const FName NintendoIdToken;
ONLINESERVICESINTERFACE_API extern const FName NintendoNsaIdToken;
ONLINESERVICESINTERFACE_API extern const FName UplayAccessToken;
ONLINESERVICESINTERFACE_API extern const FName OpenIdAccessToken;
ONLINESERVICESINTERFACE_API extern const FName DeviceIdAccessToken;
ONLINESERVICESINTERFACE_API extern const FName AppleIdToken;
ONLINESERVICESINTERFACE_API extern const FName GoogleIdToken;
ONLINESERVICESINTERFACE_API extern const FName OculusUserIdNonce;
ONLINESERVICESINTERFACE_API extern const FName ItchioJwt;
ONLINESERVICESINTERFACE_API extern const FName ItchioKey;
ONLINESERVICESINTERFACE_API extern const FName EpicIdToken;
ONLINESERVICESINTERFACE_API extern const FName AmazonAccessToken;
}

namespace ExternalServerAuthTicketType
{
ONLINESERVICESINTERFACE_API extern const FName PsnAuthCode;
ONLINESERVICESINTERFACE_API extern const FName XblXstsToken;
}

namespace AccountAttributeData
{
ONLINESERVICESINTERFACE_API extern const FSchemaAttributeId DisplayName;
}

enum class ELoginStatus : uint8
{
	/** Player has not logged in or chosen a local profile. */
	NotLoggedIn,
	/** Player is using a local profile but is not logged in. */
	UsingLocalProfile,
	/** Player is logged in but may have reduced functionality with online services. */
	LoggedInReducedFunctionality,
	/** Player has been validated by the platform specific authentication service. */
	LoggedIn
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ELoginStatus Value);
ONLINESERVICESINTERFACE_API bool LexTryParseString(ELoginStatus& OutValue, const TCHAR* InStr);
ONLINESERVICESINTERFACE_API void LexFromString(ELoginStatus& OutValue, const TCHAR* InStr);

inline bool IsOnlineStatus(ELoginStatus LoginStatus)
{
	return LoginStatus == ELoginStatus::LoggedIn || LoginStatus == ELoginStatus::LoggedInReducedFunctionality;
}

enum class ERemoteAuthTicketAudience : uint8
{
	/** Generate a ticket appropriate for peer validation. */
	Peer,
	/**
	 * Generate a ticket appropriate for dedicated server validation.
	 * Depending on the platform, dedicated servers may have access to additional APIs used to verify tickets.
	 */
	DedicatedServer,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ERemoteAuthTicketAudience Value);
ONLINESERVICESINTERFACE_API bool LexTryParseString(ERemoteAuthTicketAudience& OutValue, const TCHAR* InStr);
ONLINESERVICESINTERFACE_API void LexFromString(ERemoteAuthTicketAudience& OutValue, const TCHAR* InStr);

/** Some auth interfaces have more than one method for providing credentials when linking to an
 *  external account. An example usage is when the auth interface can provide a token for linking
 *  the local hardware device as the primary method before falling back to a token linked to the
 *  users online account.
 */
enum class EExternalAuthTokenMethod : uint8
{
	/** Acquire an external auth token using the primary method provided by the auth interface. */
	Primary,
	/** Acquire an external auth token using the secondary method provided by the auth interface. */
	Secondary,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EExternalAuthTokenMethod Value);
ONLINESERVICESINTERFACE_API bool LexTryParseString(EExternalAuthTokenMethod& OutValue, const TCHAR* InStr);
ONLINESERVICESINTERFACE_API void LexFromString(EExternalAuthTokenMethod& OutValue, const TCHAR* InStr);

struct FAccountInfo
{
	/** The account id for the user which represents the user's online platform account. */
	FAccountId AccountId;
	/** The platform user id associated with the online user. */
	FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
	/** Login status */
	ELoginStatus LoginStatus = ELoginStatus::NotLoggedIn;
	/** Additional account attributes. */
	TMap<FSchemaAttributeId, FSchemaVariant> Attributes;
};

/** Token appropriate for allowing a trusted server to verify authentication.
 *  In most implementations this token will be an OpenId token.
 */
struct FExternalAuthToken
{
	FName Type;
	FString Data;
};

/** Ticket appropriate for allowing a trusted server to access service APIs on the users behalf. */
struct FExternalServerAuthTicket
{
	FName Type;
	FString Data;
};

/** Token used during Login operation. The data provided as the credentials token varies based
 * on the type of login being performed.
 */
using FCredentialsToken = TVariant<FString, FExternalAuthToken>;

struct FVerifiedAuthSession
{
	FVerifiedAuthSessionId SessionId;
	FAccountId RemoteAccountId;
	double CreationTime;
};

struct FVerifiedAuthTicket
{
	FName Type;
	FString Data;
};

struct FAuthLogin
{
	static constexpr TCHAR Name[] = TEXT("Login");

	struct Params
	{
		/** The PlatformUserId of the Local User making the request. */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		FName CredentialsType;
		FString CredentialsId;
		TVariant<FString, FExternalAuthToken> CredentialsToken;
		TArray<FString> Scopes;
	};

	struct Result
	{
		TSharedRef<FAccountInfo> AccountInfo;
	};
};

struct FAuthLogout
{
	static constexpr TCHAR Name[] = TEXT("Logout");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
		/** Whether to remove persistent credentials on logout. */
		bool bDestroyAuth = false;
	};

	struct Result
	{
	};
};

struct FAuthCreateAccount
{
	static constexpr TCHAR Name[] = TEXT("CreateAccount");

	struct Params
	{
		/** The PlatformUserId of the Local User making the request. */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		/** The id associated with external auth data used to continue a previous login attempt. */
		FLoginContinuationId ContinuationId;
	};

	struct Result
	{
		TSharedRef<const FAccountInfo> AccountInfo;
	};
};

struct FAuthLinkAccount
{
	static constexpr TCHAR Name[] = TEXT("LinkAccount");

	struct Params
	{
		/** The PlatformUserId of the Local User making the request. */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		/** The id associated with external auth data used to continue a previous login attempt. */
		FLoginContinuationId ContinuationId;
		/** Implementation dependent tags which affect how the account is linked. */
		TArray<FName> Tags;
	};

	struct Result
	{
		TSharedRef<const FAccountInfo> AccountInfo;
	};
};

struct FAuthModifyAccountAttributes
{
	static constexpr TCHAR Name[] = TEXT("ModifyAccountAttributes");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
		/** New or changed attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> MutatedAttributes;
		/** Attributes to be cleared. */
		TSet<FSchemaAttributeId> ClearedAttributes;
	};

	struct Result
	{
	};
};

struct FAuthQueryExternalServerAuthTicket
{
	static constexpr TCHAR Name[] = TEXT("QueryExternalServerAuthTicket");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		FExternalServerAuthTicket ExternalServerAuthTicket;
	};
};

struct FAuthQueryExternalAuthToken
{
	static constexpr TCHAR Name[] = TEXT("QueryExternalAuthToken");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
		/** The method of external auth to provide. */
		EExternalAuthTokenMethod Method = EExternalAuthTokenMethod::Primary;
		/** The Relying Party for the external auth token. */
		FString RelyingParty;
	};

	struct Result
	{
		FExternalAuthToken ExternalAuthToken;
	};
};

struct FAuthQueryVerifiedAuthTicket
{
	static constexpr TCHAR Name[] = TEXT("QueryVerifiedAuthTicket");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
		/** The intended purpose of the auth ticket. */
		ERemoteAuthTicketAudience Audience = ERemoteAuthTicketAudience::Peer;
	};

	struct Result
	{
		/** Local ticket id used to reference the ticket in further operations. */
		FVerifiedAuthTicketId VerifiedAuthTicketId;
		/** Ticket used to begin a verified auth session with a remote host. */
		FVerifiedAuthTicket VerifiedAuthTicket;
	};
};

struct FAuthCancelVerifiedAuthTicket
{
	static constexpr TCHAR Name[] = TEXT("CancelVerifiedAuthTicket");

	struct Params
	{
		/** The online account id of the Local User making the request. */
		FAccountId LocalAccountId;
		/** Local ticket id used to reference the verified auth ticket. */
		FVerifiedAuthTicketId VerifiedAuthTicketId;
	};

	struct Result
	{
	};
};

struct FAuthBeginVerifiedAuthSession
{
	static constexpr TCHAR Name[] = TEXT("BeginVerifiedAuthSession");

	struct Params
	{
		/** The remote user for which to start the verified auth session. */
		FAccountId RemoteAccountId;
		/** The ticket used to verify the identity of the user. */
		FVerifiedAuthTicket Ticket;
	};

	struct Result
	{
		FVerifiedAuthSession Session;
	};
};

struct FAuthEndVerifiedAuthSession
{
	static constexpr TCHAR Name[] = TEXT("EndVerifiedAuthSession");

	struct Params
	{
		FVerifiedAuthSessionId SessionId;
	};

	struct Result
	{
	};
};

struct FAuthGetLocalOnlineUserByOnlineAccountId
{
	static constexpr TCHAR Name[] = TEXT("GetLocalOnlineUserByOnlineAccountId");

	struct Params
	{
		/** Account Id of the Local User making the request */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		TSharedRef<FAccountInfo> AccountInfo;
	};
};

struct FAuthGetLocalOnlineUserByPlatformUserId
{
	static constexpr TCHAR Name[] = TEXT("GetLocalOnlineUserByPlatformUserId");

	struct Params
	{
		/** The PlatformUserId of the Local User making the request. */
		FPlatformUserId PlatformUserId;
	};

	struct Result
	{
		TSharedRef<FAccountInfo> AccountInfo;
	};
};

struct FAuthGetAllLocalOnlineUsers
{
	static constexpr TCHAR Name[] = TEXT("GetAllLocalOnlineUsers");

	struct Params
	{
	};

	struct Result
	{
		TArray<TSharedRef<FAccountInfo>> AccountInfo;
	};
};

struct FAuthGetLinkAccountContinuationId
{
	static constexpr TCHAR Name[] = TEXT("GetLinkAccountContinuationId");

	struct Params
	{
		/** The PlatformUserId of the Local User making the request. */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
	};

	struct Result
	{
		/** The continuation id which identifies the external auth to be linked. */
		FLoginContinuationId ContinuationId;
	};
};

struct FAuthGetRelyingParty
{
	static constexpr TCHAR Name[] = TEXT("GetRelyingParty");

	struct Params
	{
	};

	struct Result
	{
		FString RelyingParty;
	};
};

/** Struct for LoginStatusChanged event */
struct FAuthLoginStatusChanged
{
	/* The affected account. */
	TSharedRef<FAccountInfo> AccountInfo;
	/* The new login status. */
	ELoginStatus LoginStatus = ELoginStatus::NotLoggedIn;
};

/** Struct for PendingAuthExpiration event */
struct FAuthPendingAuthExpiration
{
	/* The affected account. */
	TSharedRef<FAccountInfo> AccountInfo;
};

/** Struct for AccountAttributesChanged event */
struct FAuthAccountAttributesChanged
{
	/* The affected account. */
	TSharedRef<FAccountInfo> AccountInfo;
	/* Added attributes and their values. */
	TMap<FSchemaAttributeId, FSchemaVariant> AddedAttributes;
	/* Removed attributes with their previous values. */
	TMap<FSchemaAttributeId, FSchemaVariant> RemovedAttributes;
	/* Changed attributes with their old and new values. */
	TMap<FSchemaAttributeId, TPair<FSchemaVariant, FSchemaVariant>> ChangedAttributes;
};

class IAuth
{
public:
	virtual ~IAuth() = default;

	/**
	 * Authenticate a local user. If necessary, retrieve and maintain an access token for the
	 * duration of the auth session.
	 * 
	 * Returns Errors::InvalidUser when no user has been associated with the provided credentials.
	 * GetLinkAccountContinuationId can then be used to retrieve a continuance token which can be
	 * passed to either CreateAccount or LinkAccount.
	 */
	virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) = 0;

	/**
	 * Concludes the auth session for the local user.
	 */
	virtual TOnlineAsyncOpHandle<FAuthLogout> Logout(FAuthLogout::Params&& Params) = 0;

	/**
	* Create a new account and link it to an external auth method using a continuation token from a previous login attempt.
	*/
	virtual TOnlineAsyncOpHandle<FAuthCreateAccount> CreateAccount(FAuthCreateAccount::Params&& Params) = 0;

	/**
	 * Link a logged in account to an external auth method using a continuation token from a previous login attempt.
	 */
	virtual TOnlineAsyncOpHandle<FAuthLinkAccount> LinkAccount(FAuthLinkAccount::Params&& Params) = 0;

	/**
	 * Modify attributes associated with an authenticated account. Signals OnAccountAttributesChanged on completion.
	 */
	virtual TOnlineAsyncOpHandle<FAuthModifyAccountAttributes> ModifyAccountAttributes(FAuthModifyAccountAttributes::Params&& Params) = 0;

	/**
	 * Queries a ticket which is appropriate for making server-to-server calls on behalf of the
	 * signed in user. Tickets are intended to be single use - users must call the API again to
	 * retrieve a new ticket when making repeated calls which use a ticket.
	 */
	virtual TOnlineAsyncOpHandle<FAuthQueryExternalServerAuthTicket> QueryExternalServerAuthTicket(FAuthQueryExternalServerAuthTicket::Params&& Params) = 0;

	/**
	 * Retrieves a token appropriate for linking the service account with a service account of a
	 * different service type. On most platforms this will return an OpenId token. May return a
	 * cached token, but will handle refreshing if necessary.
	 */
	virtual TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params) = 0;

	/**
	 * Retrieves a ticket which is used to create a verified authentication session on a remote
	 * client. When establishing a verified authentication session the user must always call
	 * QueryVerifiedAuthTicket to retrieve a new ticket.
	 * 
	 * To prevent the transmission of PII to a non-trusted destination an audience enumeration
	 * is required which may generate an empty ticket in the case of some peer-to-peer
	 * implementations. On those platforms user authentication is handled implicitly through the
	 * peer-to-peer networking model.
	 *
	 * It is the responsibility of the game code to provide an audience enumeration value
	 * appropriate for the usage.
	 */
	virtual TOnlineAsyncOpHandle<FAuthQueryVerifiedAuthTicket> QueryVerifiedAuthTicket(FAuthQueryVerifiedAuthTicket::Params&& Params) = 0;

	/**
	 * Cancels the ticket associated with a verified auth session. When the session created using
	 * a verified auth ticket is no longer in use, CancelVerifiedAuthTicket must be called to clean
	 * up any resources associated with the ticket.
	 */
	virtual TOnlineAsyncOpHandle<FAuthCancelVerifiedAuthTicket> CancelVerifiedAuthTicket(FAuthCancelVerifiedAuthTicket::Params&& Params) = 0;

	/**
	 * Starts a verified auth session for a remote user. Depending on implementation
	 * RemoteAuthTicket will be used to verify the identity of RemoteUserId. The intended usage is
	 * to start a verified auth session for any client connecting any other client or server. Having
	 * a verified auth session handles identity verification in both peer-to-peer and client-server
	 * topologies. In peer-to-peer topologies authentication may not occur due to implicit trust
	 * established by the peer-to-peer network.
	 *
	 * Creating a verified auth session will generate a FVerifiedAuthSessionId. This id is a
	 * cryptographically secure random id intended to be transmitted between the established peers
	 * to refer to the remote auth session. This id is needed when the client changes how they
	 * communicate with the server. An example is connecting to a server reservation beacon to
	 * reserve a spot before traveling to the server url using a different connection.
	 */
	virtual TOnlineAsyncOpHandle<FAuthBeginVerifiedAuthSession> BeginVerifiedAuthSession(FAuthBeginVerifiedAuthSession::Params&& Params) = 0;

	/**
	 * Clean up the remote verified auth session and handle any required book-keeping.
	 */
	virtual TOnlineAsyncOpHandle<FAuthEndVerifiedAuthSession> EndVerifiedAuthSession(FAuthEndVerifiedAuthSession::Params&& Params) = 0;

	/**
	 * Retrieve a logged in user account.
	 */
	virtual TOnlineResult<FAuthGetLocalOnlineUserByOnlineAccountId> GetLocalOnlineUserByOnlineAccountId(FAuthGetLocalOnlineUserByOnlineAccountId::Params&& Params) const = 0;

	/**
	 * Retrieve a logged in user account.
	 */
	virtual TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> GetLocalOnlineUserByPlatformUserId(FAuthGetLocalOnlineUserByPlatformUserId::Params&& Params) const = 0;

	/**
	 * Retrieve all logged in user accounts.
	 */
	virtual TOnlineResult<FAuthGetAllLocalOnlineUsers> GetAllLocalOnlineUsers(FAuthGetAllLocalOnlineUsers::Params&& Params) const = 0;

	/**
	 * Retrieve the continuation id from a previous login attempt which used an external auth token and failed with Errors::InvalidUser.
	 */
	virtual TOnlineResult<FAuthGetLinkAccountContinuationId> GetLinkAccountContinuationId(FAuthGetLinkAccountContinuationId::Params&& Params) const = 0;

	/**
	 * Retrieve the relying party to use when fetching an external token to authenticate with this interface.
	 */
	virtual TOnlineResult<FAuthGetRelyingParty> GetRelyingParty(FAuthGetRelyingParty::Params&& Params) const = 0;

	/**
	 * Triggered when the login status for a logged in user changes.
	 */
	virtual TOnlineEvent<void(const FAuthLoginStatusChanged&)> OnLoginStatusChanged() = 0;

	/**
	 * Triggered when the auth token will expire soon. Services which allow login using an external
	 * auth token may trigger this notification when an updated external auth token is required to
	 * maintain the auth session without interruption.
	 */
	virtual TOnlineEvent<void(const FAuthPendingAuthExpiration&)> OnPendingAuthExpiration() = 0;

	/**
	 * Triggered when the additional attributes associated with an authenticated account are changed.
	 */
	virtual TOnlineEvent<void(const FAuthAccountAttributesChanged&)> OnAccountAttributesChanged() = 0;

	/**
	 * Helper for querying the login status of a local user.
	 */
	virtual bool IsLoggedIn(const FAccountId& AccountId) const = 0;

	/**
	 * Helper for querying the login status of a local user by platform user id.
	 */
	virtual bool IsLoggedIn(const FPlatformUserId& PlatformUserId) const = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAccountInfo)
	ONLINE_STRUCT_FIELD(FAccountInfo, AccountId),
	ONLINE_STRUCT_FIELD(FAccountInfo, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAccountInfo, LoginStatus),
	ONLINE_STRUCT_FIELD(FAccountInfo, Attributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalAuthToken)
	ONLINE_STRUCT_FIELD(FExternalAuthToken, Type),
	ONLINE_STRUCT_FIELD(FExternalAuthToken, Data)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalServerAuthTicket)
	ONLINE_STRUCT_FIELD(FExternalServerAuthTicket, Type),
	ONLINE_STRUCT_FIELD(FExternalServerAuthTicket, Data)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FVerifiedAuthSession)
	ONLINE_STRUCT_FIELD(FVerifiedAuthSession, SessionId),
	ONLINE_STRUCT_FIELD(FVerifiedAuthSession, RemoteAccountId),
	ONLINE_STRUCT_FIELD(FVerifiedAuthSession, CreationTime)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FVerifiedAuthTicket)
	ONLINE_STRUCT_FIELD(FVerifiedAuthTicket, Type),
	ONLINE_STRUCT_FIELD(FVerifiedAuthTicket, Data)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogin::Params)
	ONLINE_STRUCT_FIELD(FAuthLogin::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthLogin::Params, CredentialsType),
	ONLINE_STRUCT_FIELD(FAuthLogin::Params, CredentialsId),
	ONLINE_STRUCT_FIELD(FAuthLogin::Params, CredentialsToken),
	ONLINE_STRUCT_FIELD(FAuthLogin::Params, Scopes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogin::Result)
	ONLINE_STRUCT_FIELD(FAuthLogin::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogout::Params)
	ONLINE_STRUCT_FIELD(FAuthLogout::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAuthLogout::Params, bDestroyAuth)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLogout::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthCreateAccount::Params)
	ONLINE_STRUCT_FIELD(FAuthCreateAccount::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthCreateAccount::Params, ContinuationId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthCreateAccount::Result)
	ONLINE_STRUCT_FIELD(FAuthCreateAccount::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLinkAccount::Params)
	ONLINE_STRUCT_FIELD(FAuthLinkAccount::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FAuthLinkAccount::Params, ContinuationId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLinkAccount::Result)
	ONLINE_STRUCT_FIELD(FAuthLinkAccount::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthModifyAccountAttributes::Params)
	ONLINE_STRUCT_FIELD(FAuthModifyAccountAttributes::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAuthModifyAccountAttributes::Params, MutatedAttributes),
	ONLINE_STRUCT_FIELD(FAuthModifyAccountAttributes::Params, ClearedAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthModifyAccountAttributes::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryExternalServerAuthTicket::Params)
	ONLINE_STRUCT_FIELD(FAuthQueryExternalServerAuthTicket::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryExternalServerAuthTicket::Result)
	ONLINE_STRUCT_FIELD(FAuthQueryExternalServerAuthTicket::Result, ExternalServerAuthTicket)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryExternalAuthToken::Params)
	ONLINE_STRUCT_FIELD(FAuthQueryExternalAuthToken::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAuthQueryExternalAuthToken::Params, Method)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryExternalAuthToken::Result)
	ONLINE_STRUCT_FIELD(FAuthQueryExternalAuthToken::Result, ExternalAuthToken)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryVerifiedAuthTicket::Params)
	ONLINE_STRUCT_FIELD(FAuthQueryVerifiedAuthTicket::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAuthQueryVerifiedAuthTicket::Params, Audience)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthQueryVerifiedAuthTicket::Result)
	ONLINE_STRUCT_FIELD(FAuthQueryVerifiedAuthTicket::Result, VerifiedAuthTicket)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthCancelVerifiedAuthTicket::Params)
	ONLINE_STRUCT_FIELD(FAuthCancelVerifiedAuthTicket::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAuthCancelVerifiedAuthTicket::Params, VerifiedAuthTicketId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthCancelVerifiedAuthTicket::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthBeginVerifiedAuthSession::Params)
	ONLINE_STRUCT_FIELD(FAuthBeginVerifiedAuthSession::Params, RemoteAccountId),
	ONLINE_STRUCT_FIELD(FAuthBeginVerifiedAuthSession::Params, Ticket)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthBeginVerifiedAuthSession::Result)
	ONLINE_STRUCT_FIELD(FAuthBeginVerifiedAuthSession::Result, Session)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthEndVerifiedAuthSession::Params)
	ONLINE_STRUCT_FIELD(FAuthEndVerifiedAuthSession::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthEndVerifiedAuthSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLocalOnlineUserByOnlineAccountId::Params)
	ONLINE_STRUCT_FIELD(FAuthGetLocalOnlineUserByOnlineAccountId::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLocalOnlineUserByOnlineAccountId::Result)
	ONLINE_STRUCT_FIELD(FAuthGetLocalOnlineUserByOnlineAccountId::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLocalOnlineUserByPlatformUserId::Params)
	ONLINE_STRUCT_FIELD(FAuthGetLocalOnlineUserByPlatformUserId::Params, PlatformUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLocalOnlineUserByPlatformUserId::Result)
	ONLINE_STRUCT_FIELD(FAuthGetLocalOnlineUserByPlatformUserId::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetAllLocalOnlineUsers::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetAllLocalOnlineUsers::Result)
	ONLINE_STRUCT_FIELD(FAuthGetAllLocalOnlineUsers::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLinkAccountContinuationId::Params)
	ONLINE_STRUCT_FIELD(FAuthGetLinkAccountContinuationId::Params, PlatformUserId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetLinkAccountContinuationId::Result)
	ONLINE_STRUCT_FIELD(FAuthGetLinkAccountContinuationId::Result, ContinuationId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetRelyingParty::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthGetRelyingParty::Result)
	ONLINE_STRUCT_FIELD(FAuthGetRelyingParty::Result, RelyingParty)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthLoginStatusChanged)
	ONLINE_STRUCT_FIELD(FAuthLoginStatusChanged, AccountInfo),
	ONLINE_STRUCT_FIELD(FAuthLoginStatusChanged, LoginStatus)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthPendingAuthExpiration)
	ONLINE_STRUCT_FIELD(FAuthPendingAuthExpiration, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthAccountAttributesChanged)
	ONLINE_STRUCT_FIELD(FAuthAccountAttributesChanged, AccountInfo),
	ONLINE_STRUCT_FIELD(FAuthAccountAttributesChanged, AddedAttributes),
	ONLINE_STRUCT_FIELD(FAuthAccountAttributesChanged, RemovedAttributes),
	ONLINE_STRUCT_FIELD(FAuthAccountAttributesChanged, ChangedAttributes)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
