// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineError.h"
#include "Online/OnlineErrorCode.h"

#define LOCTEXT_NAMESPACE "AuthErrors"

namespace UE::Online::Errors {
	UE_ONLINE_ERROR_CATEGORY(Auth, Engine, 0x4, "Auth")
	UE_ONLINE_ERROR(Auth, AlreadyLoggedIn, 1, TEXT("AlreadyLoggedIn"), LOCTEXT("AlreadyLoggedIn", "That user is already logged in"))

/* UE::Online::Errors */ }

#undef LOCTEXT_NAMESPACE
