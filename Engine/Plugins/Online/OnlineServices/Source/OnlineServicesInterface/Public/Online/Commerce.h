// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

/* Unique identifier for an offer */
using FOfferId = FString;

/* Unique identifier for an entitlement */
using FEntitlementId = FString;
	
enum class EOfferType : uint8
{
	Nonconsumable,
	Consumable,
	Subscription,
	Unknown
};

ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EOfferType Status);
ONLINESERVICESINTERFACE_API void LexFromString(EOfferType& OutStatus, const TCHAR* InStr);

// Contains all the information required to display an offer to a user.
struct FOffer
{
	/* The Id of this offer */
	FOfferId OfferId;
	/* Localized title of this offer */
	FText Title;
	/* Localized description of this offer */
	FText Description;
	/* Localized long description of this offer */
	FText LongDescription;
	/* The maximum number of times the offer can be purchased (-1 for no limit) */
	int32 PurchaseLimit;
	/* Currency Code for this offer */
	FString CurrencyCode;
	/* Regular non-sale price as text for display */
	FText FormattedRegularPrice;
	/* Regular non-sale price in numeric form for comparison/sorting */
	uint64 RegularPrice;
	/* Final-Pricing (Post-Sales/Discounts) as text for display */
	FText FormattedPrice;
	/* Final-Price (Post-Sales/Discounts) in numeric form for comparison/sorting */
	uint64 Price;
	/* Number of decimal places in Price/RegularPrice */
	int32 PriceDecimalPoint;
	/* Date the offer was first available/will be available */
	TOptional<FDateTime> ReleaseDate;
	/* Date the information is no longer valid (offer no longer available or discount expires) */
	TOptional<FDateTime> ExpirationDate;
	/* Additional implementation specified fields. See the implementation specific documentation for what this will contain */
	TMap<FString, FString> AdditionalData;
	// TODO: Expose offer image data? additional subscription related fields? Offer type (consumable, subscription, dlc, etc)?
};

struct FCommerceQueryOffers
{
	static constexpr TCHAR Name[] = TEXT("QueryOffers");
	
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Optional catalog namespace to use for this query. Can be left blank to use the "default" namespace */
		FString OverrideCatalogNamespace;
	};

	struct Result
	{

	};
};

// Query the offers matching specified offer ids
struct FCommerceQueryOffersById
{
	static constexpr TCHAR Name[] = TEXT("QueryOffersById");

	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* List of offer ids */
		TArray<FOfferId> OfferIds;
	};

	struct Result
	{
		/* List of offer ids that were found */
		TArray<FOfferId> OfferIds;
	};
};

// Get the complete list of cached offers
struct FCommerceGetOffers
{
	static constexpr TCHAR Name[] = TEXT("GetOffers");
	
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/* List of offers */
		TArray<FOffer> Offers;
	};
};

// Get the offers for a list of offer ids
struct FCommerceGetOffersById
{
	static constexpr TCHAR Name[] = TEXT("GetOffersById");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* List of offer ids */
		TArray<FOfferId> OfferIds;
	};

	struct Result
	{
		/* List of offers */
		TArray<FOffer> Offers;
	};
};

//Bring up the platform store ui
struct FCommerceShowStoreUI
{

	static constexpr TCHAR Name[] = TEXT("ShowStoreUI");

	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
	};

	struct Result
	{

	};
};

struct FPurchaseOffer
{
	/* The Id of the offer */
	FOfferId OfferId;
	/* Quantity of the offer to purchase */
	int32 Quantity;
};

// Initiate the checkout process for purchasing one or more offers
struct FCommerceCheckout
{
	static constexpr TCHAR Name[] = TEXT("Checkout");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Offers to purchase */
		TArray<FPurchaseOffer> Offers;
	};

	struct Result
	{
		/* If available, the transaction id for the completed purchase */
		TOptional<FString> TransactionId;
	};
};

struct FEntitlement
{
	/* Id of the entitlement */
	FEntitlementId EntitlementId;
	/* Type of the entitlement */
	FName EntitlementType;
	/* The Id of the product */
	FString ProductId;
	/* True if the entitlement has been marked as redeemed */
	bool bRedeemed;
	/* Quantity */
	int32 Quantity;
	/* If set, specifies when the entitlement was acquired */
	TOptional<FDateTime> AcquiredDate;
	/* If set, specifies when the entitlement will expire */
	TOptional<FDateTime> ExpiryDate;
};

// Retrieve the list of entitlements from a transaction id returned from a checkout call
struct FCommerceQueryTransactionEntitlements
{

	static constexpr TCHAR Name[] = TEXT("QueryTransactionEntitlements");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Transaction id for a completed purchase */
		FString TransactionId;
	};

	struct Result
	{
		/* Array of entitlements granted in the transaction */
		TArray<FEntitlement> Entitlements;
	};
};

// Get the list of all entitlements
struct FCommerceQueryEntitlements
{
	static constexpr TCHAR Name[] = TEXT("QueryEntitlements");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Whether or not to fetch redeemed entitlements */
		bool bIncludeRedeemed;
	};

	struct Result
	{
		/* Empty, complete entitlement list can be retrieved via GetEntitlements */
	};
};

// Get the complete list of cached entitlements
struct FCommerceGetEntitlements
{
	static constexpr TCHAR Name[] = TEXT("GetEntitlements");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/* Array of entitlements owned by the user */
		TArray<FEntitlement> Entitlements;
	};
};

// Mark an entitlement as redeemed.Used when there is no external service managing redemption of entitlements or the redemption must be done on the client side.
struct FCommerceRedeemEntitlement
{
	static constexpr TCHAR Name[] = TEXT("RedeemEntitlement");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Entitlement id to mark as redeemed */
		FEntitlementId EntitlementId;
		/* Quantity to consume */
		int32 Quantity;
	};

	struct Result
	{

	};
};

// Get a token suitable for making service to service Commerce related calls
struct FCommerceRetrieveS2SToken
{
	static constexpr TCHAR Name[] = TEXT("RetrieveS2SToken");
		
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* Implementation specific, used to specify the type of token to retrieve. See implementation specific documentation for valid values */
		FString TokenType;
	};

	struct Result
	{
		/* Token or code that can be used for service to service calls. See implementation specific documentation for details */
		TArray<uint8> Token;
	};
};

// event
struct FCommerceOnPurchaseComplete
{
public:
	
	/* Local user */
	FAccountId LocalAccountId;
	
	/* If available, the transaction id for the completed purchase */
	TOptional<FString> TransactionId;

};

class ICommerce
{
public:
	virtual ~ICommerce() = default;

	/* Query the list of all offers for the target user */
	virtual TOnlineAsyncOpHandle<FCommerceQueryOffers> QueryOffers(FCommerceQueryOffers::Params&& Params) = 0;

	/* Query the offers matching specified offer ids */
	virtual TOnlineAsyncOpHandle<FCommerceQueryOffersById> QueryOffersById(FCommerceQueryOffersById::Params&& Params) = 0;

	/* Get the complete list of cached offers */
	virtual TOnlineResult<FCommerceGetOffers> GetOffers(FCommerceGetOffers::Params&& Params) = 0;

	/* Get the offers for a list of offer ids */
	virtual TOnlineResult<FCommerceGetOffersById> GetOffersById(FCommerceGetOffersById::Params&& Params) = 0;

	/* Bring up the platform store ui */
	virtual TOnlineAsyncOpHandle<FCommerceShowStoreUI> ShowStoreUI(FCommerceShowStoreUI::Params&& Params) = 0;

	/* Initiate the checkout process for purchasing one or more offers */
	virtual TOnlineAsyncOpHandle<FCommerceCheckout> Checkout(FCommerceCheckout::Params&& Params) = 0;

	/* Retrieve the list of entitlements from a transaction id returned from a checkout call */
	virtual TOnlineAsyncOpHandle<FCommerceQueryTransactionEntitlements> QueryTransactionEntitlements(FCommerceQueryTransactionEntitlements::Params&& Params) = 0;

	/* Get the list of all entitlements */
	virtual TOnlineAsyncOpHandle<FCommerceQueryEntitlements> QueryEntitlements(FCommerceQueryEntitlements::Params&& Params) = 0;

	/* Get the complete list of cached entitlements */
	virtual TOnlineResult<FCommerceGetEntitlements> GetEntitlements(FCommerceGetEntitlements::Params&& Params) = 0;

	/* Mark an entitlement as redeemed. Used when there is no external service managing redemption of entitlements or the redemption must be done on the client side */
	virtual TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params) = 0;

	/* Get a token suitable for making service to service Commerce related calls */
	virtual TOnlineAsyncOpHandle<FCommerceRetrieveS2SToken> RetrieveS2SToken(FCommerceRetrieveS2SToken::Params&& Params) = 0;

	/* Fires whenever a purchase is completed */
	virtual TOnlineEvent<void(const FCommerceOnPurchaseComplete&)> OnPurchaseCompleted() = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FOffer)
	ONLINE_STRUCT_FIELD(FOffer, OfferId),
	ONLINE_STRUCT_FIELD(FOffer, Title),
	ONLINE_STRUCT_FIELD(FOffer, Description),
	ONLINE_STRUCT_FIELD(FOffer, LongDescription),
	ONLINE_STRUCT_FIELD(FOffer, PurchaseLimit),
	ONLINE_STRUCT_FIELD(FOffer, CurrencyCode),
	ONLINE_STRUCT_FIELD(FOffer, FormattedRegularPrice),
	ONLINE_STRUCT_FIELD(FOffer, RegularPrice),
	ONLINE_STRUCT_FIELD(FOffer, FormattedPrice),
	ONLINE_STRUCT_FIELD(FOffer, Price),
	ONLINE_STRUCT_FIELD(FOffer, PriceDecimalPoint),
	ONLINE_STRUCT_FIELD(FOffer, ReleaseDate),
	ONLINE_STRUCT_FIELD(FOffer, ExpirationDate),
	ONLINE_STRUCT_FIELD(FOffer, AdditionalData)
END_ONLINE_STRUCT_META()


BEGIN_ONLINE_STRUCT_META(FPurchaseOffer)
	ONLINE_STRUCT_FIELD(FPurchaseOffer, OfferId),
	ONLINE_STRUCT_FIELD(FPurchaseOffer, Quantity)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FEntitlement)
	ONLINE_STRUCT_FIELD(FEntitlement, EntitlementId),
	ONLINE_STRUCT_FIELD(FEntitlement, EntitlementType),
	ONLINE_STRUCT_FIELD(FEntitlement, ProductId),
	ONLINE_STRUCT_FIELD(FEntitlement, bRedeemed),
	ONLINE_STRUCT_FIELD(FEntitlement, Quantity),
	ONLINE_STRUCT_FIELD(FEntitlement, AcquiredDate),
	ONLINE_STRUCT_FIELD(FEntitlement, ExpiryDate)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceQueryOffers::Params)
	ONLINE_STRUCT_FIELD(FCommerceQueryOffers::Params, LocalAccountId)
END_ONLINE_STRUCT_META()
	BEGIN_ONLINE_STRUCT_META(FCommerceQueryOffers::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceQueryOffersById::Params)
	ONLINE_STRUCT_FIELD(FCommerceQueryOffersById::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceQueryOffersById::Params, OfferIds)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceQueryOffersById::Result)
	ONLINE_STRUCT_FIELD(FCommerceQueryOffersById::Result, OfferIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceGetOffers::Params)
	ONLINE_STRUCT_FIELD(FCommerceGetOffers::Params, LocalAccountId)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceGetOffers::Result)
	ONLINE_STRUCT_FIELD(FCommerceGetOffers::Result, Offers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceGetOffersById::Params)
	ONLINE_STRUCT_FIELD(FCommerceGetOffersById::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceGetOffersById::Params, OfferIds)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceGetOffersById::Result)
ONLINE_STRUCT_FIELD(FCommerceGetOffersById::Result, Offers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceShowStoreUI::Params)
	ONLINE_STRUCT_FIELD(FCommerceShowStoreUI::Params, LocalAccountId)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceShowStoreUI::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceCheckout::Params)
	ONLINE_STRUCT_FIELD(FCommerceCheckout::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceCheckout::Params, Offers)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceCheckout::Result)
	ONLINE_STRUCT_FIELD(FCommerceCheckout::Result, TransactionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceQueryTransactionEntitlements::Params)
	ONLINE_STRUCT_FIELD(FCommerceQueryTransactionEntitlements::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceQueryTransactionEntitlements::Params, TransactionId)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceQueryTransactionEntitlements::Result)
	ONLINE_STRUCT_FIELD(FCommerceQueryTransactionEntitlements::Result, Entitlements)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceQueryEntitlements::Params)
	ONLINE_STRUCT_FIELD(FCommerceQueryEntitlements::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceQueryEntitlements::Params, bIncludeRedeemed)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceQueryEntitlements::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceGetEntitlements::Params)
	ONLINE_STRUCT_FIELD(FCommerceGetEntitlements::Params, LocalAccountId)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceGetEntitlements::Result)
	ONLINE_STRUCT_FIELD(FCommerceGetEntitlements::Result, Entitlements)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceRedeemEntitlement::Params)
	ONLINE_STRUCT_FIELD(FCommerceRedeemEntitlement::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceRedeemEntitlement::Params, EntitlementId),
	ONLINE_STRUCT_FIELD(FCommerceRedeemEntitlement::Params, Quantity)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceRedeemEntitlement::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceRetrieveS2SToken::Params)
	ONLINE_STRUCT_FIELD(FCommerceRetrieveS2SToken::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceRetrieveS2SToken::Params, TokenType)
END_ONLINE_STRUCT_META()
BEGIN_ONLINE_STRUCT_META(FCommerceRetrieveS2SToken::Result)
	ONLINE_STRUCT_FIELD(FCommerceRetrieveS2SToken::Result, Token)

END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCommerceOnPurchaseComplete)
	ONLINE_STRUCT_FIELD(FCommerceOnPurchaseComplete, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCommerceOnPurchaseComplete, TransactionId)
END_ONLINE_STRUCT_META()


} // namespace Meta
} // namespace UE::Online
