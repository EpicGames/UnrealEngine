// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }
struct FScriptContainerElement;

namespace UE::Online {

enum class EOnlineServicesConnectionStatus : uint8
{
	/** Connected to the online servers */
	Connected,
	/** Successfully disconnected from the online servers */
	NotConnected
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EOnlineServicesConnectionStatus Status);
ONLINESERVICESINTERFACE_API void LexFromString(EOnlineServicesConnectionStatus& OutStatus, const TCHAR* InStr);

struct FGetConnectionStatus
{
	static constexpr TCHAR Name[] = TEXT("GetConnectionStatus");

	/** Input struct for Connectivity::FGetConnectionStatus */
	struct Params
	{
		/** Optional. If set, the name of the specific service being accessed (service dependent). */
		FString ServiceName;
	};

	/** Output struct for Connectivity::FGetConnectionStatus */
	struct Result
	{
		/** Connection status */
		EOnlineServicesConnectionStatus Status;
	};
};

/** Struct for ConnectionStatusChanged event */
struct FConnectionStatusChanged
{
	/** The name of the service that is reporting (service dependent). Can be empty. */
	FString ServiceName;
	/** Previous connection status */
	EOnlineServicesConnectionStatus PreviousStatus;
	/** Current connection status */
	EOnlineServicesConnectionStatus CurrentStatus;
};

class IConnectivity
{
public:
	virtual ~IConnectivity() = default;

	/**
		* Accessor for the connection status
		*/
	virtual TOnlineResult<FGetConnectionStatus> GetConnectionStatus(FGetConnectionStatus::Params&& Params) = 0;

	/**
		* Event triggered when the connection status changes
		*/
	virtual TOnlineEvent<void(const FConnectionStatusChanged&)> OnConnectionStatusChanged() = 0;
};

namespace Meta {
// TODO: Move to OnlineServices_Meta.inl file?

BEGIN_ONLINE_STRUCT_META(FGetConnectionStatus::Params)
	ONLINE_STRUCT_FIELD(FGetConnectionStatus::Params, ServiceName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetConnectionStatus::Result)
	ONLINE_STRUCT_FIELD(FGetConnectionStatus::Result, Status)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FConnectionStatusChanged)
	ONLINE_STRUCT_FIELD(FConnectionStatusChanged, ServiceName),
	ONLINE_STRUCT_FIELD(FConnectionStatusChanged, PreviousStatus),
	ONLINE_STRUCT_FIELD(FConnectionStatusChanged, CurrentStatus)
END_ONLINE_STRUCT_META()

/* Meta*/}

/* UE::Online */}
