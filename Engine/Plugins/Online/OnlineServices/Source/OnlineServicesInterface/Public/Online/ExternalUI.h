// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

class FOnlineError;
struct FAccountInfo;

struct FExternalUIShowLoginUI
{
	static constexpr TCHAR Name[] = TEXT("ShowLoginUI");

	struct Params
	{
		/** Platform user id logging in */
		FPlatformUserId PlatformUserId = PLATFORMUSERID_NONE;
		/** Auth scopes to request */
		TArray<FString> Scopes;
	};

	struct Result
	{
		/** The account info that was logged into */
		TSharedRef<FAccountInfo> AccountInfo;
	};
};

struct FExternalUIShowFriendsUI
{
	static constexpr TCHAR Name[] = TEXT("ShowFriendsUI");

	struct Params
	{
		/** Local user id to show the friends UI for */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

/** Struct for ExternalUIStatusChanged event */
struct FExternalUIStatusChanged
{
	/** Whether the external UI is being opened or not */
	bool bIsOpening = false;
};

class IExternalUI
{
public:
	virtual ~IExternalUI() = default;

	/**
	 * Shows the online service's login UI
	 */
	virtual TOnlineAsyncOpHandle<FExternalUIShowLoginUI> ShowLoginUI(FExternalUIShowLoginUI::Params&& Params) = 0;

	/**
	 * Shows the online service's friends UI
	 */
	virtual TOnlineAsyncOpHandle<FExternalUIShowFriendsUI> ShowFriendsUI(FExternalUIShowFriendsUI::Params&& Params) = 0;

	/**
	 * Event triggered when an external UI's status changes
	 */
	virtual TOnlineEvent<void(const FExternalUIStatusChanged&)> OnExternalUIStatusChanged() = 0;
};

namespace Meta {
// TODO: Move to ExternalUI_Meta.inl file?

BEGIN_ONLINE_STRUCT_META(FExternalUIShowLoginUI::Params)
	ONLINE_STRUCT_FIELD(FExternalUIShowLoginUI::Params, PlatformUserId),
	ONLINE_STRUCT_FIELD(FExternalUIShowLoginUI::Params, Scopes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalUIShowLoginUI::Result)
	ONLINE_STRUCT_FIELD(FExternalUIShowLoginUI::Result, AccountInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalUIShowFriendsUI::Params)
	ONLINE_STRUCT_FIELD(FExternalUIShowFriendsUI::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalUIShowFriendsUI::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FExternalUIStatusChanged)
	ONLINE_STRUCT_FIELD(FExternalUIStatusChanged, bIsOpening)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
