// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

#define UE_LEADERBOARD_RANK_UNKNOWN -1

namespace UE::Online {

struct FLeaderboardEntry
{
	/* The account id of this leaderboard entry */
	FAccountId AccountId;
	/* The rank of this account */
	int32 Rank;
	/* The score of this account */
	int64 Score;
};

struct FReadEntriesForUsers
{
	static constexpr TCHAR Name[] = TEXT("ReadEntriesForUsers");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The account ids of users */
		TArray<FAccountId> AccountIds;
		/* The leaderboard name */
		FString BoardName;
	};

	struct Result
	{
		/* The result leaderboard entries */
		TArray<FLeaderboardEntry> Entries;
	};
};

struct FReadEntriesAroundRank
{
	static constexpr TCHAR Name[] = TEXT("ReadEntriesAroundRank");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The rank index to start read */
		uint32 Rank;
		/* How many entries to fetch */
		uint32 Limit;
		/* The leaderboard to read */
		FString BoardName;
	};

	struct Result
	{
		/* The result leaderboard entries */
		TArray<FLeaderboardEntry> Entries;
	};
};

struct FReadEntriesAroundUser
{
	static constexpr TCHAR Name[] = TEXT("ReadEntriesAroundUser");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The account id of specified user */
		FAccountId AccountId;
		/* The offset to the rank of user */
		int32 Offset;
		/* How many entries to fetch */
		uint32 Limit;
		/* The leaderboard to read */
		FString BoardName;
	};

	struct Result
	{
		/* The result leaderboard entries */
		TArray<FLeaderboardEntry> Entries;
	};
};

/**
 * Interface definition for the leaderboards service 
 */
class ILeaderboards
{
public:
	virtual ~ILeaderboards() = default;

	/**
	 * Read leaderboard entries for specific users
	 */
	virtual TOnlineAsyncOpHandle<FReadEntriesForUsers> ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params) = 0;

	/**
	 * Read leaderboard entries around specified rank index
	 */
	virtual TOnlineAsyncOpHandle<FReadEntriesAroundRank> ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params) = 0;

	/**
	 * Read leaderboard entries around specified user. If the Offset is bigger than the Limit, the user 
	 * may not appear in the returned entries. It makes sense in situations like getting prev page of users 
	 * above my rank
	 */
	virtual TOnlineAsyncOpHandle<FReadEntriesAroundUser> ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params) = 0;
};


namespace Meta {

BEGIN_ONLINE_STRUCT_META(FLeaderboardEntry)
	ONLINE_STRUCT_FIELD(FLeaderboardEntry, AccountId),
	ONLINE_STRUCT_FIELD(FLeaderboardEntry, Rank),
	ONLINE_STRUCT_FIELD(FLeaderboardEntry, Score)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesForUsers::Params)
	ONLINE_STRUCT_FIELD(FReadEntriesForUsers::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FReadEntriesForUsers::Params, AccountIds),
	ONLINE_STRUCT_FIELD(FReadEntriesForUsers::Params, BoardName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesForUsers::Result)
	ONLINE_STRUCT_FIELD(FReadEntriesForUsers::Result, Entries)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesAroundRank::Params)
	ONLINE_STRUCT_FIELD(FReadEntriesAroundRank::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundRank::Params, Rank),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundRank::Params, Limit),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundRank::Params, BoardName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesAroundRank::Result)
	ONLINE_STRUCT_FIELD(FReadEntriesAroundRank::Result, Entries)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesAroundUser::Params)
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Params, AccountId),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Params, Offset),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Params, Limit),
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Params, BoardName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FReadEntriesAroundUser::Result)
	ONLINE_STRUCT_FIELD(FReadEntriesAroundUser::Result, Entries)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
