// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineError.h"
#include "Online/OnlineResult.h"
#include "Online/SchemaTypes.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }

namespace UE::Online {

enum class ELobbyJoinPolicy : uint8
{
	/** 
	* Lobby can be found through searches based on attribute matching,
	* by knowing the lobby id, or by invitation.
	*/
	PublicAdvertised,

	/** Lobby may be joined by knowing the lobby id or by invitation. */
	PublicNotAdvertised,

	/** Lobby may only be joined by invitation. */
	InvitationOnly,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ELobbyJoinPolicy Policy);
ONLINESERVICESINTERFACE_API void LexFromString(ELobbyJoinPolicy& OutPolicy, const TCHAR* InStr);

enum class ELobbyMemberLeaveReason
{
	/** The lobby member explicitly left the lobby. */
	Left,

	/** The lobby member was kicked from the lobby by the lobby owner. */
	Kicked,

	/** The lobby member unexpectedly left. */
	Disconnected,

	/**
	* The lobby was destroyed by the service.
	* All members have left.
	*/
	Closed
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ELobbyMemberLeaveReason LeaveReason);
ONLINESERVICESINTERFACE_API void LexFromString(ELobbyMemberLeaveReason& OutLeaveReason, const TCHAR* InStr);

struct FLobbyMember
{
	FAccountId AccountId;
	TMap<FSchemaAttributeId, FSchemaVariant> Attributes;
	bool bIsLocalMember = false;
};

struct FLobby
{
	FLobbyId LobbyId;
	FAccountId OwnerAccountId;
	FName LocalName;
	FSchemaId SchemaId;
	int32 MaxMembers;
	ELobbyJoinPolicy JoinPolicy;
	TMap<FSchemaAttributeId, FSchemaVariant> Attributes;
	TMap<FAccountId, TSharedRef<const FLobbyMember>> Members;
};

struct FFindLobbySearchFilter
{
	/** The name of the attribute to be compared. */
	FSchemaAttributeId AttributeName;

	/** The type of comparison to perform. */
	ESchemaAttributeComparisonOp ComparisonOp;

	/** Value to use when comparing the attribute. */
	FSchemaVariant ComparisonValue;
};

ONLINESERVICESINTERFACE_API void SortLobbies(const TArray<FFindLobbySearchFilter>& Filters, TArray<TSharedRef<const FLobby>>& Lobbies);

struct FCreateLobby
{
	static constexpr TCHAR Name[] = TEXT("CreateLobby");

	/** Input struct for Lobbies::CreateLobby */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** The local name for the lobby. */
		FName LocalName;

		/** The schema which will be applied to the lobby. */
		FSchemaId SchemaId;

		/** Whether this lobby should be set as the user's new presence lobby. False by default */
		bool bPresenceEnabled = false;

		/** The maximum number of members who can fit in the lobby. */
		int32 MaxMembers = 0;

		/** Initial join policy setting. */
		ELobbyJoinPolicy JoinPolicy = ELobbyJoinPolicy::InvitationOnly;

		/** Initial attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> Attributes;

		/** Initial user attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> UserAttributes;
	};

	/** Output struct for Lobbies::CreateLobby */
	struct Result
	{
		// Todo: investigate returning TSharedRef
		TSharedPtr<const FLobby> Lobby;
	};
};

struct FFindLobbies
{
	static constexpr TCHAR Name[] = TEXT("FindLobbies");

	/** Input struct for Lobbies::FindLobbies */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/**
		* Max results to return in one search.
		*  Actual count may be smaller based on implementation.
		*/
		uint32 MaxResults = 20;

		/** Filters to apply when searching for lobbies. */
		TArray<FFindLobbySearchFilter> Filters;

		/** Find lobbies containing the target user. */
		TOptional<FAccountId> TargetUser;

		/** Find join info for the target lobby id. */
		TOptional<FLobbyId> LobbyId;
	};

	/** Output struct for Lobbies::FindLobbies */
	struct Result
	{
		TArray<TSharedRef<const FLobby>> Lobbies;
	};
};

struct FRestoreLobbies
{
	static constexpr TCHAR Name[] = TEXT("RestoreLobbies");

	/** Input struct for Lobbies::RestoreLobbies */
	struct Params
	{
	};

	/** Output struct for Lobbies::RestoreLobbies */
	struct Result
	{
	};
};

struct FJoinLobby
{
	static constexpr TCHAR Name[] = TEXT("JoinLobby");

	/** Input struct for Lobbies::JoinLobby */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** The local name for the lobby. */
		FName LocalName;

		/** The id of the lobby to be joined. */
		FLobbyId LobbyId;

		/** Whether this lobby should be set as the user's new presence lobby. False by default */
		bool bPresenceEnabled = false;

		/** Initial user attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> UserAttributes;
	};

	/** Output struct for Lobbies::JoinLobby */
	struct Result
	{
		// Todo: investigate returning TSharedRef
		TSharedPtr<const FLobby> Lobby;
	};
};

struct FLeaveLobby
{
	static constexpr TCHAR Name[] = TEXT("LeaveLobby");

	/** Input struct for Lobbies::LeaveLobby */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby to leave. */
		FLobbyId LobbyId;
	};

	/** Output struct for Lobbies::LeaveLobby */
	struct Result
	{
	};
};

struct FInviteLobbyMember
{
	static constexpr TCHAR Name[] = TEXT("InviteLobbyMember");

	/** Input struct for Lobbies::InviteLobbyMember */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby for which the invitation will be sent. */
		FLobbyId LobbyId;

		/** Id of the player who will be sent the invitation. */
		FAccountId TargetAccountId;
	};

	/** Output struct for Lobbies::InviteLobbyMember */
	struct Result
	{
	};
};

struct FDeclineLobbyInvitation
{
	static constexpr TCHAR Name[] = TEXT("DeclineLobbyInvitation");

	/** Input struct for Lobbies::DeclineLobbyInvitation */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby for which the invitations will be declined. */
		FLobbyId LobbyId;
	};

	/** Output struct for Lobbies::DeclineLobbyInvitation */
	struct Result
	{
	};
};

struct FKickLobbyMember
{
	static constexpr TCHAR Name[] = TEXT("KickLobbyMember");

	/** Input struct for Lobbies::KickLobbyMember */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby. */
		FLobbyId LobbyId;

		/** The target user to be kicked. */
		FAccountId TargetAccountId;
	};

	/** Output struct for Lobbies::KickLobbyMember */
	struct Result
	{
	};
};

struct FPromoteLobbyMember
{
	static constexpr TCHAR Name[] = TEXT("PromoteLobbyMember");

	/** Input struct for Lobbies::PromoteLobbyMember */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby. */
		FLobbyId LobbyId;

		/** The target user to be promoted to owner. */
		FAccountId TargetAccountId;
	};

	/** Output struct for Lobbies::PromoteLobbyMember */
	struct Result
	{
	};
};

struct FModifyLobbySchema
{
	static constexpr TCHAR Name[] = TEXT("ModifyLobbySchema");

	/** Input struct for Lobbies::ModifyLobbySchema */
	struct Params
	{
	};

	/** Output struct for Lobbies::ModifyLobbySchema */
	struct Result
	{
	};
};

struct FModifyLobbyJoinPolicy
{
	static constexpr TCHAR Name[] = TEXT("ModifyLobbyJoinPolicy");

	/** Input struct for Lobbies::ModifyLobbyJoinPolicy */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby. */
		FLobbyId LobbyId;

		/** The new join policy setting. */
		ELobbyJoinPolicy JoinPolicy = ELobbyJoinPolicy::InvitationOnly;
	};

	/** Output struct for Lobbies::ModifyLobbyJoinPolicy */
	struct Result
	{
	};
};

struct FModifyLobbyAttributes
{
	static constexpr TCHAR Name[] = TEXT("ModifyLobbyAttributes");

	/** Input struct for Lobbies::ModifyLobbyAttributes */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby. */
		FLobbyId LobbyId;

		/** New or changed lobby attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> UpdatedAttributes;

		/** Attributes to be cleared. */
		TSet<FSchemaAttributeId> RemovedAttributes;
	};

	/** Output struct for Lobbies::ModifyLobbyAttributes */
	struct Result
	{
	};
};

struct FModifyLobbyMemberAttributes
{
	static constexpr TCHAR Name[] = TEXT("ModifyLobbyMemberAttributes");

	/** Input struct for Lobbies::ModifyLobbyMemberAttributes */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;

		/** Id of the lobby. */
		FLobbyId LobbyId;

		/** New or changed lobby attributes. */
		TMap<FSchemaAttributeId, FSchemaVariant> UpdatedAttributes;

		/** Attributes to be cleared. */
		TSet<FSchemaAttributeId> RemovedAttributes;
	};

	/** Output struct for Lobbies::ModifyLobbyMemberAttributes */
	struct Result
	{
	};
};

struct FGetPresenceLobby
{
	static constexpr TCHAR Name[] = TEXT("GetPresenceLobby");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Reference to the lobby set as presence lobby for the given user */
		TSharedRef<const FLobby> Lobby;
	};
};

struct FIsPresenceLobby
{
	static constexpr TCHAR Name[] = TEXT("IsPresenceLobby");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Id handle for the lobby to be compared */
		FLobbyId LobbyId;
	};

	struct Result
	{
		/** Whether the lobby mapped to the given id is set as the presence lobby for the given user */
		bool bIsPresenceLobby;
	};
};

struct FGetJoinedLobbies
{
	static constexpr TCHAR Name[] = TEXT("GetJoinedLobbies");

	/** Input struct for Lobbies::GetJoinedLobbies */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;
	};

	/** Output struct for Lobbies::GetJoinedLobbies */
	struct Result
	{
		TArray<TSharedRef<const FLobby>> Lobbies;
	};
};

struct FGetReceivedInvitations
{
	static constexpr TCHAR Name[] = TEXT("GetReceivedInvitations");

	/** Input struct for Lobbies::GetReceivedInvitations */
	struct Params
	{
		/** The local user agent which will perform the action. */
		FAccountId LocalAccountId;
	};

	/** Output struct for Lobbies::GetReceivedInvitations */
	struct Result
	{
	};
};

/** Struct for LobbyJoined event */
struct FLobbyJoined
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;
};

/** Struct for LobbyLeft event */
struct FLobbyLeft
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;
};

/** Struct for LobbyMemberJoined event */
struct FLobbyMemberJoined
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;

	/** Member data access. */
	TSharedRef<const FLobbyMember> Member;
};

/** Struct for LobbyMemberLeft event */
struct FLobbyMemberLeft
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;

	/** Member data access. */
	TSharedRef<const FLobbyMember> Member;

	/** Context for the member leaving. */
	ELobbyMemberLeaveReason Reason;
};

/** Struct for LobbyLeaderChanged event */
struct FLobbyLeaderChanged
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;

	/** Leader data access. */
	TSharedRef<const FLobbyMember> Leader;
};

/** Struct for LobbySchemaChanged event */
struct FLobbySchemaChanged
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;
	/** The previous schema id. */
	FSchemaId PreviousSchemaId;
	/** The new schema id. */
	FSchemaId NewSchemaId;
};

/** Struct for LobbyAttributesChanged event */
struct FLobbyAttributesChanged
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;

	/** Added attributes. */
	TMap<FSchemaAttributeId, FSchemaVariant> AddedAttributes;

	/** Changed attributes with their old and new values. */
	TMap<FSchemaAttributeId, TPair<FSchemaVariant, FSchemaVariant>> ChangedAttributes;

	/** Removed attribute ids. */
	TSet<FSchemaAttributeId> RemovedAttributes;

};

/** Struct for LobbyMemberAttributesChanged event */
struct FLobbyMemberAttributesChanged
{
	/** Lobby data access. */
	TSharedRef<const FLobby> Lobby;

	/** Member data access. */
	TSharedRef<const FLobbyMember> Member;

	/** Added attributes. */
	TMap<FSchemaAttributeId, FSchemaVariant> AddedAttributes;

	/** Changed attributes with their old and new values. */
	TMap<FSchemaAttributeId, TPair<FSchemaVariant, FSchemaVariant>> ChangedAttributes;

	/** Removed attribute ids. */
	TSet<FSchemaAttributeId> RemovedAttributes;
};

/** Struct for LobbyInvitationAdded event */
struct FLobbyInvitationAdded
{
	/** The local user associated with the invitation. */
	FAccountId LocalAccountId;

	/** The user who sent the invitation. */
	FAccountId SenderId;

	/** The invited lobby. */
	TSharedRef<const FLobby> Lobby;
};

/** Struct for LobbyInvitationRemoved event */
struct FLobbyInvitationRemoved
{
	/** The local user associated with the invitation. */
	FAccountId LocalAccountId;

	/** The user who sent the invitation. */
	FAccountId SenderId;

	/** The invited lobby. */
	TSharedRef<const FLobby> Lobby;

};

/** Lobby join requested source */
enum class EUILobbyJoinRequestedSource : uint8
{
	/** Unspecified by the online service */
	Unspecified,
	/** From an invitation */
	FromInvitation,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUILobbyJoinRequestedSource UILobbyJoinRequestedSource);
ONLINESERVICESINTERFACE_API void LexFromString(EUILobbyJoinRequestedSource& OutUILobbyJoinRequestedSource, const TCHAR* InStr);

/** Struct for UILobbyJoinedRequested event */
struct FUILobbyJoinRequested
{
	/** The local user associated with the join request. */
	FAccountId LocalAccountId;

	/** The lobby if the local user requested to join, or the online error if there was a failure */
	TResult<TSharedRef<const FLobby>, FOnlineError> Result;

	/** Join request source */
	EUILobbyJoinRequestedSource JoinRequestedSource = EUILobbyJoinRequestedSource::Unspecified;
};

class ILobbies
{
public:
	virtual ~ILobbies() = default;

	//----------------------------------------------------------------------------------------------
	// Operations

	/**
	 * Create and join a new lobby.
	 *
	 * @param Params for the CreateLobby call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FCreateLobby> CreateLobby(FCreateLobby::Params&& Params) = 0;

	/**
	 * Search for lobbies using filtering parameters.
	 *
	 * @param Params for the FindLobby call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FFindLobbies> FindLobbies(FFindLobbies::Params&& Params) = 0;

	/**
	 * Try to rejoin previously joined lobbies.
	 *
	 * @param Params for the RestoreLobbies call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FRestoreLobbies> RestoreLobbies(FRestoreLobbies::Params&& Params) = 0;

	/**
	 * Join a lobby using its id.
	 *
	 * @param Params for the JoinLobby call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FJoinLobby> JoinLobby(FJoinLobby::Params&& Params) = 0;

	/**
	 * Leave a joined lobby.
	 *
	 * @param Params for the LeaveLobby call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FLeaveLobby> LeaveLobby(FLeaveLobby::Params&& Params) = 0;

	/**
	 * Invite a player to join a lobby.
	 *
	 * @param Params for the InviteLobbyMember call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FInviteLobbyMember> InviteLobbyMember(FInviteLobbyMember::Params&& Params) = 0;

	/**
	 * Decline an invitation to join a lobby.
	 *
	 * @param Params for the DeclineLobbyInvitation call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FDeclineLobbyInvitation> DeclineLobbyInvitation(FDeclineLobbyInvitation::Params&& Params) = 0;

	/**
	 * Kick a member from a the target lobby.
	 *
	 * @param Params for the KickLobbyMember call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FKickLobbyMember> KickLobbyMember(FKickLobbyMember::Params&& Params) = 0;

	/**
	 * Promote another lobby member to leader.
	 * The local user must be the current lobby leader to promote another member.
	 *
	 * @param Params for the PromoteLobbyMember call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FPromoteLobbyMember> PromoteLobbyMember(FPromoteLobbyMember::Params&& Params) = 0;

	//----------------------------------------------------------------------------------------------
	// Mutations

	/**
	 * Change the schema applied to the lobby and member attributes.
	 * Only the lobby leader may change the schema. Existing attributes not present in the new
	 * schema will be cleared.
	 *
	 * @param Params for the ModifyLobbySchema call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FModifyLobbySchema> ModifyLobbySchema(FModifyLobbySchema::Params&& Params) = 0;

	/**
	 * Change the join policy applied to the lobby.
	 * Only the lobby leader may change the join policy.
	 *
	 * @param Params for the ModifyLobbyJoinPolicy call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FModifyLobbyJoinPolicy> ModifyLobbyJoinPolicy(FModifyLobbyJoinPolicy::Params&& Params) = 0;

	/**
	 * Change the attributes applied to the lobby.
	 * Only the lobby leader may change the lobby attributes.
	 * Attributes are validated against the lobby schema before an update can succeed.
	 *
	 * @param Params for the ModifyLobbyAttributes call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FModifyLobbyAttributes> ModifyLobbyAttributes(FModifyLobbyAttributes::Params&& Params) = 0;

	/**
	 * Change the attributes applied to a lobby member.
	 * Lobby members may only change their own attributes.
	 * Attributes are validated against the lobby schema before an update can succeed.
	 *
	 * @param Params for the ModifyLobbyAttributes call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FModifyLobbyMemberAttributes> ModifyLobbyMemberAttributes(FModifyLobbyMemberAttributes::Params&& Params) = 0;

	//----------------------------------------------------------------------------------------------
	// Accessors

	/**
	 * Retrieve the presence Lobby for the given user, if any is set
	 * 
	 * @params Params for the GetPresenceLobby call
	 * @return
	 */
	virtual TOnlineResult<FGetPresenceLobby> GetPresenceLobby(FGetPresenceLobby::Params&& Params) = 0;

	/**
	 * Return whether the Lobby with the given id is set as presence lobby for the given user
	 * 
	 * @params Params for the IsPresenceLobby call
	 * @return
	 */
	virtual TOnlineResult<FIsPresenceLobby> IsPresenceLobby(FIsPresenceLobby::Params&& Params) = 0;

	/**
	 * Retrieve the list of joined lobbies for the target local user.
	 *
	 * @param Params for the GetJoinedLobbies call
	 * @return
	 */
	virtual TOnlineResult<FGetJoinedLobbies> GetJoinedLobbies(FGetJoinedLobbies::Params&& Params) = 0;

	/**
	 * Retrieve the list of received invitations for the target local user.
	 *
	 * @param Params for the GetJoinedLobbies call
	 * @return
	 */
	virtual TOnlineResult<FGetReceivedInvitations> GetReceivedInvitations(FGetReceivedInvitations::Params&& Params) = 0;

	//----------------------------------------------------------------------------------------------
	// Events

	/**
	 * Get the event that is triggered when a lobby is joined.
	 * This will happen as a result of creating or joining a lobby.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyJoined&)> OnLobbyJoined() = 0;

	/**
	 * Get the event that is triggered when a lobby has been left by all local members.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyLeft&)> OnLobbyLeft() = 0;

	/**
	 * Get the event that is triggered when a lobby member joins.
	 * This will happen as a result of creating or joining a lobby for local users, and will trigger
	 * when a remote user joins.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyMemberJoined&)> OnLobbyMemberJoined() = 0;

	/**
	 * Get the event that is triggered when a lobby member leaves a joined lobby.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyMemberLeft&)> OnLobbyMemberLeft() = 0;

	/**
	 * Get the event that is triggered when the leadership of a lobby changes.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyLeaderChanged&)> OnLobbyLeaderChanged() = 0;

	/**
	 * Get the event that is triggered when the attribute schema of a lobby changes.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbySchemaChanged&)> OnLobbySchemaChanged() = 0;

	/**
	 * Get the event that is triggered when lobby attributes have changed.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyAttributesChanged&)> OnLobbyAttributesChanged() = 0;

	/**
	 * Get the event that is triggered when lobby member attributes have changed.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyMemberAttributesChanged&)> OnLobbyMemberAttributesChanged() = 0;

	/**
	 * Get the event that is triggered when an invitation is received.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyInvitationAdded&)> OnLobbyInvitationAdded() = 0;

	/**
	 * Get the event that is triggered when an invitation is removed.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FLobbyInvitationRemoved&)> OnLobbyInvitationRemoved() = 0;

	/**
	 * Get the event that is triggered when a join is requested through an external mechanism.
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FUILobbyJoinRequested&)> OnUILobbyJoinRequested() = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FLobbyMember)
	ONLINE_STRUCT_FIELD(FLobbyMember, AccountId),
	ONLINE_STRUCT_FIELD(FLobbyMember, Attributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobby)
	ONLINE_STRUCT_FIELD(FLobby, LobbyId),
	ONLINE_STRUCT_FIELD(FLobby, OwnerAccountId),
	ONLINE_STRUCT_FIELD(FLobby, LocalName),
	ONLINE_STRUCT_FIELD(FLobby, SchemaId),
	ONLINE_STRUCT_FIELD(FLobby, MaxMembers),
	ONLINE_STRUCT_FIELD(FLobby, JoinPolicy),
	ONLINE_STRUCT_FIELD(FLobby, Attributes),
	ONLINE_STRUCT_FIELD(FLobby, Members)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFindLobbySearchFilter)
	ONLINE_STRUCT_FIELD(FFindLobbySearchFilter, AttributeName),
	ONLINE_STRUCT_FIELD(FFindLobbySearchFilter, ComparisonOp),
	ONLINE_STRUCT_FIELD(FFindLobbySearchFilter, ComparisonValue)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCreateLobby::Params)
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, LocalName),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, SchemaId),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, bPresenceEnabled),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, MaxMembers),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, JoinPolicy),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, Attributes),
	ONLINE_STRUCT_FIELD(FCreateLobby::Params, UserAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCreateLobby::Result)
	ONLINE_STRUCT_FIELD(FCreateLobby::Result, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFindLobbies::Params)
	ONLINE_STRUCT_FIELD(FFindLobbies::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FFindLobbies::Params, MaxResults),
	ONLINE_STRUCT_FIELD(FFindLobbies::Params, Filters),
	ONLINE_STRUCT_FIELD(FFindLobbies::Params, TargetUser),
	ONLINE_STRUCT_FIELD(FFindLobbies::Params, LobbyId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFindLobbies::Result)
	ONLINE_STRUCT_FIELD(FFindLobbies::Result, Lobbies)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRestoreLobbies::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRestoreLobbies::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FJoinLobby::Params)
	ONLINE_STRUCT_FIELD(FJoinLobby::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FJoinLobby::Params, LocalName),
	ONLINE_STRUCT_FIELD(FJoinLobby::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FJoinLobby::Params, bPresenceEnabled),
	ONLINE_STRUCT_FIELD(FJoinLobby::Params, UserAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FJoinLobby::Result)
	ONLINE_STRUCT_FIELD(FJoinLobby::Result, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLeaveLobby::Params)
	ONLINE_STRUCT_FIELD(FLeaveLobby::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLeaveLobby::Params, LobbyId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLeaveLobby::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FInviteLobbyMember::Params)
	ONLINE_STRUCT_FIELD(FInviteLobbyMember::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FInviteLobbyMember::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FInviteLobbyMember::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FInviteLobbyMember::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FDeclineLobbyInvitation::Params)
	ONLINE_STRUCT_FIELD(FDeclineLobbyInvitation::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FDeclineLobbyInvitation::Params, LobbyId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FDeclineLobbyInvitation::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FKickLobbyMember::Params)
	ONLINE_STRUCT_FIELD(FKickLobbyMember::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FKickLobbyMember::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FKickLobbyMember::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FKickLobbyMember::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPromoteLobbyMember::Params)
	ONLINE_STRUCT_FIELD(FPromoteLobbyMember::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FPromoteLobbyMember::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FPromoteLobbyMember::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPromoteLobbyMember::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbySchema::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbySchema::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyJoinPolicy::Params)
	ONLINE_STRUCT_FIELD(FModifyLobbyJoinPolicy::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FModifyLobbyJoinPolicy::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FModifyLobbyJoinPolicy::Params, JoinPolicy)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyJoinPolicy::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyAttributes::Params)
	ONLINE_STRUCT_FIELD(FModifyLobbyAttributes::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FModifyLobbyAttributes::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FModifyLobbyAttributes::Params, UpdatedAttributes),
	ONLINE_STRUCT_FIELD(FModifyLobbyAttributes::Params, RemovedAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyAttributes::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyMemberAttributes::Params)
	ONLINE_STRUCT_FIELD(FModifyLobbyMemberAttributes::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FModifyLobbyMemberAttributes::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FModifyLobbyMemberAttributes::Params, UpdatedAttributes),
	ONLINE_STRUCT_FIELD(FModifyLobbyMemberAttributes::Params, RemovedAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FModifyLobbyMemberAttributes::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetPresenceLobby::Params)
	ONLINE_STRUCT_FIELD(FGetPresenceLobby::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetPresenceLobby::Result)
	ONLINE_STRUCT_FIELD(FGetPresenceLobby::Result, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FIsPresenceLobby::Params)
	ONLINE_STRUCT_FIELD(FIsPresenceLobby::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FIsPresenceLobby::Params, LobbyId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FIsPresenceLobby::Result)
	ONLINE_STRUCT_FIELD(FIsPresenceLobby::Result, bIsPresenceLobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetJoinedLobbies::Params)
	ONLINE_STRUCT_FIELD(FGetJoinedLobbies::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetJoinedLobbies::Result)
	ONLINE_STRUCT_FIELD(FGetJoinedLobbies::Result, Lobbies)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetReceivedInvitations::Params)
	ONLINE_STRUCT_FIELD(FGetReceivedInvitations::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetReceivedInvitations::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyJoined)
	ONLINE_STRUCT_FIELD(FLobbyJoined, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyLeft)
	ONLINE_STRUCT_FIELD(FLobbyLeft, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyMemberJoined)
	ONLINE_STRUCT_FIELD(FLobbyMemberJoined, Lobby),
	ONLINE_STRUCT_FIELD(FLobbyMemberJoined, Member)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyMemberLeft)
	ONLINE_STRUCT_FIELD(FLobbyMemberLeft, Lobby),
	ONLINE_STRUCT_FIELD(FLobbyMemberLeft, Member),
	ONLINE_STRUCT_FIELD(FLobbyMemberLeft, Reason)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyLeaderChanged)
	ONLINE_STRUCT_FIELD(FLobbyLeaderChanged, Lobby),
	ONLINE_STRUCT_FIELD(FLobbyLeaderChanged, Leader)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbySchemaChanged)
	ONLINE_STRUCT_FIELD(FLobbySchemaChanged, Lobby),
	ONLINE_STRUCT_FIELD(FLobbySchemaChanged, PreviousSchemaId),
	ONLINE_STRUCT_FIELD(FLobbySchemaChanged, NewSchemaId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyAttributesChanged)
	ONLINE_STRUCT_FIELD(FLobbyAttributesChanged, Lobby),
	ONLINE_STRUCT_FIELD(FLobbyAttributesChanged, AddedAttributes),
	ONLINE_STRUCT_FIELD(FLobbyAttributesChanged, ChangedAttributes),
	ONLINE_STRUCT_FIELD(FLobbyAttributesChanged, RemovedAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyMemberAttributesChanged)
	ONLINE_STRUCT_FIELD(FLobbyMemberAttributesChanged, Lobby),
	ONLINE_STRUCT_FIELD(FLobbyMemberAttributesChanged, Member),
	ONLINE_STRUCT_FIELD(FLobbyMemberAttributesChanged, AddedAttributes),
	ONLINE_STRUCT_FIELD(FLobbyMemberAttributesChanged, ChangedAttributes),
	ONLINE_STRUCT_FIELD(FLobbyMemberAttributesChanged, RemovedAttributes)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyInvitationAdded)
	ONLINE_STRUCT_FIELD(FLobbyInvitationAdded, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbyInvitationAdded, SenderId),
	ONLINE_STRUCT_FIELD(FLobbyInvitationAdded, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLobbyInvitationRemoved)
	ONLINE_STRUCT_FIELD(FLobbyInvitationRemoved, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLobbyInvitationRemoved, SenderId),
	ONLINE_STRUCT_FIELD(FLobbyInvitationRemoved, Lobby)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUILobbyJoinRequested)
	ONLINE_STRUCT_FIELD(FUILobbyJoinRequested, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUILobbyJoinRequested, Result),
	ONLINE_STRUCT_FIELD(FUILobbyJoinRequested, JoinRequestedSource)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
