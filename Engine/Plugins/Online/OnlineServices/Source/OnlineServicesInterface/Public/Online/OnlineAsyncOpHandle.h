// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineErrorDefinitions.h"



namespace UE::Online { template <typename OpType> class TOnlineResult; }

class UObjectBase;

namespace UE::Online {

namespace Private
{
	struct CAsSharedCallable
	{
		template <typename T>
		auto Requires(T&& Object) -> decltype(Object->AsShared());
	};

	template <typename T>
	struct TIsSharedPtrOrRef
	{
		static inline constexpr bool value = false;
	};

	template <typename T, ESPMode SPMode>
	struct TIsSharedPtrOrRef<TSharedPtr<T, SPMode>>
	{
		static inline constexpr bool value = true;
	};

	template <typename T, ESPMode SPMode>
	struct TIsSharedPtrOrRef<TSharedRef<T, SPMode>>
	{
		static inline constexpr bool value = true;
	};

	template <typename ObjectType, typename = void>
	struct TSharedPtrHelper
	{
		static inline constexpr bool bIsThreadSafe = false;
		static inline constexpr bool bIsSharedPtr = false;
	};

	template <typename T>
	struct TSharedPtrHelper2
	{
		using PtrType = decltype(((std::remove_pointer_t<T>*)nullptr)->AsShared());
		static inline constexpr bool bIsThreadSafe = TSharedPtrHelper<PtrType>::bIsThreadSafe;
		static inline constexpr bool bIsSharedPtr = true;
		using WeakPtrType = typename TSharedPtrHelper<PtrType>::WeakPtrType;
		static inline WeakPtrType GetWeak(T Ptr) { return Ptr->AsShared(); }
	};

	template <typename T, ESPMode SPMode>
	struct TSharedPtrHelper2<TSharedPtr<T, SPMode>>
	{
		static inline constexpr bool bIsThreadSafe = SPMode == ESPMode::ThreadSafe;
		static inline constexpr bool bIsSharedPtr = true;
		using WeakPtrType = TWeakPtr<T, SPMode>;
		static inline WeakPtrType GetWeak(const TSharedPtr<T, SPMode>& Ptr) { return Ptr; }
	};

	template <typename T, ESPMode SPMode>
	struct TSharedPtrHelper2<TSharedRef<T, SPMode>>
	{
		static inline constexpr bool bIsThreadSafe = SPMode == ESPMode::ThreadSafe;
		static inline constexpr bool bIsSharedPtr = true;
		using WeakPtrType = TWeakPtr<T, SPMode>;
		static inline WeakPtrType GetWeak(const TSharedRef<T, SPMode>& Ptr) { return Ptr; }
	};

	template <typename T>
	struct TSharedPtrHelper<T, std::enable_if_t<TModels_V<CAsSharedCallable, T> && !TIsSharedPtrOrRef<T>::value, void>> : public TSharedPtrHelper2<T>
	{
	};

	template <typename T, ESPMode SPMode>
	struct TSharedPtrHelper<TSharedPtr<T, SPMode>, void> : public TSharedPtrHelper2<TSharedPtr<T, SPMode>>
	{
	};

	template <typename T, ESPMode SPMode>
	struct TSharedPtrHelper<TSharedRef<T, SPMode>, void> : public TSharedPtrHelper2<TSharedRef<T, SPMode>>
	{
	};

	template <typename DelegateSignature, typename CallableType, typename... VarTypes>
	TDelegate<DelegateSignature> ConstructFunctionDelegate(CallableType&& Callable, VarTypes&&... Vars)
	{
		if constexpr (std::is_function_v<std::remove_reference_t<std::remove_pointer_t<CallableType>>>)
		{
			return TDelegate<DelegateSignature>::CreateStatic(Forward<CallableType>(Callable), Forward<VarTypes>(Vars)...);
		}
		else
		{
			return TDelegate<DelegateSignature>::CreateLambda(Forward<CallableType>(Callable), Forward<VarTypes>(Vars)...);
		}
	}

	template <typename DelegateSignature, typename ObjectType, typename CallableType, typename... VarTypes>
	TDelegate<DelegateSignature> ConstructObjectDelegate(ObjectType Object, CallableType&& Callable, VarTypes&&... Vars)
	{
		if constexpr (std::is_member_function_pointer_v<CallableType>)
		{
			if constexpr (std::is_convertible_v<ObjectType, const UObjectBase*>)
			{
				return TDelegate<DelegateSignature>::CreateUObject(Object, Forward<decltype(Callable)>(Callable), Forward<VarTypes>(Vars)...);
			}
			else if constexpr (TSharedPtrHelper<ObjectType>::bIsSharedPtr)
			{
				if constexpr (TSharedPtrHelper<ObjectType>::bIsThreadSafe)
				{
					return TDelegate<DelegateSignature>::CreateThreadSafeSP(Object, Callable, Forward<VarTypes>(Vars)...);
				}
				else
				{
					return TDelegate<DelegateSignature>::CreateSP(Object, Callable, Forward<VarTypes>(Vars)...);
				}
			}
			else
			{
				return TDelegate<DelegateSignature>::CreateRaw(Object, Callable, Forward<VarTypes>(Vars)...);
			}
		}
		else
		{
			if constexpr (std::is_convertible_v<ObjectType, const UObjectBase*>)
			{
				if constexpr (std::is_same_v<FName, std::remove_cv_t<CallableType>>)
				{
					return TDelegate<DelegateSignature>::CreateUFunction(Object, Callable, Forward<VarTypes>(Vars)...);
				}
				else
				{
					return TDelegate<DelegateSignature>::CreateWeakLambda(Object, Callable, Forward<VarTypes>(Vars)...);
				}
			}
			else
			{
				static_assert(TSharedPtrHelper<ObjectType>::bIsSharedPtr, "When using non pointer to member functions, the first parameter can only be a UObject*, TSharedRef, or pointer to a class that derives from TSharedFromThis");
				auto InnerDelegate = ConstructFunctionDelegate<DelegateSignature>(Callable, Forward<VarTypes>(Vars)...);
				return TDelegate<DelegateSignature>::CreateLambda([WeakObject = TSharedPtrHelper<ObjectType>::GetWeak(Object), Delegate = MoveTemp(InnerDelegate)](auto&&... Params)
					{
						auto StrongObject = WeakObject.Pin();
						if (StrongObject.IsValid())
						{
							Delegate.ExecuteIfBound(Forward<decltype(Params)>(Params)...);
						}
					});
			}
		}
	}

	template <typename DelegateSignature, typename ObjectOrCallableType, typename... VarTypes>
	[[nodiscard]] inline TDelegate<DelegateSignature> ConstructDelegate(ObjectOrCallableType&& ObjectOrCallable, VarTypes&&... Vars)
	{
		if constexpr((std::is_pointer_v<std::remove_reference_t<ObjectOrCallableType>> && !std::is_function_v<std::remove_reference_t<std::remove_pointer_t<ObjectOrCallableType>>>) || TSharedPtrHelper<std::remove_reference_t<ObjectOrCallableType>>::bIsSharedPtr)
		{
			return ConstructObjectDelegate<DelegateSignature>(Forward<ObjectOrCallableType>(ObjectOrCallable), Forward<VarTypes>(Vars)...);
		}
		else
		{
			return ConstructFunctionDelegate<DelegateSignature>(Forward<ObjectOrCallableType>(ObjectOrCallable), Forward<VarTypes>(Vars)...);
		}
	}

/* Private */ }

template<typename DelegateSignature>
class TOnlineCallback
{
	// Will hit TDelegate's static_asserts
	TDelegate<DelegateSignature> Delegate;
};

/**
 *
 */
template <typename InRetValType, typename... ParamTypes>
class TOnlineCallback<InRetValType(ParamTypes...)>
{
public:
	using DelegateSignature = InRetValType(ParamTypes...);

	/**
	 * 
	 */
	template<typename... ConstructDelegateParamTypes>
	void Bind(ConstructDelegateParamTypes&&... Params)
	{
		Delegate = Private::ConstructDelegate<DelegateSignature>(Forward<ConstructDelegateParamTypes>(Params)...);
	}

	/**
	 *
	 */
	void Unbind()
	{
		Delegate.Unbind();
	}

	/**
	 *
	 */
	void ExecuteIfBound(ParamTypes&&... Params)
	{
		Delegate.ExecuteIfBound(Forward<ParamTypes>(Params)...);
	}

private:
	TDelegate<DelegateSignature> Delegate;
};

/**
 *
 */
class FOnlineEventDelegateHandle
{
public:
	FOnlineEventDelegateHandle() = default;
	FOnlineEventDelegateHandle(FOnlineEventDelegateHandle&& Other)
	{
		DelegateUnbinder = MoveTemp(Other.DelegateUnbinder);
	}

	FOnlineEventDelegateHandle& operator=(FOnlineEventDelegateHandle&& Other)
	{
		Unbind();
		DelegateUnbinder = MoveTemp(Other.DelegateUnbinder);
		return *this;
	}

	/**
	 *
	 */
	template <typename DelegateSignature>
	FOnlineEventDelegateHandle(FDelegateHandle Handle, TSharedRef<TMulticastDelegate<DelegateSignature>>& DelegateRef)
	{
		DelegateUnbinder = [Handle, DelegateWeakPtr = TWeakPtr<TMulticastDelegate<DelegateSignature>>(DelegateRef)]()
			{
				TSharedPtr<TMulticastDelegate<DelegateSignature>> DelegateSP = DelegateWeakPtr.Pin();
				if (DelegateSP)
				{
					check(IsInGameThread());
					DelegateSP->Remove(Handle);
				}
			};
	}

	/**
	 *
	 */
	~FOnlineEventDelegateHandle()
	{
		Unbind();
	}

	/**
	 * 
	 */
	void Unbind()
	{
		if (DelegateUnbinder)
		{
			DelegateUnbinder();
			DelegateUnbinder = nullptr;
		}
	}

private:
	FOnlineEventDelegateHandle(const FOnlineEventDelegateHandle&) = delete;
	FOnlineEventDelegateHandle& operator=(const FOnlineEventDelegateHandle&) = delete;

	TUniqueFunction<void()> DelegateUnbinder;
};

template<typename DelegateSignature>
class TOnlineEventCallable
{
	// Will hit TMulticastDelegate's static_asserts
	TMulticastDelegate<DelegateSignature> Delegate;
};

/**
 *
 */
template <typename... ParamTypes>
class TOnlineEventCallable<void(ParamTypes...)>
{
public:
	using DelegateSignature = void(ParamTypes...);

	/**
	 *
	 */
	TOnlineEventCallable()
		: DelegateRef(MakeShared<TMulticastDelegate<DelegateSignature>>())
	{
	}

	/**
	 *
	 */
	template<typename... ConstructDelegateParamTypes>
	FOnlineEventDelegateHandle Add(ConstructDelegateParamTypes&&... Params)
	{
		FDelegateHandle Handle = DelegateRef->Add(Private::ConstructDelegate<DelegateSignature>(Forward<ConstructDelegateParamTypes>(Params)...));
		return FOnlineEventDelegateHandle(Handle, DelegateRef);
	}

	void Clear()
	{
		DelegateRef->Clear();
	}

	/**
	 *
	 */
	void Broadcast(ParamTypes&&... Params)
	{
		DelegateRef->Broadcast(Forward<ParamTypes>(Params)...);
	}

private:
	TSharedRef<TMulticastDelegate<DelegateSignature>> DelegateRef;
};

/**
 *
 */
template <typename DelegateSignature>
class TOnlineEvent
{
public:
	/**
	 *
	 */
	TOnlineEvent(TOnlineEventCallable<DelegateSignature>& InEvent)
		: Event(InEvent)
	{
	}

	/**
	 *
	 */
	template<typename... ConstructDelegateParamTypes>
	[[nodiscard]] FOnlineEventDelegateHandle Add(ConstructDelegateParamTypes&&... Params)
	{
		return Event.Add(Forward<ConstructDelegateParamTypes>(Params)...);
	}

private:
	TOnlineEventCallable<DelegateSignature>& Event;
};

enum class EAsyncOpState : uint8
{
	Invalid,
	Queued,
	Running,
	RetryQueued,
	Complete,
	Cancelled
};

inline const TCHAR* LexToString(EAsyncOpState Value)
{
	switch (Value)
	{
	case EAsyncOpState::Queued: return TEXT("Queued");
	case EAsyncOpState::Running: return TEXT("Running");
	case EAsyncOpState::RetryQueued: return TEXT("RetryQueued");
	case EAsyncOpState::Complete: return TEXT("Complete");
	case EAsyncOpState::Cancelled: return TEXT("Cancelled");
	default: checkNoEntry(); // Intentional fall through
	case EAsyncOpState::Invalid: return TEXT("Invalid");
	}
}

class IOnlineAsyncOp
{
public:
	virtual ~IOnlineAsyncOp() = default;

	virtual void Cancel() = 0;
	virtual EAsyncOpState GetState() const = 0;
	virtual void DetachHandle(class IAsyncHandle* Handle) = 0;
};

struct FAsyncProgress
{
	/* Step that is running */
	const FString Step;
	/* How much has been completed */
	double Progress;
	/* How much total there is to do */
	double MaxProgress;
	/* Localized message suitable for showing to end user */
	FText ProgressMessage;
};

struct FWillRetry
{
	FOnlineError Reason;
	double RetryInSeconds;
	int RetryAttempt;
	int MaxRetryAttempts;
};

template <typename OpType> class TOnlineAsyncOpHandle;

namespace Private
{
	// State shared between the Op and Handle
	template <typename OpType>
	class IOnlineAsyncOpSharedState
	{
	public:
		virtual ~IOnlineAsyncOpSharedState() = default;
		virtual void Cancel(const FOnlineError& Reason) = 0;
		virtual EAsyncOpState GetState() const = 0;
		virtual void SetOnProgress(TDelegate<void(const FAsyncProgress&)>&& Delegate) = 0;
		virtual void SetOnWillRetry(TDelegate<void(TOnlineAsyncOpHandle<OpType>&, const FWillRetry&)>&& Delegate) = 0;
		virtual void SetOnComplete(TDelegate<void(const TOnlineResult<OpType>&)>&& Delegate) = 0;
	};
}

/**
 *
 */
template <typename OpType>
class TOnlineAsyncOpHandle
{
public:
	/**
	 *
	 */
	TOnlineAsyncOpHandle(TSharedRef<Private::IOnlineAsyncOpSharedState<OpType>>&& InSharedState)
		: State(MoveTemp(InSharedState))
	{
	}

	// need to add a comment here explaining how this works:
	// There's an optional first parameter that is a class instance (raw pointer or shared/weakptr/ref/weakobjptr). If possible, this will be checked for validity before calling the callback
	// Next is the callback, signatures in FAsyncOpState (TODO: Move that to a more visible location), can be a member function
	// After that is any additional params that should be captured (like the ue4 delegates support) (or we remove this capability)
	// Always called on the game thread

	/**
	 * 
	 *
	 * @param 
	 * @return The async op handle
	 */
	template <typename... ArgTypes>
	TOnlineAsyncOpHandle& OnProgress(ArgTypes&&... Args)
	{
		State->SetOnProgress(Private::ConstructDelegate<void(const FAsyncProgress&)>(Forward<ArgTypes>(Args)...));

		return *this;
	}

	/**
	 * 
	 *
	 * @param 
	 * @return The async op handle
	 */
	template <typename... ArgTypes>
	TOnlineAsyncOpHandle& OnWillRetry(ArgTypes&&... Args)
	{
		State->SetOnWillRetry(Private::ConstructDelegate<void(TOnlineAsyncOpHandle<OpType>&, const FWillRetry&)>(Forward<ArgTypes>(Args)...));

		return *this;
	}

	/**
	 * 
	 *
	 * @param
	 * @return The async op handle
	 */
	template <typename... ArgTypes>
	TOnlineAsyncOpHandle& OnComplete(ArgTypes&&... Args)
	{
		State->SetOnComplete(Private::ConstructDelegate<void(const TOnlineResult<OpType>&)>(Forward<ArgTypes>(Args)...));

		return *this;
	}

	/**
	 * Cancel the operation
	 *
	 * @param Reason Allows a more specific reson why the operation is cancelled to be specified
	 */
	void Cancel(const FOnlineError& Reason = Errors::Cancelled())
	{
		State->Cancel(Reason);
	}

	/**
	 * Get the state of the operation
	 * 
	 * @return The state of the operation
	 */
	EAsyncOpState GetState() const
	{
		return State->GetState();
	}

private:
	TSharedPtr<Private::IOnlineAsyncOpSharedState<OpType>> State;
};

/* UE::Online */ }
