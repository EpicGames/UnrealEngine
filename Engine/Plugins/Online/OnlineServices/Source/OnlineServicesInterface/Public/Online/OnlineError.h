// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Internationalization/Internationalization.h"
#include "OnlineErrorCode.h"

namespace UE::Online {

using ErrorCodeType = uint64;
class FOnlineError;
#define LOCTEXT_NAMESPACE "OnlineError"

// Implementations of this would contain any contextual information about the error (eg, if an FText or FString exposed some fields to be filled in at run time), and likely the platform specific error representation (eg, NSError on apple that we can use to get a string to display to user and a string to log)
class IOnlineErrorDetails
{
public:
	virtual ~IOnlineErrorDetails() = default;

	virtual FString GetFriendlyErrorCode(const FOnlineError&) const = 0;
	virtual FText GetText(const FOnlineError&) const = 0;
	virtual FString GetLogString(const FOnlineError&) const = 0;
};

class FOnlineErrorDetails : public IOnlineErrorDetails
{
public:
	FOnlineErrorDetails(FString&& InFriendlyErrorCode, FString&& InLogString, FText&& InText) 
		: FriendlyErrorCode(MoveTemp(InFriendlyErrorCode))
		, LogString(MoveTemp(InLogString))
		, Text(MoveTemp(InText))
	{
	}

	virtual ~FOnlineErrorDetails() = default;

	virtual FString GetFriendlyErrorCode(const FOnlineError&) const override
	{
		return FriendlyErrorCode;
	}

	virtual FString GetLogString(const FOnlineError&) const override
	{
		return LogString;
	}

	virtual FText GetText(const FOnlineError&) const override
	{
		return Text;
	}

protected:
	FOnlineErrorDetails()
	{
	}

	FString FriendlyErrorCode;
	FString LogString;
	FText Text;
};




class FOnlineError
{
public:
	explicit FOnlineError(ErrorCodeType InErrorCode, TSharedPtr<const IOnlineErrorDetails, ESPMode::ThreadSafe> InDetails = nullptr, TSharedPtr<const FOnlineError, ESPMode::ThreadSafe> InInner = nullptr)
		: Details(InDetails)
		, Inner(InInner)
		, ErrorCode(InErrorCode)
	{
	}
	explicit FOnlineError(bool) = delete;

	FText GetText() const
	{
		if (Details)
		{
			return Details->GetText(*this); // alternatively could be done via lookup on an error registry
		}
		else
		{
			return FText::FromString(GetErrorId()); // generic error message with code
		}
	}

	FString GetLogString(bool bIncludePrefix = true) const
	{
#if !NO_LOGGING
		FString MyLogString = TEXT("");

		if (Details)
		{
			// likely also print out the error code first, with Details->GetLogString only providing additional information like a human readable string
			FString LogPrefix = TEXT("");
			if(bIncludePrefix)
			{
				LogPrefix = FString::Printf(TEXT("[%s] "), *GetErrorId());
			}

			FString LogString = Details->GetLogString(*this);
			MyLogString = FString::Printf(TEXT("%s%s"), *LogPrefix, *Details->GetLogString(*this));
		}
		else
		{
			MyLogString = FString::Printf(TEXT("[%s]"), *GetErrorId()); 
		}

		if (GetInner() != nullptr)
		{
			FString InnerLogStr = GetInner()->GetLogString(false);
			if (!InnerLogStr.IsEmpty())
			{
				return FString::Printf(TEXT("%s (%s)"), *MyLogString, *InnerLogStr);
			}
			else
			{
				return MyLogString;
			}
		}

		return MyLogString;
#else
		return TEXT("");
#endif
	}

	FString GetErrorId() const
	{
		FString ErrorId = FString::Printf(TEXT("%s"), *Errors::ErrorCode::ToString(ErrorCode));

		const FOnlineError* LoopInner = GetInner();
		while (LoopInner != nullptr)
		{
			ErrorId += FString::Printf(TEXT("-%s"), *Errors::ErrorCode::ToString(LoopInner->ErrorCode));
			LoopInner = LoopInner->GetInner();
		}

		return ErrorId;
	}

	FString GetFriendlyErrorCode() const
	{
		if (Details)
		{
			return Details->GetFriendlyErrorCode(*this);
		}
		else
		{
			return GetErrorId();
		}
	}

	const FOnlineError* GetInner() const
	{
		return Inner.Get();
	}

	const IOnlineErrorDetails* GetErrorDetails() const
	{
		return Details.Get();
	}

	ErrorCodeType GetSystem() const
	{
		return Errors::ErrorCodeSystem(ErrorCode);
	}

	ErrorCodeType GetCategory() const
	{
		return Errors::ErrorCodeCategory(ErrorCode);
	}

	ErrorCodeType GetValue() const
	{
		return Errors::ErrorCodeValue(ErrorCode);
	}

	ErrorCodeType GetErrorCode() const
	{
		return ErrorCode;
	}

	ONLINESERVICESINTERFACE_API bool IsSuccess() const;

private:
	// TSharedPtr instead of TUniquePtr so that we can copy errors easily
	TSharedPtr<const IOnlineErrorDetails, ESPMode::ThreadSafe> Details;
	TSharedPtr<const FOnlineError, ESPMode::ThreadSafe> Inner;
	ErrorCodeType ErrorCode;

	ONLINESERVICESINTERFACE_API friend void SerializeForLog(FCbWriter& Writer, const FOnlineError& OnlineError);
};





// the comparison with ErrorCodeType is the one I would expect to see in almost every case
ONLINESERVICESINTERFACE_API bool operator==(const FOnlineError& Lhs, const FOnlineError& Rhs);
inline bool operator!=(const FOnlineError& Lhs, const FOnlineError& Rhs) { return !(Lhs == Rhs); }
ONLINESERVICESINTERFACE_API bool operator==(const FOnlineError& OnlineError, ErrorCodeType OtherErrorCode);
inline bool operator==(ErrorCodeType OtherErrorCode, const FOnlineError& OnlineError) { return OnlineError == OtherErrorCode; }
inline bool operator!=(const FOnlineError& OnlineError, ErrorCodeType OtherErrorCode) { return !(OnlineError == OtherErrorCode); }
inline bool operator!=(ErrorCodeType OtherErrorCode, const FOnlineError& OnlineError) { return !(OnlineError == OtherErrorCode); }

inline FString ToLogString(const FOnlineError& Error)
{
	return Error.GetLogString();
}

} // namespace UE::Online

#undef LOCTEXT_NAMESPACE
