// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"

class FString;
template <typename OptionalType> struct TOptional;

namespace UE::Online::Errors {
using ErrorCodeType = uint64;


// error code as struct with bitfields
struct FOnlineErrorCode
{
	uint64 System : 4; // Engine, Game, Third Party Plugin
	uint64 Category : 28; // HRESULT, Http, nn::Result, etc
	uint64 Code : 32; // HRESULT value, Http status code, nn::Result underlying int value, etc
};

namespace ErrorCode
{
	ONLINESERVICESINTERFACE_API FString ToString(ErrorCodeType ErrorCode);
	ONLINESERVICESINTERFACE_API TOptional<ErrorCodeType> FromString(FStringView ErrorCode);

	constexpr ErrorCodeType Create(uint32 System, uint32 Category, uint32 Code)
	{
		return ((System & 0xfull) << 60ull) | ((Category & 0x0fffffffull) << 32ull) | static_cast<ErrorCodeType>(Code);
	}

	namespace System
	{
		inline constexpr ErrorCodeType Engine = 1;
		inline constexpr ErrorCodeType Game = 2;
		inline constexpr ErrorCodeType ThirdPartyPlugin = 3;
	}
}

// These three functions are responsible for reading the parts of the uint64 error representation
// The uint64 error is formatted as follows (big endian):
// ssss cccc cccc cccc cccc cccc cccc cccc vvvv vvvv vvvv vvvv vvvv vvvv vvvv vvvv 
ONLINESERVICESINTERFACE_API uint64 ErrorCodeSystem(ErrorCodeType ErrorCode);
ONLINESERVICESINTERFACE_API uint64 ErrorCodeCategory(ErrorCodeType ErrorCode);
ONLINESERVICESINTERFACE_API uint64 ErrorCodeValue(ErrorCodeType ErrorCode); 

// define an error category
#define UE_ONLINE_ERROR_CATEGORY(Name, InSystem, Value, Description) \
namespace ErrorCode { namespace Category { static constexpr int Category_Value_##Value##_Is_Taken_In_##InSystem = 0; } }\
namespace ErrorCode { namespace Category { static constexpr int Name = Value; } }\
namespace ErrorCode { namespace Category { static constexpr int Name##_System = UE::Online::Errors::ErrorCode::System::InSystem; } }  

// reserve an error category range
#define UE_ONLINE_ERROR_CATEGORY_RANGE(Name, System, StartValue, EndValue, Description) \
namespace ErrorCode { namespace Category { \
    static constexpr Name##_Start = StartValue; \
    static constexpr Name##_End = EndValue; \
} }

// Defines an individual error. Not intended to be used directly.
//	- Example access: (UE::Online::Errors::)NoConnection();
	// - Can also specify an inner, e.g. RequestFailure(InvalidCreds());
//  - Parameters:
	// - CategoryName: The name of a category previously defined by ONLINE_ERROR_CATEGORY
	// - Name: Code-facing name
	// - ErrorCodeValue: Unique identifiable integer that will encode this error string when displaying the condensed version to end user
	// - ErrorMessage: Message displayed in the log when this error is printed
	// - ErrorText: End-User-facing message shown when this error occurs
	// todo: Do we need to add a value in here to have a "System" other than "Engine"? Maybe we can tie this to the category?
#define UE_ONLINE_ERROR_INTERNAL(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText) \
	inline UE::Online::FOnlineError Name(const TOptional<UE::Online::FOnlineError>& Inner = TOptional<UE::Online::FOnlineError>()) \
		{ \
			TSharedPtr<UE::Online::FOnlineError, ESPMode::ThreadSafe> InnerPtr; \
			if (Inner) \
			{ \
				InnerPtr = MakeShared<UE::Online::FOnlineError, ESPMode::ThreadSafe>(Inner.GetValue()); \
			} \
			return UE::Online::FOnlineError( UE::Online::Errors::ErrorCode::Create(ErrorCode::Category::CategoryName##_System, ErrorCode::Category::CategoryName, ErrorCodeValue), MakeShared<UE::Online::FOnlineErrorDetails, ESPMode::ThreadSafe>(FString(#CategoryName"."#Name), ErrorMessage, ErrorText), InnerPtr); \
		}

/** Macro to define an error within a category. Must be used within the UE::Online::Errors namespace. The error will be accessible as UE::Online::Errors::CategoryName::Name() */
#define UE_ONLINE_ERROR(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText) \
    namespace ErrorCode { namespace CategoryName { static constexpr ErrorCodeType Name = Create(Category::CategoryName##_System, Category::CategoryName, ErrorCodeValue); } } \
	namespace CategoryName { \
		UE_ONLINE_ERROR_INTERNAL(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText)\
	}

/** Macro to define a common error in the UE::Online::Errors namespace. Must be used within the UE::Online::Errors namespace. The error will be accessible as UE::Online::Errors::Name() */
#define UE_ONLINE_ERROR_COMMON(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText) \
    namespace ErrorCode { namespace CategoryName { static constexpr ErrorCodeType Name = Create(Category::CategoryName##_System, Category::CategoryName, ErrorCodeValue); } } \
	UE_ONLINE_ERROR_INTERNAL(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText)

/** Macro to define an error in a user defined namespace. The error will be accessible as CurrentNamespace::Name() */
#define UE_ONLINE_ERROR_EXTERNAL(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText) \
    namespace ErrorCode { namespace CategoryName { static constexpr UE::Online::Errors::ErrorCodeType Name = UE::Online::Errors::ErrorCode::Create(Category::CategoryName##_System, ErrorCode::Category::CategoryName, ErrorCodeValue); } } \
	UE_ONLINE_ERROR_INTERNAL(CategoryName, Name, ErrorCodeValue, ErrorMessage, ErrorText)

} /* namespace UE::Online::Errors */
