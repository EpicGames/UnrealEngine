// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/OnlineError.h"
#include "Online/OnlineErrorCode.h"

#define LOCTEXT_NAMESPACE "OnlineErrors"

namespace UE::Online::Errors {

UE_ONLINE_ERROR_CATEGORY(Common, Engine, 0x1, "Online Services")

UE_ONLINE_ERROR_COMMON(Common, Success, 0, TEXT("success"), LOCTEXT("Success", "Success"))
UE_ONLINE_ERROR_COMMON(Common, NoConnection, 1, TEXT("no_connection"), LOCTEXT("NotConnected", "No valid connection"))
UE_ONLINE_ERROR_COMMON(Common, RequestFailure, 2, TEXT("request_failure"), LOCTEXT("RequestFailure", "Failed to send request"))
UE_ONLINE_ERROR_COMMON(Common, InvalidCreds, 3, TEXT("invalid_creds"), LOCTEXT("InvalidCreds", "Invalid credentials"))
UE_ONLINE_ERROR_COMMON(Common, InvalidUser, 4, TEXT("invalid_user"), LOCTEXT("InvalidUser", "No valid user"))
UE_ONLINE_ERROR_COMMON(Common, InvalidAuth, 5, TEXT("invalid_auth"), LOCTEXT("InvalidAuth", "No valid auth"))
UE_ONLINE_ERROR_COMMON(Common, AccessDenied, 6, TEXT("access_denied"), LOCTEXT("AccessDenied", "Access denied"))
UE_ONLINE_ERROR_COMMON(Common, TooManyRequests, 7, TEXT("too_many_requests"), LOCTEXT("TooManyRequests", "Too many requests"))
UE_ONLINE_ERROR_COMMON(Common, AlreadyPending, 8, TEXT("already_pending"), LOCTEXT("AlreadyPending", "Request already pending"))
UE_ONLINE_ERROR_COMMON(Common, InvalidParams, 9, TEXT("invalid_params"), LOCTEXT("InvalidParams", "Invalid params specified"))
UE_ONLINE_ERROR_COMMON(Common, CantParse, 10, TEXT("cant_parse"), LOCTEXT("CantParse", "Cannot parse results"))
UE_ONLINE_ERROR_COMMON(Common, InvalidResults, 11, TEXT("invalid_results"), LOCTEXT("InvalidResults", "Results were invalid"))
UE_ONLINE_ERROR_COMMON(Common, IncompatibleVersion, 12, TEXT("incompatible_version"), LOCTEXT("IncompatibleVersion", "Incompatible client version"))
UE_ONLINE_ERROR_COMMON(Common, NotConfigured, 13, TEXT("not_configured"), LOCTEXT("NotConfigured", "No valid configuration"))
UE_ONLINE_ERROR_COMMON(Common, NotImplemented, 14, TEXT("not_implemented"), LOCTEXT("NotImplemented", "Not implemented"))
UE_ONLINE_ERROR_COMMON(Common, MissingInterface, 15, TEXT("missing_interface"), LOCTEXT("MissingInterface", "Interface not found"))
UE_ONLINE_ERROR_COMMON(Common, Cancelled, 16, TEXT("cancelled"), LOCTEXT("Cancelled", "Operation was cancelled"))
UE_ONLINE_ERROR_COMMON(Common, NotLoggedIn, 17, TEXT("not_logged_in"), LOCTEXT("NotLoggedIn", "User is not logged in"))
UE_ONLINE_ERROR_COMMON(Common, NotFound, 18, TEXT("not_found"), LOCTEXT("NotFound", "Request not found"))
UE_ONLINE_ERROR_COMMON(Common, WillRetry, 19, TEXT("will_retry"), LOCTEXT("WillRetry", "Retrying request"))
UE_ONLINE_ERROR_COMMON(Common, Timeout, 20, TEXT("timeout"), LOCTEXT("Timeout", "Operation timed out"))
UE_ONLINE_ERROR_COMMON(Common, InvalidState, 21, TEXT("invalid_state"), LOCTEXT("InvalidState", "Invalid state"))
UE_ONLINE_ERROR_COMMON(Common, Unknown, 22, TEXT("unknown"), LOCTEXT("Unknown", "Unknown Error"))
UE_ONLINE_ERROR_COMMON(Common, NoChange, 23, TEXT("no_change"), LOCTEXT("NoChange", "No change"))
UE_ONLINE_ERROR_COMMON(Common, AlreadyInSession, 24, TEXT("already_in_session"), LOCTEXT("AlreadyInSession", "Player is already in session"))
UE_ONLINE_ERROR_COMMON(Common, NotInSession, 25, TEXT("not_in_session"), LOCTEXT("NotInSession", "Player is not in any session"))
UE_ONLINE_ERROR_COMMON(Common, SessionJoinDenied, 26, TEXT("session_join_denied"), LOCTEXT("SessionJoinDenied", "Server denied session join"))
UE_ONLINE_ERROR_COMMON(Common, SessionFull, 27, TEXT("session_full"), LOCTEXT("SessionFull", "Server session is full"))
} /* namespace UE::Online::Errors */

#undef LOCTEXT_NAMESPACE
