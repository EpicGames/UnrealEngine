// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Tuple.h"

namespace UE::Online::Meta {

/**
 * Concept that checks for the existence of a Super typedef/using declaration
 */
struct CSuperDefined
{
	template <typename T>
	auto Requires() -> typename T::Super&;
};

namespace Private {

template <typename T, bool THasSuper = TModels_V<CSuperDefined, T>>
struct TBaseClassHelper
{
};

template <typename T>
struct TBaseClassHelper<T, false>
{
	using Type = T;
};

template <typename T>
struct TBaseClassHelper<T, true>
{
	using Type = typename TBaseClassHelper<typename T::Super>::Type;
};

template <typename StructType, typename FieldType>
struct TField
{
	constexpr TField(FieldType StructType::* InPointer, const TCHAR* InName) : Pointer(InPointer), Name(InName) {}
	using Type = FieldType;
	Type StructType::* Pointer;
	const TCHAR* Name;
};

template <typename StructType, typename FieldType>
constexpr TField<StructType, FieldType> MakeTField(FieldType StructType::* Pointer, const TCHAR* Name)
{
	return TField<StructType, FieldType>(Pointer, Name);
}

/* Private*/ }

/**
 * If all types in a class hierarchy have a Super typedef or using declaration, Type will be the type of the last Super definition
 */
template <typename T>
struct TBaseClass
{
	using Type = typename Private::TBaseClassHelper<T>::Type;
};

/**
 * Convenience template alias for TBaseClass<T>::Type
 */
template <typename T>
using TBaseClass_T = typename TBaseClass<T>::Type;

/**
 * Struct that needs to be specialized to provide the Fields in the struct. Defined using the ONLINE_STRUCT macros below
 */
template <typename StructType>
struct TStructDetails;

/**
 * Struct field definitions, manually defined via the ONLINE_STRUCT macros below
 */
template <typename StructType>
struct TStruct : public TStructDetails<std::remove_cv_t<std::remove_reference_t<StructType>>>
{
};

/**
 * Visit the TFields for a struct. TStructDetails must be specialized for the StructType
 * 
 * @param Func The functor to apply.
 */
template <typename StructType, typename FuncType>
inline void VisitFields(FuncType&& Func)
{
	VisitTupleElements(Forward<FuncType>(Func), TStruct<StructType>::Fields());
}

/**
 * Visit the fields of a struct. TStructDetails must be specialized for the StructType
 *
 * @param Object The object whose fields will be applied to the functor
 * @param Func The functor to apply.
 */
template <typename StructType, typename FuncType>
inline void VisitFields(StructType&& Object, FuncType&& Func)
{
	VisitFields<StructType>(
		[&Object, &Func](const auto& Field)
		{
			Func(Field.Name, Object.*Field.Pointer);
		});
}

/**
 * Visit the TFields of a struct. TStructDetails must be specialized for the StructType
 *
 * @param Object The object whose fields will be applied to the functor
 * @param Func The functor to apply.
 */
template <typename StructType, typename FuncType>
inline void VisitFields(const StructType& Object, FuncType&& Func)
{
	VisitFields<StructType>(
		[&Object, &Func](const auto& Field)
		{
			Func(Field.Name, Object.*Field.Pointer);
		});
}

/**
 * Concept that checks if we have metadata for a struct
 */
struct COnlineMetadataAvailable
{
	template <typename T>
	auto Requires() -> decltype(TStructDetails<std::remove_cv_t<std::remove_reference_t<T>>>::Fields());
};

/* UE::Online::Meta*/ }

/**
 * The following macros are used to define the metadata for Online structs
 * Must be inside of the UE::Online::Meta namespace
 */
#define BEGIN_ONLINE_STRUCT_META(StructType) template <> struct TStructDetails<StructType> { static const auto& Fields() { static auto Fields = MakeTuple(
#define ONLINE_STRUCT_FIELD(StructType, FieldName) Private::MakeTField(&StructType::FieldName, TEXT(#FieldName))
#define ONLINE_STRUCT_FIELD_NAMED(StructType, FieldName, FieldNameString) Private::MakeTField(&StructType::FieldName, FieldNameString)
#define END_ONLINE_STRUCT_META() ); return Fields; } };
