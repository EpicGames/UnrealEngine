// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "Misc/TVariant.h"
#include "Online/OnlineUtils.h"


namespace UE::Online { class FOnlineError; }

namespace UE::Online {

/**
 * A container that holds either a successful result or an erroneous result.
 *
 * @param SuccessType - the type to use for successful results
 * @param ErrorType - the type to use for erroneous results
 */
template <typename SuccessType, typename ErrorType>
class TResult
{
public:
	/**
	 * Construct a result with an Ok value
	 */
	explicit TResult(const SuccessType& OkValue)
		: Storage(TInPlaceType<SuccessType>(), OkValue)
	{
	}

	/**
	 * Construct a result with an Ok value
	 */
	explicit TResult(SuccessType&& OkValue)
		: Storage(TInPlaceType<SuccessType>(), MoveTemp(OkValue))
	{
	}

	/**
	 * Construct a result with an error value
	 */
	explicit TResult(const ErrorType& ErrValue)
		: Storage(TInPlaceType<ErrorType>(), ErrValue)
	{
	}

	/**
	 * Construct a result with an error value
	 */
	explicit TResult(ErrorType&& ErrValue)
		: Storage(TInPlaceType<ErrorType>(), MoveTemp(ErrValue))
	{
	}

	/**
	 * Copy construct a result of the same type
	 */
	TResult(const TResult& Other) = default;

	/**
	 * Move construct a result
	 */
	TResult(TResult&& Other) = default;

	/**
	 * Assign a result from another result
	 */
	TResult& operator=(const TResult& Other)
	{
		if (&Other != this)
		{
			Storage = Other.Storage;
		}
		return *this;
	}

	/**
	 * Move-assign a result
	 */
	TResult& operator=(TResult&& Other)
	{
		if (&Other != this)
		{
			Storage = MoveTemp(Other.Storage);
		}
		return *this;
	}

	virtual ~TResult() = default;

	/**
	 * Check if the value held in the result is a SuccessType.
	 *
	 * If the held value is a SuccessType, it is permissible to call GetOkValue. Otherwise it is permissible to call GetErrorValue.
	 *
	 * @return true if the held value is a SuccessType, false otherwise.
	 */
	bool IsOk() const
	{
		return Storage.template IsType<SuccessType>();
	}

	/**
	 * Check if the value held in the result is a ErrorType.
	 *
	 * If the held value is a ErrorType, it is permissible to call GetErrorValue. Otherwise it is permissible to call GetOkValue.
	 *
	 * @return true if the held value is a ErrorType, false otherwise.
	 */
	bool IsError() const
	{
		return Storage.template IsType<ErrorType>();
	}

	/** Get the Ok value stored in the result. This must not be called on a result holding the error type */
	const SuccessType& GetOkValue() const
	{
		checkf(IsOk(), TEXT("It is an error to call GetOkValue() on a TResult that does not hold an ok value. Please either check IsOk() or use TryGetOkValue"));
		return Storage.template Get<SuccessType>();
	}

	/** Get the Ok value stored in the result. This must not be called on a result holding the error type */
	SuccessType& GetOkValue()
	{
		checkf(IsOk(), TEXT("It is an error to call GetOkValue() on a TResult that does not hold an ok value. Please either check IsOk() or use TryGetOkValue"));
		return Storage.template Get<SuccessType>();
	}

	/** Get the Error value stored in the result. This must not be called on a result holding the success type */
	const ErrorType& GetErrorValue() const
	{
		checkf(IsError(), TEXT("It is an error to call GetErrorValue() on a TResult that does not hold an error value. Please either check IsError() or use TryGetErrorValue"));
		return Storage.template Get<ErrorType>();
	}

	/** Get the Error value stored in the result. This must not be called on a result holding the success type */
	ErrorType& GetErrorValue()
	{
		checkf(IsError(), TEXT("It is an error to call GetErrorValue() on a TResult that does not hold an error value. Please either check IsError() or use TryGetErrorValue"));
		return Storage.template Get<ErrorType>();
	}

	/**
	 * Convert from TResult<Success, Error> to Success* if the result is successful.
	 *
	 * @return A pointer to a value of type SuccessType if the result is successful, otherwise nullptr
	 */
	const SuccessType* TryGetOkValue() const
	{
		return const_cast<TResult*>(this)->TryGetOkValue();
	}

	/**
	 * Convert from TResult<Success, Error> to Success* if the result is successful.
	 *
	 * @return A pointer to a value of type SuccessType if the result is successful, otherwise nullptr
	 */
	SuccessType* TryGetOkValue()
	{
		return Storage.template TryGet<SuccessType>();
	}

	/**
	 * Convert from TResult<Success, Error> to Error* if the result is erroneous.
	 *
	 * @return A pointer to a value of type ErrorType if the result is erroneous, otherwise nullptr
	 */
	const ErrorType* TryGetErrorValue() const
	{
		return const_cast<TResult*>(this)->TryGetErrorValue();
	}

	/**
	 * Convert from TResult<Success, Error> to Error* if the result is erroneous.
	 *
	 * @return A pointer to a value of type ErrorType if the result is erroneous, otherwise nullptr
	 */
	ErrorType* TryGetErrorValue()
	{
		return Storage.template TryGet<ErrorType>();
	}

	/**
	 * Unwraps the result, returning the success value if one is held, otherwise returning the default passed.
	 *
	 * @param DefaultValue - The value to return if the result does not hold a success type.
	 */
	const SuccessType& GetOkOrDefault(const SuccessType& DefaultValue) const
	{
		return IsOk() ? GetOkValue() : DefaultValue;
	}


protected:
	TResult() = default;

private:
	/** Location that the result's value is stored */
	TVariant<SuccessType, ErrorType> Storage;
};

template <typename OpType>
class TOnlineResult : public TResult<typename OpType::Result, FOnlineError>
{
public:
	using TResult<typename OpType::Result, FOnlineError>::TResult;
};

template <typename T>
FString ToLogString(const TOnlineResult<T>& Result)
{
	if (Result.IsOk())
	{
		return ToLogString(Result.GetOkValue());
	}
	else
	{
		return ToLogString(Result.GetErrorValue());
	}
}

/* UE::Online */ }
