// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"
#include "Online/OnlineTypeInfo.h"

namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

class FString;

namespace UE::Online {

enum class EOnlinePlatformType : uint8
{
	Epic,
	Steam,
	PSN,
	Nintendo,
	XBL,
	Unknown
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EOnlinePlatformType EnumVal);
ONLINESERVICESINTERFACE_API void LexFromString(EOnlinePlatformType& OutPlatformType, const TCHAR* InStr);

// Interfaces
using IOnlineServicesPtr = TSharedPtr<class IOnlineServices>;
using IAchievementsPtr = TSharedPtr<class IAchievements>;
using IAuthPtr = TSharedPtr<class IAuth>;
using ICommercePtr = TSharedPtr<class ICommerce>;
using IUserInfoPtr = TSharedPtr<class IUserInfo>;
using ISocialPtr = TSharedPtr<class ISocial>;
using IPresencePtr = TSharedPtr<class IPresence>;
using IExternalUIPtr = TSharedPtr<class IExternalUI>;
using ILeaderboardsPtr = TSharedPtr<class ILeaderboards>;
using ILobbiesPtr = TSharedPtr<class ILobbies>;
using ISessionsPtr = TSharedPtr<class ISessions>;
using IStatsPtr = TSharedPtr<class IStats>;
using IConnectivityPtr = TSharedPtr<class IConnectivity>;
using IPrivilegesPtr = TSharedPtr<class IPrivileges>;
using ITitleFilePtr = TSharedPtr<class ITitleFile>;
using IUserFilePtr = TSharedPtr<class IUserFile>;

struct FGetResolvedConnectString
{
	static constexpr TCHAR Name[] = TEXT("GetResolvedConnectString");

	struct Params
	{
		FAccountId LocalAccountId;
		FLobbyId LobbyId;
		FOnlineSessionId SessionId;
		FName PortType;
	};

	struct Result
	{
		FString ResolvedConnectString;
	};
};

class IOnlineServices : public TSharedFromThis<IOnlineServices>
{
public:
	virtual ~IOnlineServices() = default;

	/**
	 *
	 */
	virtual void Init() = 0;

	/**
	 *
	 */
	virtual void Destroy() = 0;

	/**
	 *
	 */
	virtual IAchievementsPtr GetAchievementsInterface() = 0;

	/**
	 *
	 */
	virtual IAuthPtr GetAuthInterface() = 0;

	/**
	 *
	 */
	virtual ICommercePtr GetCommerceInterface() = 0;

	/**
	 *
	 */
	virtual IUserInfoPtr GetUserInfoInterface() = 0;


	/**
	 * Get the social interface, used to interact with friends lists, blocked user lists, and any other social relationships
	 * @return social interface implementation, may be null if not implemented for this service
	 */
	virtual ISocialPtr GetSocialInterface() = 0;

	/**
	 *
	 */
	virtual IPresencePtr GetPresenceInterface() = 0;

	/**
	 *
	 */
	virtual IExternalUIPtr GetExternalUIInterface() = 0;

	/**
	 * Get the leaderboards implementation
	 * @return leaderboards implementation, may be null if not implemented for this service
	 */
	virtual ILeaderboardsPtr GetLeaderboardsInterface() = 0;

	/**
	 * Get the lobbies implementation
	 * @return lobbies implementation, may be null if not implemented for this service
	 */
	virtual ILobbiesPtr GetLobbiesInterface() = 0;

	/**
	 * Get the sessions implementation
	 * @return sessions implementation, may be null if not implemented for this service
	 */
	virtual ISessionsPtr GetSessionsInterface() = 0;

	/**
	 * Get the stats implementation
	 * @return stats implementation, may be null if not implemented for this service
	 */
	virtual IStatsPtr GetStatsInterface() = 0;

	/**
	 * 
	 */
	virtual IConnectivityPtr GetConnectivityInterface() = 0;

	/**
	 *
	 */
	virtual IPrivilegesPtr GetPrivilegesInterface() = 0;

	/**
	 *
	 */
	virtual ITitleFilePtr GetTitleFileInterface() = 0;
	
	/**
	 *
	 */
	virtual IUserFilePtr GetUserFileInterface() = 0;

	/** 
	 * Get the connectivity string used for client travel
	 */
	virtual TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(FGetResolvedConnectString::Params&& Params) = 0;

	/**
	 * Get the provider type of this instance.
	 * @return provider type
	 */
	virtual EOnlineServices GetServicesProvider() const = 0;

	/**
	 * Get the instance name of this instance. 
	 * @return instance name
	 */
	virtual FName GetInstanceName() const = 0;

	/**
	* Get the instance config name for this instance. 
	* @return instance config name
	*/
	virtual FName GetInstanceConfigName() const = 0;

	/**
	 * Retrieve any of the standard or implementation specific interfaces
	 */
	template <typename InterfaceType>
	TSharedPtr<InterfaceType> GetInterface()
	{
		const FOnlineTypeName TypeName = TOnlineTypeInfo<InterfaceType>::GetTypeName();
		using BaseInterfaceType = Meta::TBaseClass_T<InterfaceType>;
		TSharedPtr<BaseInterfaceType> Interface;
		AssignBaseInterfaceSharedPtr(TypeName, &Interface);
		if constexpr (std::is_same_v<BaseInterfaceType, InterfaceType>)
		{
			return Interface;
		}
		else
		{
			return StaticCastSharedPtr<InterfaceType>(Interface);
		}
	}

private:
	virtual void AssignBaseInterfaceSharedPtr(const FOnlineTypeName& TypeName, void* InterfaceSP) = 0;
};

/**
 * Retrieve the unique id for the executable build. The build id is used for ensuring that client
 * and server builds are compatible.
 *
 * @return The unique id for the executable build.
 */
ONLINESERVICESINTERFACE_API int32 GetBuildUniqueId();

/**
 * Check if an instance of the online service is loaded
 *
 * @param OnlineServices Type of online services for the IOnlineServices instance
 * @param InstanceName Name of the instance
 * @return true if the instance is loaded
 */
ONLINESERVICESINTERFACE_API bool IsLoaded(EOnlineServices OnlineServices = EOnlineServices::Default, FName InstanceName = NAME_None, FName InstanceConfigName = NAME_None);

/**
 * Get an instance of the online service
 *
 * @param OnlineServices Type of online services to retrieve
 * @param InstanceName Name of the services instance to retrieve
 * @param InstanceConfigName Name of the implementation specific config to use for this services instance
 * @return The services instance or an invalid pointer if the services is unavailable
 */
ONLINESERVICESINTERFACE_API TSharedPtr<IOnlineServices> GetServices(EOnlineServices OnlineServices = EOnlineServices::Default, FName InstanceName = NAME_None, FName InstanceConfigName = NAME_None);

/**
 * Get a specific services type and cast to the specific services type
 *
 * @param InstanceName Name of the services instance to retrieve
 * @return The services instance or an invalid pointer if the services is unavailable
 */
template <typename ServicesClass>
TSharedPtr<ServicesClass> GetServices(FName InstanceName = NAME_None, FName InstanceConfigName = NAME_None)
{
	return StaticCastSharedPtr<ServicesClass>(GetServices(ServicesClass::GetServicesProvider(), InstanceName, InstanceConfigName));
}

/**
 * Destroy an instance of the online service
 *
 * @param OnlineServices Type of online services to destroy
 * @param InstanceName Name of the services instance to destroy
 */
ONLINESERVICESINTERFACE_API void DestroyService(EOnlineServices OnlineServices = EOnlineServices::Default, FName InstanceName = NAME_None, FName InstanceConfigName = NAME_None);

/**
 * Destroy all instances of the online services specified by EOnlineServices type
 *
 * @param OnlineServices Type of online services to destroy all instances from
 */
ONLINESERVICESINTERFACE_API void DestroyAllNamedServices(EOnlineServices OnlineServices);

/**
 * Destroy all instances of the online services with specified instance name
 *
 * @param InstanceName Name of online services to destroy all instances from
 */
ONLINESERVICESINTERFACE_API void DestroyAllServicesWithName(FName InstanceName);

namespace Meta {
// TODO: Move to OnlineServices_Meta.inl file?

BEGIN_ONLINE_STRUCT_META(FGetResolvedConnectString::Params)
	ONLINE_STRUCT_FIELD(FGetResolvedConnectString::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetResolvedConnectString::Params, LobbyId),
	ONLINE_STRUCT_FIELD(FGetResolvedConnectString::Params, PortType)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetResolvedConnectString::Result)
	ONLINE_STRUCT_FIELD(FGetResolvedConnectString::Result, ResolvedConnectString)
END_ONLINE_STRUCT_META()

/* Meta*/}

/* UE::Online */ }
