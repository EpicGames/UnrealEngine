// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Online/CoreOnline.h"
#include "Online/OnlineError.h"

namespace UE::Online {

class FOnlineError;
class IOnlineServices;

/**
 * Online services delegates that are more external to the online services themselves
 */

/**
 * Notification that a new online subsystem instance has been created
 *
 * @param NewSubsystem the new instance created
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnOnlineServicesCreated, TSharedRef<class IOnlineServices> /*NewServices*/);
extern ONLINESERVICESINTERFACE_API FOnOnlineServicesCreated OnOnlineServicesCreated;

// Notification params of OnAsyncOpCompleted delegate
struct FOnAsyncOpCompletedParams
{
	// The name of completed operation
	FString OpName;
	// The name of interface
	FString InterfaceName;
	// The OnlineServices instance
	TWeakPtr<IOnlineServices> OnlineServices;
	// The result of completed operation
	TOptional<UE::Online::FOnlineError> OnlineError;
	// The duration of the operation from start to complete
	double DurationInSeconds = -1.0;
};

/**
 * Notification that an online operation has completed
 *
 * !!!NOTE!!! The notification can happen on off-game threads, make sure the callbacks are thread-safe
 *
 * @param Params the struct FOnAsyncOpCompletedParams which contains related info
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAsyncOpCompleted, const FOnAsyncOpCompletedParams& Params);
extern ONLINESERVICESINTERFACE_API FOnAsyncOpCompleted OnAsyncOpCompleted;

/* UE::Online */ }
