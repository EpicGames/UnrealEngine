// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Online/CoreOnline.h"

namespace UE::Online { class IOnlineServices; }

class FLazySingleton;

namespace UE::Online {
	
enum class EOnlineServices : uint8;

class IOnlineServicesFactory
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~IOnlineServicesFactory() = default;

	/**
	 * Create an IOnlineServices instance
	 *
	 * @return Initialized IOnlineServices instance
	 */
	virtual TSharedPtr<IOnlineServices> Create(FName InstanceName, FName InstanceConfigName) = 0;
};

class FOnlineServicesRegistry
{
public:
	/**
	 * Get the FOnlineServicesRegistry singleton
	 * 
	 * @return The FOnlineServicesRegistry singleton instance
	 */
	ONLINESERVICESINTERFACE_API static FOnlineServicesRegistry& Get();

	/**
	 * Tear down the singleton instance
	 */
	ONLINESERVICESINTERFACE_API static void TearDown();

	/**
	 * Register a factory for creation of IOnlineServices instances
	 * 
	 * @param OnlineServices Services that the factory is for
	 * @param Factory Factory for creation of IOnlineServices instances
	 * @param Priority Integer priority, allows an existing OnlineServices implementation to be extended and registered with a higher priority so it is used instead
	 */
	ONLINESERVICESINTERFACE_API void RegisterServicesFactory(EOnlineServices OnlineServices, TUniquePtr<IOnlineServicesFactory>&& Factory, int32 Priority = 0);

	/**
	 * Unregister a previously registered IOnlineServices factory
	 *
	 * @param OnlineServices Services that the factory is for
	 * @param Priority Integer priority, will be unregistered only if the priority matches the one that is registered
	 */
	ONLINESERVICESINTERFACE_API void UnregisterServicesFactory(EOnlineServices OnlineServices, int32 Priority = 0);

	/**
	 * Check if an online service instance is loaded
	 *
	 * @param OnlineServices Type of online services for the IOnlineServices instance
	 * @param InstanceName Name of the instance
	 * @param InstanceConfigName Name of the config
	 *
	 * @return true if the instance is loaded
	 */
	ONLINESERVICESINTERFACE_API bool IsLoaded(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName = NAME_None) const;

	/**
	 * Get a named instance of a specific IOnlineServices
	 * 
	 * @param OnlineServices Type of online services for the IOnlineServices instance
	 * @param InstanceName Name of the instance
	 * @param InstanceConfigName Name of the config to use
	 * 
	 * @return The services instance, or an invalid pointer if the OnlineServices is unavailable
	 */
	ONLINESERVICESINTERFACE_API TSharedPtr<IOnlineServices> GetNamedServicesInstance(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName);

	/**
	 * Destroy a named instance of a specific OnlineServices
	 *
	 * @param OnlineServices Type of online services for the IOnlineServices instance
	 * @param InstanceName Name of the instance
	 * @param InstanceConfigName Name of the config
	 */
	ONLINESERVICESINTERFACE_API void DestroyNamedServicesInstance(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName);

	/**
	 * Destroy all instances of a specific OnlineServices
	 *
	 * @param OnlineServices  Type of online services for the IOnlineServices instance
	 */
	ONLINESERVICESINTERFACE_API void DestroyAllNamedServicesInstances(EOnlineServices OnlineServices);

	/**
	 * Destroy all instances of a specific InstanceName
	 *
	 * @param InstanceName  Name of online services for the IOnlineServices instance
	 */
	ONLINESERVICESINTERFACE_API void DestroyAllServicesInstancesWithName(FName InstanceName);

	/**
	 * Create and initialize a new IOnlineServices instance
	 *
	 * @param OnlineServices Type of online services for the IOnlineServices instance
	 * @param InstanceName Name of the instance
	 * @param InstanceConfigName Name of the config
	 * 
	 * @return The initialized IOnlineServices instance, or an invalid pointer if the OnlineServices is unavailable
	 */
	ONLINESERVICESINTERFACE_API TSharedPtr<IOnlineServices> CreateServices(EOnlineServices OnlineServices, FName InstanceName, FName InstanceConfigName);

	/**
	 * Get list of all instantiated OnlineServices
	 * 
	 * @param OutOnlineServices Array of online services to fill
	 */
	ONLINESERVICESINTERFACE_API void GetAllServicesInstances(TArray<TSharedRef<IOnlineServices>>& OutOnlineServices) const;

#if WITH_DEV_AUTOMATION_TESTS
	/**
	 * Adds a temporary override to the default online service. Used for testing to quickly run tests with different default online services.
	 * 
	 * @param Service the name of the Online Service to force default online service to return
	 */
	ONLINESERVICESINTERFACE_API void SetDefaultServiceOverride(EOnlineServices DefaultService);

	/**
	 * Removes an override for the target service.
	 */
	ONLINESERVICESINTERFACE_API void ClearDefaultServiceOverride();
#endif //WITH_DEV_AUTOMATION_TESTS
private:
	struct FFactoryAndPriority
	{
		FFactoryAndPriority(TUniquePtr<IOnlineServicesFactory>&& InFactory, int32 InPriority)
		: Factory(MoveTemp(InFactory))
		, Priority(InPriority)
		{
		}

		TUniquePtr<IOnlineServicesFactory> Factory;
		int32 Priority;
	};

	TMap<EOnlineServices, FFactoryAndPriority> ServicesFactories;

	using FInstanceNameInstanceConfigNamePair = TPair<FName, FName>;
	TMap<EOnlineServices, TMap<FInstanceNameInstanceConfigNamePair, TSharedRef<IOnlineServices>>> NamedServiceInstances;
	EOnlineServices DefaultServiceOverride = EOnlineServices::Default;

	FOnlineServicesRegistry() {}
	~FOnlineServicesRegistry();
	friend FLazySingleton;

	/**
	* Resolves a generic (like Platform and Default) service enum value into the value of the real corresponding service.
	*/
	EOnlineServices ResolveServiceName(EOnlineServices OnlineServices) const;
};

/* UE::Online */ }
