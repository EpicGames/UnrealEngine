// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/GeneratedTypeName.h"

#include "Templates/Models.h"
#include "Templates/TypeHash.h"

namespace UE::Online::Meta { struct CSuperDefined; }

namespace UE::Online {

struct FOnlineTypeName
{
	const TCHAR* Name = nullptr;

	bool operator==(const FOnlineTypeName& Other) const
	{
		return FCString::Strcmp(Name, Other.Name) == 0;
	}

	bool operator!=(const FOnlineTypeName& Other) const
	{
		return !operator==(Other);
	}
};

inline uint32 GetTypeHash(const FOnlineTypeName& TypeName)
{
	return FCrc::Strihash_DEPRECATED(TypeName.Name);
}

template <typename T>
struct TOnlineTypeInfo
{
	static inline FOnlineTypeName GetTypeName()
	{
		return FOnlineTypeName{ GetGeneratedTypeName<std::remove_reference_t<T>>() };
	}

	static inline bool IsA(FOnlineTypeName TypeName)
	{
		if (TypeName == GetTypeName())
		{
			return true;
		}

		if constexpr (TModels_V<Meta::CSuperDefined, T>)
		{
			return TOnlineTypeInfo<typename T::Super>::IsA(TypeName);
		}
		else
		{
			return false;
		}
	}
};

/* UE::Online */ }
