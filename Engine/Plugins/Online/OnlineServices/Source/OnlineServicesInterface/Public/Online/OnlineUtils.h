// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineMeta.h"
#include "Concepts/DerivedFrom.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Internationalization/Text.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"

struct FDateTime;

namespace UE::Online {

template <typename T> FString ToLogString(const TSet<T>& Set);
template <typename T, typename U> FString ToLogString(const TPair<T, U>& Pair);
template <typename T, ESPMode Mode> FString ToLogString(const TSharedPtr<T, Mode>& Ptr);
template <typename T, ESPMode Mode> FString ToLogString(const TSharedRef<T, Mode>& Ref);
template <typename T> FString ToLogString(const TOptional<T> Optional);
template <typename... Ts> FString ToLogString(const TVariant<Ts...>& Variant);
template <typename T, ESPMode Mode> FString ToLogString(const TSharedRef<T, Mode>& Ref);
inline FString ToLogString(const FString& String);
inline FString ToLogString(const FName& Name);
inline FString ToLogString(const FUtf8String& Name);
inline FString ToLogString(const FText& Text);
inline FString ToLogString(uint8 Value);
inline FString ToLogString(int8 Value);
inline FString ToLogString(uint16 Value);
inline FString ToLogString(int16 Value);
inline FString ToLogString(uint32 Value);
inline FString ToLogString(int32 Value);
inline FString ToLogString(uint64 Value);
inline FString ToLogString(int64 Value);
inline FString ToLogString(bool Value);
inline FString ToLogString(float Value);
inline FString ToLogString(double Value);
ONLINESERVICESINTERFACE_API FString ToLogString(const FDateTime& Time);
template <typename T> FString ToLogString(const T& Value);

template <typename T>
FString ToLogString(const TSet<T>& Set)
{
	return FString::Printf(TEXT("{%s}"), *FString::JoinBy(Set, TEXT(", "), [](const T& Value) { return ToLogString(Value); }));
}

template <typename T, typename U>
FString ToLogString(const TPair<T, U>& Pair)
{
	return FString::Printf(TEXT("%s:%s"), *ToLogString(Pair.template Get<0>()), *ToLogString(Pair.template Get<1>()));
}

template <typename T, ESPMode Mode>
FString ToLogString(const TSharedPtr<T, Mode>& Ptr)
{
	if (Ptr.IsValid())
	{
		return ToLogString(*Ptr);
	}
	else
	{
		return TEXT("null");
	}
}

template <typename T, ESPMode Mode>
FString ToLogString(const TSharedRef<T, Mode>& Ref)
{
	return ToLogString(*Ref);
}

template <typename T>
FString ToLogString(const TOptional<T> Optional)
{
	if (Optional.IsSet())
	{
		return ToLogString(Optional.GetValue());
	}
	else
	{
		return TEXT("unset");
	}
}

template <typename... Ts>
FString ToLogString(const TVariant<Ts...>& Variant)
{
	return Visit([](const auto& Value)
	{
		return ToLogString(Value);
	}, Variant);
}

inline FString ToLogString(const FString& String)
{
	return String;
}

inline FString ToLogString(const FName& Name)
{
	return Name.ToString();
}

inline FString ToLogString(const FUtf8String& String)
{
	return FString(StringCast<TCHAR>(*String, String.Len()));
}

inline FString ToLogString(const FText& Text)
{
	return Text.ToString();
}

inline FString ToLogString(uint8 Value)
{
	return FString::Printf(TEXT("%hhu"), Value);
}

inline FString ToLogString(int8 Value)
{
	return FString::Printf(TEXT("%hhi"), Value);
}

inline FString ToLogString(uint16 Value)
{
	return FString::Printf(TEXT("%hu"), Value);
}

inline FString ToLogString(int16 Value)
{
	return FString::Printf(TEXT("%hi"), Value);
}

inline FString ToLogString(uint32 Value)
{
	return FString::Printf(TEXT("%u"), Value);
}

inline FString ToLogString(int32 Value)
{
	return FString::Printf(TEXT("%i"), Value);
}

inline FString ToLogString(uint64 Value)
{
	return FString::Printf(TEXT("%llu"), Value);
}

inline FString ToLogString(int64 Value)
{
	return FString::Printf(TEXT("%lli"), Value);
}

inline FString ToLogString(float Value)
{
	return FString::Printf(TEXT("%.2f"), Value);
}

inline FString ToLogString(double Value)
{
	return FString::Printf(TEXT("%.2f"), Value);
}

inline FString ToLogString(bool Value)
{
	return ::LexToString(Value);
}

inline FString ToLogString(FPlatformUserId PlatformUserId)
{
	return ToLogString(PlatformUserId.GetInternalId());
}

template<typename T>
concept CDerivesFromArray =
	requires
	{
		typename T::ElementType;
	}
	&& CDerivedFrom<T, TArray<typename T::ElementType>>;

template<typename T>
concept CDerivesFromMap =
	requires
	{
		typename T::KeyType;
		typename T::ValueType;
	}
	&& CDerivedFrom<T, TMap<typename T::KeyType, typename T::ValueType>>;

template <typename T>
FString ToLogString(const T& Value)
{
	if constexpr (CDerivesFromArray<T>)
	{
		return FString::Printf(TEXT("[%s]"), *FString::JoinBy(Value, TEXT(", "), [](const typename T::ElementType& Element) { return ToLogString(Element); }));
	}
	else if constexpr (CDerivesFromMap<T>)
	{
		return FString::Printf(TEXT("{%s}"), *FString::JoinBy(Value, TEXT(", "), [](const TPair<typename T::KeyType, typename T::ValueType>& Pair) { return ToLogString(Pair); }));
	}
	else if constexpr (TModels_V<Meta::COnlineMetadataAvailable, T>)
	{
		FString LogString;
		LogString += TEXT("{ ");
		bool bFirst = true;
		Meta::VisitFields(Value, [&LogString, &bFirst](const TCHAR* Name, auto& Field)
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					LogString += TEXT(", ");
				}
				LogString += Name;
				LogString += TEXT(": ");
				LogString += ToLogString(Field);
			});
		LogString += TEXT(" }");

		return LogString;
	}
	else
	{
		return LexToString(Value);
	}
}

/* UE::Online */ }