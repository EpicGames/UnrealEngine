// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineError.h"
#include "Online/OnlineErrorCode.h"
#include "Online/OnlineMeta.h"
#include "Online/OnlineServices.h"
#include "Online/NestedVariantJson.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

#define LOCTEXT_NAMESPACE "Presence"

#define SENT_TIME_ATTRIBUTE_KEY FString(TEXT("SentTimeAttributeKey"))
#define SESSION_ID_ATTRIBUTE_KEY FString(TEXT("SessionIdAttributeKey"))
#define PLATFORM_ATTRIBUTE_KEY FString(TEXT("PlatformAttributeKey"))

namespace UE::Online {

using FPresenceVariant = FNestedVariantJson;

namespace Errors {
UE_ONLINE_ERROR_CATEGORY(Presence, Engine, 0x3, "Presence")
//UE_ONLINE_ERROR(Presence, OnlyFriendsSupported, 0, TEXT("OnlyFriendsSupported"), LOCTEXT("OnlyFriendsSupported", "This platform only supports listening to presence updates for friends."))
UE_ONLINE_ERROR_COMMON(Presence, CannotQueryLocalUsers, 1, TEXT("CannotQueryLocalUsers"), LOCTEXT("CannotQueryLocalUsers", "QueryPresence cannot be called on users logged into the game currently"))
}

// todo: add more potential errors here
/* Common errors used:
	NotFound
	ServiceUnavailable
	Forbidden
	RateLimit
*/

enum class EUserPresenceStatus : uint8
{
	/** User is offline */
	Offline,
	/** User is online */
	Online,
	/** User is away */
	Away,
	/** User is away for >2 hours (can change depending on platform) */
	ExtendedAway,
	/** User is in do not disturb mode */
	DoNotDisturb,
	/** Default */
	Unknown
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUserPresenceStatus EnumVal);
ONLINESERVICESINTERFACE_API void LexFromString(EUserPresenceStatus& OutStatus, const TCHAR* InStr);
ONLINESERVICESINTERFACE_API FText ToLocText(EUserPresenceStatus EnumVal);

enum class EUserPresenceJoinability : uint8
{
	/** Anyone can matchmake/discover and join this session */
	Public,
	/**  Anyone trying to join must be a friend of a lobby member */
	FriendsOnly,
	/** Anyone trying to join must have been invited first */
	InviteOnly,
	/** Not currently accepting invites */
	Private,
	/** Default */
	Unknown
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUserPresenceJoinability EnumVal);
ONLINESERVICESINTERFACE_API void LexFromString(EUserPresenceJoinability& OutJoinability, const TCHAR* InStr);

enum class EUserPresenceGameStatus : uint8
{
	/** The user is playing the same game as you */
	PlayingThisGame,
	/** The user is playing a different game than you */
	PlayingOtherGame,
	/** Default */
	Unknown
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUserPresenceGameStatus EnumVal);
ONLINESERVICESINTERFACE_API void LexFromString(EUserPresenceGameStatus& OutGameStatus, const TCHAR* InStr);

using FPresenceProperties = FPresenceVariant::FMap;
using FPresencePropertiesRef = FPresenceVariant::FMapRef;
using FPresencePropertiesPtr = FPresenceVariant::FMapPtr;
using FPresenceProperty = FPresenceProperties::ValueType;

struct FUserPresence
{
	/** User whose presence this is */
	FAccountId AccountId;
	/** Presence state */
	EUserPresenceStatus Status = EUserPresenceStatus::Unknown;
	/** Session state */
	EUserPresenceJoinability Joinability = EUserPresenceJoinability::Unknown;
	/** Playing this game */
	EUserPresenceGameStatus GameStatus = EUserPresenceGameStatus::Unknown;
	/** Stringified representation of the user status (online, offline, whatever the platform wants to define) */
	FString StatusString;
	/** Game-defined representation of the current game state (e.g. "Squads- 4 Remaining" or "Level 5 Warrior") */
	FString RichPresenceString;
	/** Type of the user's current online platform */
	EOnlinePlatformType PlatformType = EOnlinePlatformType::Unknown;
	/** Session keys, and any platform specific presence properties */
	FPresenceProperties Properties;

	bool operator==(const FUserPresence& Other) const = default;
};
	
struct FQueryPresence
{
	static constexpr TCHAR Name[] = TEXT("QueryPresence");

	struct Params
	{
		/** Local user performing the query */
		FAccountId LocalAccountId;
		/** User to query the presence for */
		FAccountId TargetAccountId;
		/** If true, then future presence updates for this user will be propagated via the OnPresenceUpdated event */
		bool bListenToChanges = true;
	};

	struct Result
	{
		/** The retrieved presence */
		TSharedRef<const FUserPresence> Presence;
	};
};


struct FBatchQueryPresence
{
	static constexpr TCHAR Name[] = TEXT("BatchQueryPresence");

	struct Params
	{
		/** Local user performing the query */
		FAccountId LocalAccountId;
		/** Users to query the presence for */
		TArray<FAccountId> TargetAccountIds;
		/** If true, then future presence updates for these users will be propagated via the OnPresenceUpdated event */
		bool bListenToChanges = true;
	};

	struct Result
	{
		/** The retrieved presence */
		TArray<TSharedRef<const FUserPresence>> Presences;
	};
};

struct FGetCachedPresence
{
	static constexpr TCHAR Name[] = TEXT("GetCachedPresence");

	struct Params
	{
		/** Local user getting the presence */
		FAccountId LocalAccountId;
		/** User to get the presence for */
		FAccountId TargetAccountId;
	};

	struct Result
	{
		/** The presence */
		TSharedRef<const FUserPresence> Presence;
	};
};

struct FUpdatePresence
{
	static constexpr TCHAR Name[] = TEXT("UpdatePresence");

	struct Params
	{
		/** Local user performing the query */
		FAccountId LocalAccountId;
		/** The new presence to send to the server */
		TSharedRef<FUserPresence> Presence;
	};

	struct Result
	{
	};
};

struct FPartialUpdatePresence
{
	static constexpr TCHAR Name[] = TEXT("PartialUpdatePresence");

	struct Params
	{
		/** Local user performing the query */
		FAccountId LocalAccountId;


		struct FMutations
		{
			/** Presence state */
			TOptional<EUserPresenceStatus> Status;
			/** Joinability state */
			TOptional<EUserPresenceJoinability> Joinability;
			/** Game status (playing/not playing this game) */
			TOptional<EUserPresenceGameStatus> GameStatus;
			/** Status string */
			TOptional<FString> StatusString;
			/** Rich presence string */
			TOptional<FString> RichPresenceString;
			/** Presence keys to update */
			FPresenceProperties UpdatedProperties;
			/** Properties to remove */
			TArray<FString> RemovedProperties;

			FMutations& operator+=(FMutations&& NewParams);
		};
		FMutations Mutations;

	};

	struct Result
	{
	};
};

inline FPartialUpdatePresence::Params::FMutations& FPartialUpdatePresence::Params::FMutations::operator+=(FPartialUpdatePresence::Params::FMutations&& NewParams)
{
	if (NewParams.Status.IsSet())
	{
		Status = NewParams.Status;
	}

	if (NewParams.Joinability.IsSet())
	{
		Joinability = NewParams.Joinability;
	}

	if (NewParams.GameStatus.IsSet())
	{
		GameStatus = NewParams.GameStatus;
	}

	if (NewParams.StatusString.IsSet())
	{
		StatusString = MoveTemp(NewParams.StatusString);
	}

	if (NewParams.RichPresenceString.IsSet())
	{
		RichPresenceString = MoveTemp(NewParams.RichPresenceString);
	}

	// Insert any updated keys / remove from removed
	for (TPair<FString, FPresenceProperty>& NewUpdatedProperty : NewParams.UpdatedProperties)
	{
		RemovedProperties.Remove(NewUpdatedProperty.Key);
		UpdatedProperties.Emplace(MoveTemp(NewUpdatedProperty.Key), MoveTemp(NewUpdatedProperty.Value));
	}

	// Merge any removed keys / remove from updated
	RemovedProperties.Reserve(RemovedProperties.Num() + NewParams.RemovedProperties.Num());
	for (FString& NewRemovedKey : NewParams.RemovedProperties)
	{
		UpdatedProperties.Remove(NewRemovedKey);
		RemovedProperties.AddUnique(MoveTemp(NewRemovedKey));
	}
	return *this;
}

inline TSharedRef<FUserPresence> ApplyPresenceMutations(const FUserPresence& BasePresence, const FPartialUpdatePresence::Params::FMutations& Mutations)
{
	TSharedRef<FUserPresence> NewPresence = MakeShared<FUserPresence>();
	*NewPresence = BasePresence;

	if (Mutations.Status.IsSet())
	{
		NewPresence->Status = Mutations.Status.GetValue();
	}

	if (Mutations.Joinability.IsSet())
	{
		NewPresence->Joinability = Mutations.Joinability.GetValue();
	}

	if (Mutations.GameStatus.IsSet())
	{
		NewPresence->GameStatus = Mutations.GameStatus.GetValue();
	}

	if (Mutations.StatusString.IsSet())
	{
		NewPresence->StatusString = Mutations.StatusString.GetValue();
	}

	if (Mutations.RichPresenceString.IsSet())
	{
		NewPresence->RichPresenceString = Mutations.RichPresenceString.GetValue();
	}

	// Insert any updated keys / remove from removed
	for (const TPair<FString, FPresenceProperty>& NewUpdatedProperty : Mutations.UpdatedProperties)
	{
		NewPresence->Properties.Add(NewUpdatedProperty.Key, NewUpdatedProperty.Value);
	}

	// Merge any removed keys / remove from updated
	for (const FString& NewRemovedKey : Mutations.RemovedProperties)
	{
		NewPresence->Properties.Remove(NewRemovedKey);
	}

	return NewPresence;
}

/** Struct for PresenceUpdated event */
struct FPresenceUpdated
{
	/** Local user receiving the presence update */
	FAccountId LocalAccountId;
	/** Presence that has updated */
	TSharedRef<const FUserPresence> UpdatedPresence;
};

class IPresence
{
public:
	virtual ~IPresence() = default;

	/**
	 * Query presence for a user
	 * 
	 * @param Params for the QueryPresence call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) = 0;
	
	/**
	 * Queries presences for all users in the array
	 * 
	 * @param Params for the BatchQueryPresence call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FBatchQueryPresence> BatchQueryPresence(FBatchQueryPresence::Params&& Params) = 0;

	/**
	 * Get the presence of a user
	 * Presence typically comes from QueryPresence or push events from the online service
	 *
	 * @param Params for the GetCachedPresence call
	 * @return
	 */
	virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) = 0;

	/**
	 * Update your presence
	 *
	 * @param Params for the UpdatePresence call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) = 0;

	/**
	 * Updates only the parts of the presence that are specified
	 *
	 * @param Params for the UpdatePresence call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) = 0;

	/**
	 * Get the event that is triggered when presence is updated
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FPresenceUpdated&)> OnPresenceUpdated() = 0;
};

namespace Meta {
// TODO: Move to Presence_Meta.inl file?

BEGIN_ONLINE_STRUCT_META(FUserPresence)
	ONLINE_STRUCT_FIELD(FUserPresence, AccountId),
	ONLINE_STRUCT_FIELD(FUserPresence, Status),
	ONLINE_STRUCT_FIELD(FUserPresence, Joinability),
	ONLINE_STRUCT_FIELD(FUserPresence, GameStatus),
	ONLINE_STRUCT_FIELD(FUserPresence, StatusString),
	ONLINE_STRUCT_FIELD(FUserPresence, RichPresenceString),
	ONLINE_STRUCT_FIELD(FUserPresence, PlatformType),
	ONLINE_STRUCT_FIELD(FUserPresence, Properties)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryPresence::Params)
	ONLINE_STRUCT_FIELD(FQueryPresence::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FQueryPresence::Params, TargetAccountId),
	ONLINE_STRUCT_FIELD(FQueryPresence::Params, bListenToChanges)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBatchQueryPresence::Params)
	ONLINE_STRUCT_FIELD(FBatchQueryPresence::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FBatchQueryPresence::Params, TargetAccountIds),
	ONLINE_STRUCT_FIELD(FBatchQueryPresence::Params, bListenToChanges)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryPresence::Result)
	ONLINE_STRUCT_FIELD(FQueryPresence::Result, Presence)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBatchQueryPresence::Result)
	ONLINE_STRUCT_FIELD(FBatchQueryPresence::Result, Presences)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetCachedPresence::Params)
	ONLINE_STRUCT_FIELD(FGetCachedPresence::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetCachedPresence::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetCachedPresence::Result)
	ONLINE_STRUCT_FIELD(FGetCachedPresence::Result, Presence)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdatePresence::Params)
	ONLINE_STRUCT_FIELD(FUpdatePresence::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUpdatePresence::Params, Presence)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdatePresence::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPartialUpdatePresence::Params)
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params, Mutations)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPartialUpdatePresence::Params::FMutations)
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, Status),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, Joinability),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, GameStatus),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, StatusString),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, RichPresenceString),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, UpdatedProperties),
	ONLINE_STRUCT_FIELD(FPartialUpdatePresence::Params::FMutations, RemovedProperties)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPartialUpdatePresence::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FPresenceUpdated)
	ONLINE_STRUCT_FIELD(FPresenceUpdated, LocalAccountId),
	ONLINE_STRUCT_FIELD(FPresenceUpdated, UpdatedPresence)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef LOCTEXT_NAMESPACE
