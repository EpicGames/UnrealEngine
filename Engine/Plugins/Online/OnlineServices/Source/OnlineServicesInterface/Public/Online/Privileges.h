// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

enum class EUserPrivileges : uint8
{
	/** Whether the user can play at all, online or offline*/
	CanPlay,
	/** Whether the user can play in online modes */
	CanPlayOnline,
	/** Whether the user can use text chat */
	CanCommunicateViaTextOnline,
	/** Whether the user can use voice chat */
	CanCommunicateViaVoiceOnline,
	/** Whether the user can use content generated by other users */
	CanUseUserGeneratedContent,
	/** Whether the user can ever participate in cross-play*/
	CanCrossPlay
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUserPrivileges Type);
ONLINESERVICESINTERFACE_API void LexFromString(EUserPrivileges& OutType, const TCHAR* InStr);

/** Whether to show UI to resolve the privilege if there is no access when GetUserPrivilege */
enum class EShowPrivilegeResolutionUI : uint8
{
	/** Behavior matching the recommended practices for the platform */
	Default,
	/** Show UI to resolve privilege */
	Show,
	/** Not show UI to resolve privilege */
	NotShow
};
ONLINESERVICESINTERFACE_API FString LexToString(EShowPrivilegeResolutionUI Type);
ONLINESERVICESINTERFACE_API void LexFromString(EShowPrivilegeResolutionUI& OutType, const FString& InStr);

enum class EPrivilegeResults : uint32
{
	/** The user has the requested privilege */
	NoFailures = 0,
	/** Patch required before the user can use the privilege */
	RequiredPatchAvailable = 1 << 0,
	/** System update required before the user can use the privilege */
	RequiredSystemUpdate = 1 << 1,
	/** Parental control failure usually */
	AgeRestrictionFailure = 1 << 2,
	/** Premium account required */
	AccountTypeFailure = 1 << 3,
	/** Invalid user */
	UserNotFound = 1 << 4,
	/** User must be logged in */
	UserNotLoggedIn = 1 << 5,
	/** User restricted from chat */
	ChatRestriction = 1 << 6,
	/** User restricted from User Generated Content */
	UGCRestriction = 1 << 7,
	/** Platform failed for unknown reason and handles its own dialogs */
	GenericFailure = 1 << 8,
	/** Online play is restricted */
	OnlinePlayRestricted = 1 << 9,
	/** Check failed because network is unavailable */
	NetworkConnectionUnavailable = 1 << 10,
};
ENUM_CLASS_FLAGS(EPrivilegeResults);
ONLINESERVICESINTERFACE_API FString LexToString(EPrivilegeResults Type);
ONLINESERVICESINTERFACE_API void LexFromString(EPrivilegeResults& OutType, const FString& InStr);

struct FQueryUserPrivilege
{
	static constexpr TCHAR Name[] = TEXT("QueryUserPrivilege");

	/** Input struct for Misc::FQueryUserPrivilege */
	struct Params
	{
		/** User id whose privilege we want to query */
		FAccountId LocalAccountId;
		/** Privilege to query */
		EUserPrivileges Privilege;
		/** Should external UI be shown when not possessing the queried privilege */
		EShowPrivilegeResolutionUI ShowExternalUI;

	};

	/** Output struct for Misc::FQueryUserPrivilege */
	struct Result
	{
		/** Privilege result, bitwise OR of various EPrivilegeResults values */
		EPrivilegeResults PrivilegeResult = EPrivilegeResults::NoFailures;
	};
};

class IPrivileges
{
public:
	/**
	* Queries about a given privilege for a user
	*/
	virtual TOnlineAsyncOpHandle<FQueryUserPrivilege> QueryUserPrivilege(FQueryUserPrivilege::Params&& Params) = 0;

	virtual ~IPrivileges() = default;
};

namespace Meta {
// TODO: Move to OnlineServices_Meta.inl file?

BEGIN_ONLINE_STRUCT_META(FQueryUserPrivilege::Params)
	ONLINE_STRUCT_FIELD(FQueryUserPrivilege::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FQueryUserPrivilege::Params, Privilege),
	ONLINE_STRUCT_FIELD(FQueryUserPrivilege::Params, ShowExternalUI)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryUserPrivilege::Result)
	ONLINE_STRUCT_FIELD(FQueryUserPrivilege::Result, PrivilegeResult)
END_ONLINE_STRUCT_META()

/* Meta*/}

/* UE::Online */}
