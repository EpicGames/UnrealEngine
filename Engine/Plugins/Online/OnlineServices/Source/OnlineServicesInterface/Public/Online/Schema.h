// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SchemaTypes.h"
#include "Templates/SharedPointer.h"

#define UE_API ONLINESERVICESINTERFACE_API

namespace UE::Online { template <typename OpType> class TOnlineResult; }

namespace UE::Online {

class FSchemaRegistry;

namespace Private {

class FSchemaCategoryInstanceBase
{
public:
	// DerivedSchemaId may remain unset in two situations:
	// 1. DerivedSchemaId will be populated by attributes from a search result. In this scenario
	//    the base schema is required to have an attribute flagged as SchemaCompatibilityId so that
	//    the derived schema can be discovered.
	// 2. Schema swapping is not enabled. All attributes are defined in the base schema.
	UE_API FSchemaCategoryInstanceBase(
		const FSchemaId& DerivedSchemaId,
		const FSchemaId& BaseSchemaId,
		const FSchemaCategoryId& CategoryId,
		const TSharedRef<const FSchemaRegistry>& SchemaRegistry);

	virtual ~FSchemaCategoryInstanceBase() = default;

	// Two phase commit to the service.
	// Prepare data to be written to the service from an client delta of attributes.
	UE_API TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChanges(FSchemaCategoryInstancePrepareClientChanges::Params&& Params) const;
	// Commit the client data once the service data has been successfully written.
	UE_API FSchemaCategoryInstanceCommitClientChanges::Result CommitClientChanges();

	// Two phase commit from the service.
	// Prepare data to be written to the client from a service snapshot of attributes.
	UE_API TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshot(FSchemaCategoryInstancePrepareServiceSnapshot::Params&& Params) const;
	// Commit a service snapshot containing all known service attributes.
	UE_API FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshot();

	// Todo: ApplyServiceDelta

	TSharedPtr<const FSchemaDefinition> GetDerivedDefinition() const { return DerivedSchemaDefinition; };
	TSharedPtr<const FSchemaDefinition> GetBaseDefinition() const { return BaseSchemaDefinition; };

	// Check whether the schema is valid.
	UE_API bool IsValid() const;

	// Check whether an client attribute is valid in the base schema.
	// Intended to be used when attribute information needs to be verified without translating to the service.
	UE_API bool VerifyBaseAttributeData(
		const FSchemaAttributeId& Id,
		const FSchemaVariant& Data,
		FSchemaServiceAttributeId& OutSchemaServiceAttributeId,
		ESchemaServiceAttributeFlags& OutSchemaServiceAttributeFlags);

protected:
	UE_API const TMap<FSchemaAttributeId, FSchemaVariant>& GetClientSnapshot() const;
	virtual TMap<FSchemaAttributeId, FSchemaVariant>& GetMutableClientSnapshot() = 0;

private:
	UE_API bool InitializeSchemaDefinition(
		int64 SchemaCompatibilityId,
		const FSchemaDefinition* OptionalBaseDefinition,
		const FSchemaCategoryId& CategoryId,
		TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
		const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const;

	UE_API bool InitializeSchemaDefinition(
		const FSchemaId& SchemaId,
		const FSchemaDefinition* OptionalBaseDefinition,
		const FSchemaCategoryId& CategoryId,
		TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
		const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const;

	UE_API bool InitializeSchemaDefinition(
		const TSharedRef<const FSchemaDefinition>& NewDefinition,
		const FSchemaDefinition* OptionalBaseDefinition,
		const FSchemaCategoryId& CategoryId,
		TSharedPtr<const FSchemaDefinition>* OutSchemaDefinition,
		const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const;

	UE_API bool GetSerializationSchema(
		const FSchemaDefinition** OutSchemaDefinition,
		const FSchemaCategoryDefinition** OutSchemaCategoryDefinition) const;

	UE_API void ResetPreparedChanges() const;

	struct FPreparedClientChanges
	{
		FSchemaServiceClientChanges ClientChanges;
		int64 SchemaCompatibilityId = 0;
		TSharedPtr<const FSchemaDefinition> DerivedSchemaDefinition;
		const FSchemaCategoryDefinition* DerivedSchemaCategoryDefinition = nullptr;
	};

	struct FPreparedServiceChanges
	{
		FSchemaServiceClientChanges ClientChanges;
		TSharedPtr<const FSchemaDefinition> DerivedSchemaDefinition;
		const FSchemaCategoryDefinition* DerivedSchemaCategoryDefinition = nullptr;
		TMap<FSchemaAttributeId, FSchemaVariant> ClientDataSnapshot;
	};

	TSharedRef<const FSchemaRegistry> SchemaRegistry;
	TSharedPtr<const FSchemaDefinition> DerivedSchemaDefinition;
	TSharedPtr<const FSchemaDefinition> BaseSchemaDefinition;
	const FSchemaCategoryDefinition* DerivedSchemaCategoryDefinition = nullptr;
	const FSchemaCategoryDefinition* BaseSchemaCategoryDefinition = nullptr;
	int64 LastSentSchemaCompatibilityId = 0;

	// Prepared changes are mutable so that the prepare methods can be const. Setting as mutable
	// is done to prevent modifying state outside of the prepared changes.
	mutable TOptional<FPreparedClientChanges> PreparedClientChanges;
	mutable TOptional<FPreparedServiceChanges> PreparedServiceChanges;
};

/* Private */ }

/** Schema category instance attribute accessor for providing the client attributes internally. */
class FSchemaCategoryInstanceInternalSnapshotAccessor
{
public:
	TMap<FSchemaAttributeId, FSchemaVariant>& GetMutableClientSnapshot()
	{
		return ClientSnapshot;
	}
private:

	TMap<FSchemaAttributeId, FSchemaVariant> ClientSnapshot;
};

/**
  * Schema category instance class templated by access to client attributes.
  * 
  * The passed in accessor is an interface which must provide a definition for GetMutableClientSnapshot.
  */
template <typename FSnapshotAccessor>
class TSchemaCategoryInstance final : public Private::FSchemaCategoryInstanceBase
{
public:
	TSchemaCategoryInstance(
		const FSchemaId& DerivedSchemaId,
		const FSchemaId& BaseSchemaId,
		const FSchemaCategoryId& CategoryId,
		const TSharedRef<const FSchemaRegistry>& SchemaRegistry,
		FSnapshotAccessor&& InSnapshotAccessor = FSnapshotAccessor())
		: Private::FSchemaCategoryInstanceBase(DerivedSchemaId, BaseSchemaId, CategoryId, SchemaRegistry)
		, SnapshotAccessor(MoveTemp(InSnapshotAccessor))
	{
	}

private:
	virtual TMap<FSchemaAttributeId, FSchemaVariant>& GetMutableClientSnapshot() override
	{
		return SnapshotAccessor.GetMutableClientSnapshot();
	}

	FSnapshotAccessor SnapshotAccessor;
};

/** Default implementation with both translation and client attribute snapshot data contained within the category instance. */
using FSchemaCategoryInstance = TSchemaCategoryInstance<FSchemaCategoryInstanceInternalSnapshotAccessor>;

class FSchemaRegistry
{
public:
	// Parse the loaded config structures.
	UE_API bool ParseConfig(const FSchemaRegistryDescriptorConfig& Config);

	UE_API TSharedPtr<const FSchemaDefinition> GetDefinition(const FSchemaId& SchemaId) const;
	UE_API TSharedPtr<const FSchemaDefinition> GetDefinition(int64 CompatibilityId) const;
	UE_API bool IsSchemaChildOf(const FSchemaId& SchemaId, const FSchemaId& ParentSchemaId) const;

private:

	TMap<FSchemaId, TSharedRef<const FSchemaDefinition>> SchemaDefinitionsById;
	TMap<int64, TSharedRef<const FSchemaDefinition>> SchemaDefinitionsByCompatibilityId;
};

/* UE::Online */ }

#undef UE_API
