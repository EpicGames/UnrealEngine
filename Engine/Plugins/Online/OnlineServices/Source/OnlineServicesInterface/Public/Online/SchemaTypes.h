// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Misc/TVariant.h"
#include "Misc/TVariantMeta.h"
#include "Online/OnlineMeta.h"
#include "UObject/NameTypes.h"

#define UE_API ONLINESERVICESINTERFACE_API

namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

/** Schema name - used to lookup and interact with a schema. */
using FSchemaId = FName;
/**
  * Category within a schema.
  * Most schema will have two sets of configuration - one for the singular object and one for
  * member data within that object. The category name is expected to be built into the code which
  * uses a schema and is not expected to change. A schema may not add any categories not already
  * declared in the parent schema.
  * 
  * Example: A lobby contains both lobby data and lobby member data, but during lobby creation the
  * client only specifies a single schema Id. The schema id is required to have categories for
  * both Lobby and LobbyMember.
  */
using FSchemaCategoryId = FName;
/**
  * The id used to refer to a schema attribute. These ids are declared in the client config
  * and used in client code to look up an attribute value.
  */
using FSchemaAttributeId = FName;
/**
  * The id used to refer to a schema service attribute. These ids are declared in config and used
  * directly by the platform service when serializing attributes.
  */
using FSchemaServiceAttributeId = FName;
/**
  * The id used to refer to a schema service descriptor. Similar to a schema, service descriptors
  * declare a set of attributes. Service descriptors are tied directly to how a service serializes
  * data to its backend.
  * 
  * Example: In the case of a lobby the backing service is capable of storing attributes for a
  * lobby and a lobby member. There will be separate service descriptors for lobby and lobby member
  * where each will list the attributes available to be set as well as the types of data each
  * attribute can store. The lobby implementation refers to the attributes names in the service
  * descriptor directly when handling the serialization of attributes.
  */
using FSchemaServiceDescriptorId = FName;

/** Flags to indicate comparison types between Schema Attributes. Used in searches */
enum class ESchemaAttributeComparisonOp : uint8
{
	Equals,
	NotEquals,
	GreaterThan,
	GreaterThanEquals,
	LessThan,
	LessThanEquals,
	Near,
	In,
	NotIn
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaAttributeComparisonOp EnumVal);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaAttributeComparisonOp& OutComparison, const TCHAR* InStr);

// TODO: This enum fulfills the same function as the attribute flags, and will be removed once full Schema support is added to Sessions/Lobbies/Auth
enum class ESchemaAttributeVisibility : uint8
{
	Public,
	Private,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaAttributeVisibility Visibility);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaAttributeVisibility& OutVisibility, const TCHAR* InStr);

/**
  * Flags to indicate required properties for a schema attribute.
  */
enum class ESchemaAttributeFlags : uint8
{
	/**
	  * No special properties needed.
	  */
	None = 0,
	/**
	  * Attribute is available for search operations.
	  * 
	  * Searchable attributes are required to exist only in a base schema used as a parent of all
	  * schema for a particular interface.
	  */
	Searchable = 1 << 0,
	/**
	  * Attribute is publicly visible.
	  */
	Public = 1 << 1,
	/**
	  * Attribute is visible to members only.
	  */
	Private = 1 << 2,
	/**
	  * If multiple schema are supported using the same service interface, a base schema must exist
	  * with a field marked as schema compatibility id. This field will allow remote clients to
	  * determine what schema should be used to handle attributes.
	  */
	SchemaCompatibilityId = 1 << 3,
};
ENUM_CLASS_FLAGS(ESchemaAttributeFlags);
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaAttributeFlags SchemaAttributeFlags);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaAttributeFlags& OutSchemaAttributeFlags, const TCHAR* InStr);

/**
  * Flags to indicate the data types supported by a service attribute.
  */
enum class ESchemaAttributeType : uint8
{
	/**
	  * Unset.
	  */
	None,
	/**
	  * Boolean values.
	  */
	Bool,
	/**
	  * Signed 64 bit integers.
	  */
	Int64,
	/**
	  * Double precision floating point.
	  */
	Double,
	/**
	  * Variable length string data.
	  */
	String,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaAttributeType SchemaAttributeType);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaAttributeType& OutSchemaAttributeType, const TCHAR* InStr);

/**
  * Flags to indicate the available support for a service attribute.
  */
enum class ESchemaServiceAttributeFlags : uint8
{
	/**
	  * No special properties needed.
	  */
	None = 0,
	/**
	  * Attribute is available for search operations.
	  */
	Searchable = 1 << 0,
	/**
	  * Attribute is publicly visible.
	  */
	Public = 1 << 1,
	/**
	  * Attribute is visible to members only.
	  */
	Private = 1 << 2,
};
ENUM_CLASS_FLAGS(ESchemaServiceAttributeFlags);
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaServiceAttributeFlags SchemaServiceAttributeFlags);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaServiceAttributeFlags& OutSchemaServiceAttributeFlags, const TCHAR* InStr);

/**
  * Flags to indicate the data types supported by a service attribute.
  */
enum class ESchemaServiceAttributeSupportedTypeFlags : uint8
{
	None = 0,
	/**
	  * Boolean values.
	  */
	Bool = 1 << 0,
	/**
	  * Signed 64 bit integers.
	  */
	Int64 = 1 << 1,
	/**
	  * Double precision floating point.
	  */
	Double = 1 << 2,
	/**
	  * Variable length string data.
	  */
	String = 1 << 3,
};
ENUM_CLASS_FLAGS(ESchemaServiceAttributeSupportedTypeFlags);
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESchemaServiceAttributeSupportedTypeFlags SchemaServiceAttributeSupportedTypeFlags);
ONLINESERVICESINTERFACE_API void LexFromString(ESchemaServiceAttributeSupportedTypeFlags& OutSchemaServiceAttributeSupportedTypeFlags, const TCHAR* InStr);

class FSchemaVariant final
{
public:
	using FVariantType = TVariant<FString, int64, double, bool>;
	FSchemaVariant() = default;
	FSchemaVariant(const FSchemaVariant& InOther) = default;
	UE_API FSchemaVariant(FSchemaVariant&& InOther);
	UE_API FSchemaVariant& operator=(FSchemaVariant&&);
	FSchemaVariant& operator=(const FSchemaVariant&) = default;

	FSchemaVariant(const TCHAR* InData) { Set(InData); }
	FSchemaVariant(const FString& InData) { Set(InData); }
	FSchemaVariant(FString&& InData) { Set(MoveTemp(InData)); }
	FSchemaVariant(int64 InData) { Set(InData); }
	FSchemaVariant(double InData) { Set(InData); }
	FSchemaVariant(bool InData) { Set(InData); }
	UE_API void Set(const TCHAR* AsString);
	UE_API void Set(const FString& AsString);
	UE_API void Set(FString&& AsString);
	UE_API void Set(int64 AsInt);
	UE_API void Set(double AsDouble);
	UE_API void Set(bool bAsBool);

	ESchemaAttributeType GetType() const { return VariantType; }

	UE_API int64 GetInt64() const;
	UE_API double GetDouble() const;
	UE_API bool GetBoolean() const;
	UE_API const FString& GetString() const;

	UE_API FString ToLogString() const;

	UE_API bool operator==(const FSchemaVariant& Other) const;
	bool operator!=(const FSchemaVariant& Other) const { return !(*this == Other); }
public:
	FVariantType VariantData;
	ESchemaAttributeType VariantType = ESchemaAttributeType::None;
};

/**
  * Adaptor for handling LexToString with a schema variant.
  * The adapter prevents unintentional implicit conversion of types to FSchemaVariant when calling LexToString.
  */
template <typename T> class FLexToStringAdaptor;
template <>
class FLexToStringAdaptor<FSchemaVariant>
{
public:
	FLexToStringAdaptor(const FSchemaVariant& SchemaVariant)
		: SchemaVariant(SchemaVariant)
	{
	}

	const FSchemaVariant& SchemaVariant;
};
ONLINESERVICESINTERFACE_API const FString LexToString(const FLexToStringAdaptor<FSchemaVariant>& Adaptor);
ONLINESERVICESINTERFACE_API void LexFromString(FSchemaVariant& OutSchemaVariant, const TCHAR* InStr);

/**
  * Describes available options for an attribute made available by the underlying service.
  */
struct FSchemaServiceAttributeDescriptor
{
	/**
	  * The id to associate with this attribute.
	  */
	FSchemaServiceAttributeId Id;
	/**
	  * Declares what data types can be handled by the service attribute.
	  */
	TArray<ESchemaServiceAttributeSupportedTypeFlags> SupportedTypes;
	/**
	  * Additional behavior required by the attribute. Examples include making an attribute
	  * public or available to search.
	  */
	TArray<ESchemaServiceAttributeFlags> Flags;
	/**
	  * Declares the maximum allowed size for the service attribute.
	  *
	  * Applicable when using a variable sized type such as String.
	  */
	int32 MaxSize = 0;
};

/**
  * Service descriptor used for declaring a set of attributes a service provides.
  */
struct FSchemaServiceDescriptor
{
	/**
	  * The id associated with this service descriptor.
	  */
	FSchemaServiceDescriptorId Id;
	/**
	  * Service attribute descriptions.
	  */
	TArray<FSchemaServiceAttributeId> AttributeIds;
};

/**
  * Describes the required attribute configuration for a schema attribute.
  */
struct FSchemaAttributeDescriptor
{
	/**
	  * The id to associate with this attribute.
	  */
	FSchemaAttributeId Id;
	/**
	  * The type of data contained within the attribute.
	  */
	ESchemaAttributeType Type = ESchemaAttributeType::None;
	/**
	  * Additional behavior required by the attribute. Examples include making an attribute
	  * public or available to search.
	  */
	TArray<ESchemaAttributeFlags> Flags;
	/**
	  * Whether to group the attribute together with other attributes explicitly.
	  * Providing a group id allows the client to explicitly serialize attributes to the same
	  * service attribute for increased network efficiency.
	  *
	  * Id 0 uses automatic grouping.
	 */
	int32 UpdateGroupId = 0;
	/**
	  * Declares the maximum allowed serialized size for a variably sized attribute.
	  *
	  * Applicable when using a variable sized type such as String.
	  */
	int32 MaxSize = 0;
};

/**
  * Describes a category within a schema.
  */
struct FSchemaCategoryDescriptor
{
	/**
	  * Category for which the schema applies.
	  * 
	  * Example: "Lobby" vs "LobbyMember"
	  */
	FSchemaCategoryId Id;
	/**
	  * Service descriptor to use for initializing schema.
	  * May be empty when a schema only applies to local data.
	  * May only be set by base parent schema.
	  */
	FSchemaServiceDescriptorId ServiceDescriptorId;
};

/**
  * Adds a set of attributes to a given category in a given schema.
  */
struct FSchemaCategoryAttributesDescriptor
{
	/**
	 * The schema to add attributes to.
	 */
	FSchemaId SchemaId;
	/**
	 * The category to add attributes to.
	 */
	FSchemaCategoryId CategoryId;
	/**
	 * The attributes to add.
	 */
	TArray<FSchemaAttributeId> AttributeIds;
};

/**
  * Describes all properties needed to use a schema.
  */
struct FSchemaDescriptor
{
	/**
	  * The id associated with this schema.
	  */
	FSchemaId Id;
	/**
	  * Parent schema (not required).
	  * Attributes used in a parent schema may not be modified by a child schema.
	  */
	FSchemaId ParentId;
	/**
	  * Categories of data within the schema. Each category is handled differently within the
	  * service so will have different configuration of attributes and service attributes.
	  */
	TArray<FSchemaCategoryId> CategoryIds;
};

/**
  * Set of all descriptors used to initialize a SchemaRegistry.
  */
struct FSchemaRegistryDescriptorConfig
{
	/**
	 * List of all supported schema.
	 */
	TArray<FSchemaDescriptor> SchemaDescriptors;
	/**
	 * List of all schema categories
	 */
	TArray<FSchemaCategoryDescriptor> SchemaCategoryDescriptors;
	/**
	 * List of all schema attributes
	 */
	TArray<FSchemaAttributeDescriptor> SchemaAttributeDescriptors;
	/**
	 * List of additional attributes to add to the given category for the given schema.
	 */
	TArray<FSchemaCategoryAttributesDescriptor> SchemaCategoryAttributeDescriptors;

	/**
	 * List of all service descriptors.
	 */
	TArray<FSchemaServiceDescriptor> ServiceDescriptors;
	/**
	 * List of all service attributes
	 */
	TArray<FSchemaServiceAttributeDescriptor> ServiceAttributeDescriptors;
};

/**
  * The runtime definition of a schema service attribute.
  */
struct FSchemaServiceAttributeDefinition
{
	/**
	  * The id to associate with this attribute.
	  */
	FSchemaServiceAttributeId Id;
	/**
	  * The supported type. Only one type is supported in a service attribute definition.
	  */
	ESchemaServiceAttributeSupportedTypeFlags Type;
	/**
	  * Additional behavior required by the attribute. Examples include making an attribute
	  * public, private or available to search.
	  */
	ESchemaServiceAttributeFlags Flags = ESchemaServiceAttributeFlags::None;
	/**
	  * Declares the maximum allowed size for the service attribute.
	  *
	  * Applicable when using a variable sized type such as String.
	  */
	int32 MaxSize = 0;
	/**
	  * The attribute ids contained within the service attribute.
	  */
	TArray<FSchemaAttributeId> SchemaAttributeIds;
};

/**
  * The runtime definition of a schema attribute.
  */
struct FSchemaAttributeDefinition
{
	/**
	  * The id to associate with this attribute.
	  */
	FSchemaAttributeId Id;
	/**
	  * The type of data contained within the attribute.
	  */
	ESchemaAttributeType Type = ESchemaAttributeType::None;
	/**
	  * Additional behavior required by the attribute. Examples include making an attribute
	  * public, private or available to search.
	  */
	ESchemaAttributeFlags Flags;
	/**
	  * Declares the maximum allowed serialized size for a variably sized attribute.
	  *
	  * Applicable when using a variable sized type such as String.
	  */
	int32 MaxSize = 0;
	/**
	  * The id of the associated service attribute.
	  */
	FSchemaServiceAttributeId ServiceAttributeId;
};

/**
  * The runtime definition of a schema category.
  * The contained data can be used for validation and translation.
  */
struct FSchemaCategoryDefinition
{
	/**
	  * Category for which the schema applies.
	  *
	  * Example: "Lobby" vs "LobbyMember"
	  */
	FSchemaCategoryId Id;
	/**
	  * The Id of the attribute containing the schema compatibility id.
	  * Changes to the schema compatibility attribute are not allowed externally. This id is used
	  * to fail attempted external changes.
	  */
	FSchemaAttributeId SchemaCompatibilityAttributeId;
	/**
	  * The Id of the service attribute containing the schema compatibility id.
	  * This Id is used to detect the schema to use when handling search results and to handle
	  * schema changes when the remote owner makes a change.
	  */
	FSchemaServiceAttributeId SchemaCompatibilityServiceAttributeId;
	/**
	  * Mapping of schema attribute ids to attribute definitions.
	  */
	TMap<FSchemaAttributeId, FSchemaAttributeDefinition> SchemaAttributeDefinitions;
	/**
	  * Mapping of service attribute ids to service attribute definitions.
	  */
	TMap<FSchemaServiceAttributeId, FSchemaServiceAttributeDefinition> ServiceAttributeDefinitions;
};

/**
  * The runtime definition of a schema.
  * The contained data can be used for validation and translation.
  */
struct FSchemaDefinition
{
	/**
	  * The id associated with this schema.
	  */
	FSchemaId Id;
	/**
	  * The compatibility id consists of the CRC of the schema name in the top 32 bits and the CRC
	  * of all the schema options in the bottom 32 bits.
	  */
	int64 CompatibilityId = 0;
	/**
	  * Supported categories within the schema.
	  */
	TMap<FSchemaCategoryId, FSchemaCategoryDefinition> Categories;
	/**
	  * Set of parent schema ids for parent checks.
	  */
	TSet<FSchemaId> ParentSchemaIds;
};

/**
  * The set of data used to write a variant to a service.
  */
struct FSchemaServiceAttributeData
{
	/**
	  * The id to associate with this attribute.
	  */
	FSchemaServiceAttributeId Id;
	/**
	  * Additional behavior required by the attribute. Examples include making an attribute
	  * public or available to search.
	  */
	ESchemaServiceAttributeFlags Flags = ESchemaServiceAttributeFlags::None;
	/**
	  * The attribute value.
	  */
	FSchemaVariant Value;
};

/** Full snapshot of all set service attributes. */
struct FSchemaServiceSnapshot
{
	TMap<FSchemaServiceAttributeId, FSchemaVariant> Attributes;
};

/**
  * Changes from the client data to be translated into service changes.
  */
struct FSchemaClientChanges
{
	/**
	  * Change to schema id.
	  */
	TOptional<FSchemaId> SchemaId;
	/**
	  * Added or changed attributes and their values.
	  */
	TMap<FSchemaAttributeId, FSchemaVariant> UpdatedAttributes;
	/**
	  * Attributes to be cleared.
	  */
	TSet<FSchemaAttributeId> RemovedAttributes;
};

/**
  * Changes to be applied to the service data following client schema attribute changes.
  */
struct FSchemaClientServiceChanges
{
	/**
	  * Added or changed attributes and their values.
	  */
	TMap<FSchemaServiceAttributeId, FSchemaServiceAttributeData> UpdatedAttributes;
	/**
	  * Removed attribute ids.
	  */
	TSet<FSchemaServiceAttributeId> RemovedAttributes;
};

/**
  * Changes to be applied to the client data following service attribute changes.
  */
struct FSchemaServiceClientChanges
{
	/**
	  * Change to schema id.
	  */
	TOptional<FSchemaId> SchemaId;
	/**
	  * Added attributes.
	  */
	TMap<FSchemaAttributeId, FSchemaVariant> AddedAttributes;
	/**
	  * Changed attributes with their old and new values.
	  */
	TMap<FSchemaAttributeId, TPair<FSchemaVariant, FSchemaVariant>> ChangedAttributes;
	/**
	  * Removed attribute ids.
	  */
	TSet<FSchemaAttributeId> RemovedAttributes;
};

struct FSchemaCategoryInstancePrepareClientChanges
{
	struct Params
	{
		/** Changes from the client to be applied to the service. */
		FSchemaClientChanges ClientChanges;
	};

	struct Result
	{
		/** Attribute changes to be notified to the service. */
		FSchemaClientServiceChanges ServiceChanges;
	};
};

struct FSchemaCategoryInstanceCommitClientChanges
{
	struct Result
	{
		/** Attribute changes to be notified to the client. */
		FSchemaServiceClientChanges ClientChanges;
	};
};

struct FSchemaCategoryInstancePrepareServiceSnapshot
{
	struct Params
	{
		/** Optional schema id for applying a schema change from external state. */
		TOptional<FSchemaId> SchemaId;
		/** Full attribute snapshot from the service. */
		FSchemaServiceSnapshot ServiceSnapshot;
	};

	struct Result
	{
		/** The new derived schema id when set from the service. */
		TOptional<FSchemaId> DerivedSchemaId;
	};
};

struct FSchemaCategoryInstanceCommitServiceSnapshot
{
	struct Result
	{
		/** Attribute changes to be notified to the client. */
		FSchemaServiceClientChanges ClientChanges;
	};
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FSchemaServiceAttributeDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaServiceAttributeDescriptor, Id),
	ONLINE_STRUCT_FIELD(FSchemaServiceAttributeDescriptor, SupportedTypes),
	ONLINE_STRUCT_FIELD(FSchemaServiceAttributeDescriptor, Flags),
	ONLINE_STRUCT_FIELD(FSchemaServiceAttributeDescriptor, MaxSize)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaServiceDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaServiceDescriptor, Id),
	ONLINE_STRUCT_FIELD(FSchemaServiceDescriptor, AttributeIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaAttributeDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaAttributeDescriptor, Id),
	ONLINE_STRUCT_FIELD(FSchemaAttributeDescriptor, Type),
	ONLINE_STRUCT_FIELD(FSchemaAttributeDescriptor, Flags),
	ONLINE_STRUCT_FIELD(FSchemaAttributeDescriptor, UpdateGroupId),
	ONLINE_STRUCT_FIELD(FSchemaAttributeDescriptor, MaxSize)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaCategoryDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaCategoryDescriptor, Id),
	ONLINE_STRUCT_FIELD(FSchemaCategoryDescriptor, ServiceDescriptorId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaDescriptor, Id),
	ONLINE_STRUCT_FIELD(FSchemaDescriptor, ParentId),
	ONLINE_STRUCT_FIELD(FSchemaDescriptor, CategoryIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaCategoryAttributesDescriptor)
	ONLINE_STRUCT_FIELD(FSchemaCategoryAttributesDescriptor, SchemaId),
	ONLINE_STRUCT_FIELD(FSchemaCategoryAttributesDescriptor, CategoryId),
	ONLINE_STRUCT_FIELD(FSchemaCategoryAttributesDescriptor, AttributeIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSchemaRegistryDescriptorConfig)
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, SchemaDescriptors),
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, SchemaCategoryDescriptors),
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, SchemaAttributeDescriptors),
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, SchemaCategoryAttributeDescriptors),
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, ServiceDescriptors),
	ONLINE_STRUCT_FIELD(FSchemaRegistryDescriptorConfig, ServiceAttributeDescriptors)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }

#undef UE_API
