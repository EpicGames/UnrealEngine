// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineError.h"
#include "Online/SchemaTypes.h"
#include "Online/OnlineResult.h"

#define UE_API ONLINESERVICESINTERFACE_API

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }

namespace UE::Online {

// ISessions Types

/** Filter class for a session search, will be compared against the session's CustomSettings */
struct FFindSessionsSearchFilter
{
	/** Name of the custom setting to be used as filter */
	FSchemaAttributeId Key;

	/** The type of comparison to perform */
	ESchemaAttributeComparisonOp ComparisonOp;

	/** Value to use when comparing the filter */
	FSchemaVariant Value;
};

/** User-defined data to be stored along with the session information. Will be evaluated in session searches */
struct FCustomSessionSetting
{
	/** Setting data value */
	FSchemaVariant Data;

	/** Manner in which this session setting is advertised with the backend or searches */
	ESchemaAttributeVisibility Visibility;

	/** Optional ID used in some platforms as the index instead of the setting name */
	int32 ID;
};

using FCustomSessionSettingsMap = TMap<FSchemaAttributeId, FCustomSessionSetting>;

/** Contains both the old and new values for an updated FCustomSessionSetting. Part of the FSessionUpdate event data */
struct FCustomSessionSettingUpdate
{
	/** Old value for the updated Setting */
	FCustomSessionSetting OldValue;

	/** New value for the updated Setting */
	FCustomSessionSetting NewValue;
};

using FCustomSessionSettingUpdateMap = TMap<FName, FCustomSessionSettingUpdate>;

using FSessionMemberIdsSet = TSet<FAccountId>;

/** Set of options to reflect how a session may be discovered in searches and joined */
enum class ESessionJoinPolicy : uint8
{
	/** The session will appear on searches an may be joined by anyone */
	Public,

	/** The session will not appear on searches and may only be joined via presence (if enabled) or invitation */
	FriendsOnly,

	/** The session will not appear on searches and may not be joined via presence, only via invitation */
	InviteOnly
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(ESessionJoinPolicy Value);
ONLINESERVICESINTERFACE_API void LexFromString(ESessionJoinPolicy& Value, const TCHAR* InStr);

/** Contains new values for an FSessions modifiable settings. Taken as a parameter by FUpdateSessionSettings method */
struct FSessionSettingsUpdate
{
	/** Set with an updated value if the SchemaName field will be changed in the update operation */
	TOptional<FSchemaId> SchemaName;
	/** Set with an updated value if the NumMaxConnections field will be changed in the update operation */
	TOptional<uint32> NumMaxConnections;
	/** Set with an updated value if the JoinPolicy field will be changed in the update operation */
	TOptional<ESessionJoinPolicy> JoinPolicy;
	/** Set with an updated value if the bAllowNewMembers field will be changed in the update operation */
	TOptional<bool> bAllowNewMembers;

	/** Updated values for custom settings to change in the update operation */
	FCustomSessionSettingsMap UpdatedCustomSettings;
	/** Names of custom settings to be removed in the update operation */
	TArray<FSchemaAttributeId> RemovedCustomSettings;

	UE_API FSessionSettingsUpdate& operator+=(FSessionSettingsUpdate&& UpdatedValue);
};

/** Contains updated data for any modifiable members of FSessionSettings. Part of the FSessionUpdated event data */
struct FSessionSettingsChanges
{
	/* If set, the FSessionSettings's SchemaName member will have been updated to this value */
	TOptional<FName> SchemaName;
	/** If set, the FSessionSettings's NumMaxConnections member will have been updated to this value */
	TOptional<uint32> NumMaxConnections;
	/** If set, the FSessionSettings's JoinPolicy member will have been updated to this value */
	TOptional<ESessionJoinPolicy> JoinPolicy;
	/** If set, the FSessionSettings's bAllowNewMembers member will have been updated to this value */
	TOptional<bool> bAllowNewMembers;

	/** New custom settings added in the update, with their values */
	FCustomSessionSettingsMap AddedCustomSettings;

	/** Existing custom settings that changed value, including new and old values */
	FCustomSessionSettingUpdateMap ChangedCustomSettings;

	/** Keys for custom settings removed in the update */
	TArray<FName> RemovedCustomSettings;
};

/** Set of all of an FSession's defining properties that can be updated by the session owner during its lifetime, using the FUpdateSessionSettings method */
struct FSessionSettings
{
	/* The name for the schema which will be applied to the session's user-defined attributes */
	FName SchemaName;

	/* Maximum number of slots for session members */
	uint32 NumMaxConnections = 0;

	/* Enum value describing the level of restriction to join the session. Public by default */
	ESessionJoinPolicy JoinPolicy = ESessionJoinPolicy::Public;;

	/* Override value to restrict the session from accepting new members, regardless of other factors. True by default */
	bool bAllowNewMembers = true;

	/* Map of user-defined settings to be passed to the platform APIs as additional information for various purposes */
	FCustomSessionSettingsMap CustomSettings;

	UE_API FSessionSettings& operator+=(const FSessionSettingsChanges& UpdatedValue);
};

/** Information about an FSession that will be set at creation time and remain constant during its lifetime */
struct FSessionInfo
{
	/** The id handle for the session, platform dependent */
	FOnlineSessionId SessionId;

	/* In platforms that support this feature, it will set the session id to this value. Might be subject to minimum and maximum length restrictions */
	FString SessionIdOverride;

	/* Whether the session is only available in the local network and not via internet connection. Only available in some platforms. False by default */
	bool bIsLANSession = false;

	/* Whether the session is configured to run as a dedicated server. Only available in some platforms. False by default */
	bool bIsDedicatedServerSession = false;

	/* Whether this session will allow sanctioned players to join it. True by default */
	bool bAllowSanctionedPlayers = true;

	/* Whether this is a secure session protected by anti-cheat services. False by default */
	bool bAntiCheatProtected = false;
};

/** Interface to access all information related to Online Sessions. Read only */
class ISession
{
public:
	virtual ~ISession() = default;

	/** Retrieves the id handle for the user who created or currently owns the session */
	virtual const FAccountId GetOwnerAccountId() const = 0;

	/** Retrieves the id handle for the session */
	virtual const FOnlineSessionId GetSessionId() const = 0;

	/** Retrieves the number of available slots for new session members */
	virtual const uint32 GetNumOpenConnections() const = 0;

	/** Retrieves the set of constant information about the session */
	virtual const FSessionInfo& GetSessionInfo() const = 0;

	/** Retrieves the set of variable information about the session */
	virtual const FSessionSettings& GetSessionSettings() const = 0;

	/** Retrieves the list of users currently in the session */
	virtual const FSessionMemberIdsSet& GetSessionMembers() const = 0;

	/** Evaluates a series of factors to determine if a session is accepting new members */
	virtual bool IsJoinable() const = 0;

	/** Returns a string with the minimal information to identify the session */
	virtual FString ToLogString() const = 0;

	/** Returns a string with all the information in the session */
	virtual void DumpState() const = 0;
};
ONLINESERVICESINTERFACE_API const FString ToLogString(const ISession& Session);

/** Interface to access all information about an invitation to join an Online Session. Read only */
class ISessionInvite
{
public:
	/** Retrieves the id handle for the user who received the invite */
	virtual const FAccountId GetRecipientId() const = 0;

	/** Retrieves the id handle for the user who sent the invite */
	virtual const FAccountId GetSenderId() const = 0;
	virtual const FSessionInviteId GetInviteId() const = 0;
	virtual const FOnlineSessionId GetSessionId() const = 0;

	virtual FString ToLogString() const = 0;
};
ONLINESERVICESINTERFACE_API const FString ToLogString(const ISessionInvite& Session);

// ISessions Methods

struct FGetAllSessions
{
	static constexpr TCHAR Name[] = TEXT("GetAllSessions");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Array of sessions that the given user is member of */
		TArray<TSharedRef<const ISession>> Sessions;
	};
};

struct FGetSessionByName
{
	static constexpr TCHAR Name[] = TEXT("GetSessionByName");

	struct Params
	{
		/** Local name for the session */
		FName LocalName;
	};

	struct Result
	{
		/** Reference to the session mapped to the given name */
		TSharedRef<const ISession> Session;
	};
};

struct FGetSessionName
{
	static constexpr TCHAR Name[] = TEXT("GetSessionName");

	struct Params
	{
		/** Id handle for the session to be retrieved the name from */
		FOnlineSessionId SessionId;
	};

	struct Result
	{
		/** Session name */
		FName SessionName;
	};
};

struct FGetSessionById
{
	static constexpr TCHAR Name[] = TEXT("GetSessionById");

	struct Params
	{
		/** Id handle for the session to be retrieved */
		FOnlineSessionId SessionId;
	};

	struct Result
	{
		/** Reference to the session mapped to the given id */
		TSharedRef<const ISession> Session;
	};
};

struct FGetPresenceSession
{
	static constexpr TCHAR Name[] = TEXT("GetPresenceSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Reference to the session set as presence session for the given user */
		TSharedRef<const ISession> Session;
	};
};

struct FIsPresenceSession
{
	static constexpr TCHAR Name[] = TEXT("IsPresenceSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Id handle for the session to be compared */
		FOnlineSessionId SessionId;
	};

	struct Result
	{
		/** Whether the session mapped to the given id is set as the presence session for the given user */
		bool bIsPresenceSession;
	};
};

struct FSetPresenceSession
{
	static constexpr TCHAR Name[] = TEXT("SetPresenceSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Id handle for the session */
		FOnlineSessionId SessionId;
	};

	struct Result
	{
	};
};

struct FClearPresenceSession
{
	static constexpr TCHAR Name[] = TEXT("ClearPresenceSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FCreateSession
{
	static constexpr TCHAR Name[] = TEXT("CreateSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName;

		/** In platforms that support this feature, it will set the session id to this value. Might be subject to minimum and maximum length restrictions */
		FString SessionIdOverride;

		/** Whether this session should be set as the local user's new presence session. False by default */
		bool bPresenceEnabled = false;

		/** Whether the session is only available via the local network and not via internet connection. Only available in some platforms. False by default */
		bool bIsLANSession = false;

		/** Whether this session will allow sanctioned players to join it. True by default */
		bool bAllowSanctionedPlayers = true;

		/** Whether this is a secure session protected by anti-cheat services. False by default */
		bool bAntiCheatProtected = false;

		/** Settings object to define session properties during creation */
		FSessionSettings SessionSettings;
	};

	struct Result
	{

	};
};

struct FUpdateSessionSettings
{
	static constexpr TCHAR Name[] = TEXT("UpdateSessionSettings");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName;

		/** Changes to current session settings */
		FSessionSettingsUpdate Mutations;
	};

	struct Result
	{

	};
};

struct FLeaveSession
{
	static constexpr TCHAR Name[] = TEXT("LeaveSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName; // TODO (MATIAS): This could be a FOnlineSessionId 

		/** Whether the call should attempt to destroy the session instead of just leave it */
		bool bDestroySession;
	};

	struct Result
	{

	};
};

struct FFindSessions
{
	static constexpr TCHAR Name[] = TEXT("FindSessions");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Maximum number of results to return in the search */
		uint32 MaxResults;

		/** Whether we want to look for LAN sessions (true) or Online sessions (false). False by default */
		bool bFindLANSessions = false;

		/** Filters to apply when searching for sessions. */
		TArray<FFindSessionsSearchFilter> Filters;

		/** If set, the search will look for sessions containing the set user. */
		TOptional<FAccountId> TargetUser;

		/** If set, the search will look for the session with the set session id. */
		TOptional<FOnlineSessionId> SessionId;
	};

	struct Result
	{
		TArray<FOnlineSessionId> FoundSessionIds;
	};
};

struct FStartMatchmaking
{
	static constexpr TCHAR Name[] = TEXT("StartMatchmaking");

	struct Params
	{
		/** Session creation parameters */
		FCreateSession::Params SessionCreationParameters;

		/** Filters to apply when searching for sessions */
		TArray<FFindSessionsSearchFilter> SessionSearchFilters;
	};

	struct Result
	{

	};
};

struct FJoinSession
{
	static constexpr TCHAR Name[] = TEXT("JoinSession");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName;

		/** Id handle for the session to be joined. To be retrieved via session search, invite or UI operation */
		FOnlineSessionId SessionId;

		/** Whether this session should be set as the user's new presence session. False by default */
		bool bPresenceEnabled = false;

		/** Whether to join as a spectator instead of player */
		bool bJoinAsSpectator = false;
	};

	struct Result
	{

	};
};

struct FAddSessionMember
{
	static constexpr TCHAR Name[] = TEXT("AddSessionMember");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName;
	};

	struct Result
	{

	};
};

struct FRemoveSessionMember
{
	static constexpr TCHAR Name[] = TEXT("RemoveSessionMember");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Id handle for the user to be kicked */
		FAccountId TargetAccountId; // TODO: Can we consider adding this parameter ?
		/** Local name for the session */
		FName SessionName;
	};

	struct Result
	{

	};
};

struct FSendSessionInvite
{
	static constexpr TCHAR Name[] = TEXT("SendSessionInvite");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Local name for the session */
		FName SessionName;

		/** Array of id handles for users to which the invites will be sent */
		TArray<FAccountId> TargetUsers;
	};

	struct Result
	{

	};
};

struct FGetSessionInviteById
{
	static constexpr TCHAR Name[] = TEXT("GetSessionInviteById");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/** Id handle for the invite to be retrieved */
		FSessionInviteId SessionInviteId;
	};

	struct Result
	{
		/** Reference to the invite mapped to the given id */
		TSharedRef<const ISessionInvite> SessionInvite;
	};
};

struct FGetAllSessionInvites
{
	static constexpr TCHAR Name[] = TEXT("GetAllSessionInvites");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Array of invites received by the given user */
		TArray<TSharedRef<const ISessionInvite>> SessionInvites;
	};
};

struct FRejectSessionInvite
{
	static constexpr TCHAR Name[] = TEXT("RejectSessionInvite");

	struct Params
	{
		/** Id handle for the local user which will perform the action */
		FAccountId LocalAccountId;

		/* Id handle for the session invite to be rejected */
		FSessionInviteId SessionInviteId;
	};

	struct Result
	{
	};
};

/* Events */

struct FSessionCreated
{
	/** Id handle for the local user who created the session */
	FAccountId LocalAccountId;

	/** Id handle for the session created. */
	FOnlineSessionId SessionId;
};

struct FSessionJoined
{
	/** Id handle for the local user who joined the session */
	FAccountId LocalAccountId;

	/** Id handle for the session joined. */
	FOnlineSessionId SessionId;
};

struct FSessionLeft
{
	/** Id handle for the local users which left the session */
	FAccountId LocalAccountId;
};

/** Contains updated data for any modifiable members of ISession */
struct FSessionUpdate
{
	/** If set, the OwnerUserId member will have updated to this value */
	TOptional<FAccountId> OwnerAccountId;

	/** If set, the SessionSettings member will have updated using the struct information */
	TOptional<FSessionSettingsChanges> SessionSettingsChanges;

	/** Id handles for members that just joined the session */
	FSessionMemberIdsSet AddedSessionMembers;

	/** Id handles for members that just left the session */
	FSessionMemberIdsSet RemovedSessionMembers;

	UE_API FSessionUpdate& operator+=(const FSessionUpdate& SessionUpdate);
};

struct FSessionUpdated
{
	/** Local name for the updated session */
	FName SessionName;

	/** Updated session information */
	FSessionUpdate SessionUpdate;
};

struct FSessionInviteReceived
{
	/** Id handle for the local user which received the invite */
	FAccountId LocalAccountId;

	/** The session invite id for the invite received */
	FSessionInviteId SessionInviteId;
};

enum class EUISessionJoinRequestedSource : uint8
{
	/** Unspecified by the online service */
	Unspecified,
	/** From an invitation */
	FromInvitation,
};
ONLINESERVICESINTERFACE_API const TCHAR* LexToString(EUISessionJoinRequestedSource UISessionJoinRequestedSource);
ONLINESERVICESINTERFACE_API void LexFromString(EUISessionJoinRequestedSource& OutUISessionJoinRequestedSource, const TCHAR* InStr);

struct FUISessionJoinRequested
{
	/** Id handle for the local user associated with the join request */
	FAccountId LocalAccountId;

	/** Id handle for the session the local user requested to join, or the online error if there was a failure retrieving it */
	TResult<FOnlineSessionId, FOnlineError> Result;

	/** Join request source */
	EUISessionJoinRequestedSource JoinRequestedSource = EUISessionJoinRequestedSource::Unspecified;
};

class ISessions
{
public:
	/**
	 * Gets an array of references to all the sessions the given user is part of.
	 * 
	 * @params Parameters for the GetAllSessions call
	 * return
	 */
	virtual TOnlineResult<FGetAllSessions> GetAllSessions(FGetAllSessions::Params&& Params) const = 0;

	/**
	 * Gets a reference to the session with the given local name.
	 *
	 * @params Parameters for the GetSessionByName call
	 * return
	 */
	virtual TOnlineResult<FGetSessionByName> GetSessionByName(FGetSessionByName::Params&& Params) const = 0;

	/**
	 * Gets the name of the session with the given id handle.
	 *
	 * @params Parameters for the GetSessionName call
	 * return
	 */
	virtual TOnlineResult<FGetSessionName> GetSessionName(FGetSessionName::Params&& Params) const = 0;

	/**
	 * Gets a reference to the session with the given id handle.
	 *
	 * @params Parameters for the GetSessionById call
	 * return
	 */
	virtual TOnlineResult<FGetSessionById> GetSessionById(FGetSessionById::Params&& Params) const = 0;

	/**
	 * Gets a reference to the session set as presence session for the given user.
	 *
	 * @params Parameters for the GetPresenceSession call
	 * return
	 */
	virtual TOnlineResult<FGetPresenceSession> GetPresenceSession(FGetPresenceSession::Params&& Params) const = 0;

	/**
	 * Returns whether the session with the given id is set as the presence session for the given user.
	 *
	 * @params Parameters for the IsPresenceSession call
	 * return
	 */
	virtual TOnlineResult<FIsPresenceSession> IsPresenceSession(FIsPresenceSession::Params&& Params) const = 0;

	/**
	 * Sets the session with the given id as the presence session for the given user.
	 *
	 * @params Parameters for the SetPresenceSession call
	 * return
	 */
	virtual TOnlineResult<FSetPresenceSession> SetPresenceSession(FSetPresenceSession::Params&& Params) = 0;

	/**
	 * Clears the presence session for the given user.
	 *
	 * @params Parameters for the ClearPresenceSession call
	 * return
	 */
	virtual TOnlineResult<FClearPresenceSession> ClearPresenceSession(FClearPresenceSession::Params&& Params) = 0;

	/**
	 * Creates a new session with the given parameters, and assigns to it the given local name.
	 * Depending on the implementation, the creating user might not be added to the Session Members automatically, so a subsequent call to AddSessionMember is recommended
	 *
	 * @param Parameters for the CreateSession call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FCreateSession> CreateSession(FCreateSession::Params&& Params) = 0;

	/**
	 * Update the settings for the session with the given name.
	 * Should only be called by the session owner.
	 *
	 * @param Parameters for the UpdateSessionSettings call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FUpdateSessionSettings> UpdateSessionSettings(FUpdateSessionSettings::Params&& Params) = 0;

	/**
	 * Leaves and optionally destroys the session with the given name.
	 *
	 * @param Parameters for the LeaveSession call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FLeaveSession> LeaveSession(FLeaveSession::Params&& Params) = 0;

	/**
	 * Queries the API session service for sessions matching the given parameters.
	 *
	 * @param Parameters for the FindSessions call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FFindSessions> FindSessions(FFindSessions::Params&& Params) = 0;

	/**
	 * Starts the matchmaking process, which will either create a session with the given parameters, or join one that matches the given search filters.
	 *
	 * @param Parameters for the StartMatchmaking call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FStartMatchmaking> StartMatchmaking(FStartMatchmaking::Params&& Params) = 0;

	/**
	 * Joins  the session with the given session id, and assigns to it the given local name.
	 *
	 * @param Parameters for the JoinSession call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FJoinSession> JoinSession(FJoinSession::Params&& Params) = 0;

	/**
	 * Adds the given user as a new session member to the session with the given name
	 * The number of open slots in the session will decrease accordingly
	 * 
	 * @params Parameters for the AddSessionMember call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FAddSessionMember> AddSessionMember(FAddSessionMember::Params&& Params) = 0;

	/**
	 * Removes the given user from the session with the given name
	 * The number of open slots in the session will increase accordingly
	 *
	 * @params Parameters for the RemoveSessionMember call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FRemoveSessionMember> RemoveSessionMember(FRemoveSessionMember::Params&& Params) = 0;

	/**
	 * Sends an invite to the session with the given name to all given users.
	 *
	 * @param Parameters for the SendSessionInvite call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FSendSessionInvite> SendSessionInvite(FSendSessionInvite::Params&& Params) = 0;

	/**
	 * Gets a reference to the session invite with the given invite id.
	 *
	 * @param Parameters for the GetSessionInviteById call
	 * @return
	 */
	virtual TOnlineResult<FGetSessionInviteById> GetSessionInviteById(FGetSessionInviteById::Params&& Params) = 0;

	/**
	 * Gets an array of references to all the session invites the given user has received.
	 *
	 * @param Parameters for the SendSessionInvite call
	 * @return
	 */
	virtual TOnlineResult<FGetAllSessionInvites> GetAllSessionInvites(FGetAllSessionInvites::Params&& Params) = 0;

	/**
	 * Rejects the session invite with the given invite id.
	 *
	 * @param Parameters for the RejectSessionInvite call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FRejectSessionInvite> RejectSessionInvite(FRejectSessionInvite::Params&& Params) = 0;

	/* Events */
	
	/**
	 * This event will trigger as a result of a session being created.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FSessionCreated &)> OnSessionCreated() = 0;
	/**
	 * This event will trigger as a result of joining a session.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FSessionJoined&)> OnSessionJoined() = 0;

	/**
	 * This event will trigger as a result of leaving or destroying a session.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FSessionLeft&)> OnSessionLeft() = 0;

	/**
	 * This event will trigger as a result of updating a session's settings, or whenever a session update event is received by the API.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FSessionUpdated&)> OnSessionUpdated() = 0;

	/**
	 * This event will trigger as a result of receiving a session invite.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FSessionInviteReceived&)> OnSessionInviteReceived() = 0;

	/**
	 * This event will trigger as a result of accepting a session invite or joining a session via the platform UI.
	 *
	 * @return
	 */
	virtual TOnlineEvent<void(const FUISessionJoinRequested&)> OnUISessionJoinRequested() = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FFindSessionsSearchFilter)
	ONLINE_STRUCT_FIELD(FFindSessionsSearchFilter, Key),
	ONLINE_STRUCT_FIELD(FFindSessionsSearchFilter, ComparisonOp),
	ONLINE_STRUCT_FIELD(FFindSessionsSearchFilter, Value)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCustomSessionSetting)
	ONLINE_STRUCT_FIELD(FCustomSessionSetting, Data),
	ONLINE_STRUCT_FIELD(FCustomSessionSetting, Visibility),
	ONLINE_STRUCT_FIELD(FCustomSessionSetting, ID)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCustomSessionSettingUpdate)
	ONLINE_STRUCT_FIELD(FCustomSessionSettingUpdate, OldValue),
	ONLINE_STRUCT_FIELD(FCustomSessionSettingUpdate, NewValue)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionSettings)
	ONLINE_STRUCT_FIELD(FSessionSettings, SchemaName),
	ONLINE_STRUCT_FIELD(FSessionSettings, NumMaxConnections),
	ONLINE_STRUCT_FIELD(FSessionSettings, JoinPolicy),
	ONLINE_STRUCT_FIELD(FSessionSettings, bAllowNewMembers),
	ONLINE_STRUCT_FIELD(FSessionSettings, CustomSettings)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionInfo)
	ONLINE_STRUCT_FIELD(FSessionInfo, SessionId),
	ONLINE_STRUCT_FIELD(FSessionInfo, SessionIdOverride),
	ONLINE_STRUCT_FIELD(FSessionInfo, bIsLANSession),
	ONLINE_STRUCT_FIELD(FSessionInfo, bIsDedicatedServerSession),
	ONLINE_STRUCT_FIELD(FSessionInfo, bAllowSanctionedPlayers),
	ONLINE_STRUCT_FIELD(FSessionInfo, bAntiCheatProtected)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionSettingsUpdate)
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, SchemaName),
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, NumMaxConnections),
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, JoinPolicy),
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, bAllowNewMembers),
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, UpdatedCustomSettings),
	ONLINE_STRUCT_FIELD(FSessionSettingsUpdate, RemovedCustomSettings)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAllSessions::Params)
	ONLINE_STRUCT_FIELD(FGetAllSessions::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAllSessions::Result)
	ONLINE_STRUCT_FIELD(FGetAllSessions::Result, Sessions)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionByName::Params)
	ONLINE_STRUCT_FIELD(FGetSessionByName::Params, LocalName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionByName::Result)
	ONLINE_STRUCT_FIELD(FGetSessionByName::Result, Session)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionById::Params)
	ONLINE_STRUCT_FIELD(FGetSessionById::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionById::Result)
	ONLINE_STRUCT_FIELD(FGetSessionById::Result, Session)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionName::Params)
	ONLINE_STRUCT_FIELD(FGetSessionName::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionName::Result)
	ONLINE_STRUCT_FIELD(FGetSessionName::Result, SessionName)
END_ONLINE_STRUCT_META()


BEGIN_ONLINE_STRUCT_META(FGetPresenceSession::Params)
	ONLINE_STRUCT_FIELD(FGetPresenceSession::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetPresenceSession::Result)
	ONLINE_STRUCT_FIELD(FGetPresenceSession::Result, Session)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FIsPresenceSession::Params)
ONLINE_STRUCT_FIELD(FIsPresenceSession::Params, LocalAccountId),
ONLINE_STRUCT_FIELD(FIsPresenceSession::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FIsPresenceSession::Result)
	ONLINE_STRUCT_FIELD(FIsPresenceSession::Result, bIsPresenceSession)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSetPresenceSession::Params)
	ONLINE_STRUCT_FIELD(FSetPresenceSession::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSetPresenceSession::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSetPresenceSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FClearPresenceSession::Params)
	ONLINE_STRUCT_FIELD(FClearPresenceSession::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FClearPresenceSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCreateSession::Params)
	ONLINE_STRUCT_FIELD(FCreateSession::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, SessionName),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, SessionIdOverride),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, bPresenceEnabled),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, bIsLANSession),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, bAllowSanctionedPlayers),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, bAntiCheatProtected),
	ONLINE_STRUCT_FIELD(FCreateSession::Params, SessionSettings)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FCreateSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateSessionSettings::Params)
	ONLINE_STRUCT_FIELD(FUpdateSessionSettings::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUpdateSessionSettings::Params, SessionName),
	ONLINE_STRUCT_FIELD(FUpdateSessionSettings::Params, Mutations)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateSessionSettings::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLeaveSession::Params)
	ONLINE_STRUCT_FIELD(FLeaveSession::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FLeaveSession::Params, SessionName),
	ONLINE_STRUCT_FIELD(FLeaveSession::Params, bDestroySession)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FLeaveSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFindSessions::Params)
	ONLINE_STRUCT_FIELD(FFindSessions::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FFindSessions::Params, MaxResults),
	ONLINE_STRUCT_FIELD(FFindSessions::Params, bFindLANSessions),	
	ONLINE_STRUCT_FIELD(FFindSessions::Params, Filters),
	ONLINE_STRUCT_FIELD(FFindSessions::Params, TargetUser),
	ONLINE_STRUCT_FIELD(FFindSessions::Params, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FFindSessions::Result)
	ONLINE_STRUCT_FIELD(FFindSessions::Result, FoundSessionIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FStartMatchmaking::Params)
	ONLINE_STRUCT_FIELD(FStartMatchmaking::Params, SessionCreationParameters),
	ONLINE_STRUCT_FIELD(FStartMatchmaking::Params, SessionSearchFilters)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FStartMatchmaking::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FJoinSession::Params)
	ONLINE_STRUCT_FIELD(FJoinSession::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FJoinSession::Params, SessionName),
	ONLINE_STRUCT_FIELD(FJoinSession::Params, SessionId),
	ONLINE_STRUCT_FIELD(FJoinSession::Params, bPresenceEnabled),
	ONLINE_STRUCT_FIELD(FJoinSession::Params, bJoinAsSpectator)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FJoinSession::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAddSessionMember::Params)
	ONLINE_STRUCT_FIELD(FAddSessionMember::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAddSessionMember::Params, SessionName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAddSessionMember::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRemoveSessionMember::Params)
	ONLINE_STRUCT_FIELD(FRemoveSessionMember::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FRemoveSessionMember::Params, TargetAccountId),
	ONLINE_STRUCT_FIELD(FRemoveSessionMember::Params, SessionName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRemoveSessionMember::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendSessionInvite::Params)
	ONLINE_STRUCT_FIELD(FSendSessionInvite::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSendSessionInvite::Params, SessionName),
	ONLINE_STRUCT_FIELD(FSendSessionInvite::Params, TargetUsers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendSessionInvite::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionInviteById::Params)
	ONLINE_STRUCT_FIELD(FGetSessionInviteById::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetSessionInviteById::Params, SessionInviteId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetSessionInviteById::Result)
ONLINE_STRUCT_FIELD(FGetSessionInviteById::Result, SessionInvite)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAllSessionInvites::Params)
	ONLINE_STRUCT_FIELD(FGetAllSessionInvites::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetAllSessionInvites::Result)
	ONLINE_STRUCT_FIELD(FGetAllSessionInvites::Result, SessionInvites)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRejectSessionInvite::Params)
	ONLINE_STRUCT_FIELD(FRejectSessionInvite::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FRejectSessionInvite::Params, SessionInviteId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRejectSessionInvite::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionSettingsChanges)
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, SchemaName),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, NumMaxConnections),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, JoinPolicy),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, bAllowNewMembers),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, AddedCustomSettings),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, ChangedCustomSettings),
	ONLINE_STRUCT_FIELD(FSessionSettingsChanges, RemovedCustomSettings)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionUpdate)
	ONLINE_STRUCT_FIELD(FSessionUpdate, OwnerAccountId),
	ONLINE_STRUCT_FIELD(FSessionUpdate, SessionSettingsChanges),
	ONLINE_STRUCT_FIELD(FSessionUpdate, AddedSessionMembers),
	ONLINE_STRUCT_FIELD(FSessionUpdate, RemovedSessionMembers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionCreated)
	ONLINE_STRUCT_FIELD(FSessionCreated, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSessionCreated, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionJoined)
ONLINE_STRUCT_FIELD(FSessionJoined, LocalAccountId),
ONLINE_STRUCT_FIELD(FSessionJoined, SessionId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSessionLeft)
	ONLINE_STRUCT_FIELD(FSessionCreated, LocalAccountId)
END_ONLINE_STRUCT_META()
/* Meta*/ }

/* UE::Online */ }

#undef UE_API
