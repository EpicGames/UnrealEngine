// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

/**
 * Social relationship status
 */
enum class ERelationship : uint8
{
	/* Friend */
	Friend,
	/* Not friend */
	NotFriend,
	/* Invite Sent to User */
	InviteSent,
	/* Invite Received from User */
	InviteReceived,
	/* Local user has Blocked the other user */
	Blocked
};

const TCHAR* LexToString(ERelationship Relationship);

/**
 * Information about a friend
 */
struct FFriend
{
	/* Account Id of the Friend */
	FAccountId FriendId;
	/* Display Name of the Friend */
	FString DisplayName;
	/* Local Nickname for the Friend, Not always available */
	FString Nickname;
	/* Relationship of the friend */
	ERelationship Relationship;
};

struct FQueryFriends
{
	static constexpr TCHAR Name[] = TEXT("QueryFriends");

	/** Input struct for Social::QueryFriends */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
	};

	/**
	 * Output struct for Social::QueryFriends
	 * Obtain the friends list via GetFriends
	 */
	struct Result
	{
	};
};

struct FGetFriends
{
	static constexpr TCHAR Name[] = TEXT("GetFriends");

	/** Input struct for Social::GetFriends */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
	};

	/** Output struct for Social::GetFriends */
	struct Result
	{
		/** Array of friends */
		TArray<TSharedRef<FFriend>> Friends;
	};
};

struct FSendFriendInvite
{
	static constexpr TCHAR Name[] = TEXT("SendFriendInvite");

	/** Input struct for Social::SendFriendInvite */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
		/** Account Id of user to send a friend request to */
		FAccountId TargetAccountId;
	};

	/** Output struct for Social::SendFriendInvite */
	struct Result
	{
	};
};

struct FAcceptFriendInvite
{
	static constexpr TCHAR Name[] = TEXT("AcceptFriendInvite");

	/** Input struct for Social::AcceptFriendInvite */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
		/** Account Id of user that sent the invite */
		FAccountId TargetAccountId;
	};

	/** Output struct for Social::AcceptFriendInvite */
	struct Result
	{
	};
};

struct FRejectFriendInvite
{
	static constexpr TCHAR Name[] = TEXT("RejectFriendInvite");

	/** Input struct for Social::RejectFriendInvite */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
		/** Account Id of user that sent the invite */
		FAccountId TargetAccountId;
	};

	/** Output struct for Social::RejectFriendInvite */
	struct Result
	{
	};
};

/** Struct for RelationshipUpdated event */
struct FRelationshipUpdated
{
	/* Account Id of the Local User whose friends list changed */
	FAccountId LocalAccountId;
	/* Account Id of the User whose status changed */
	FAccountId RemoteAccountId;
	/* Previous relationship */
	ERelationship OldRelationship;
	/*  New relationship */
	ERelationship NewRelationship;
};

struct FQueryBlockedUsers
{
	static constexpr TCHAR Name[] = TEXT("QueryBlockedUsers");

	/** Input struct for Social::QueryBlockedUsers */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
	};

	/**
	 * Output struct for Social::QueryBlockedUsers
	 * Obtain the blocked users list via GetBlockedUsers
	 */
	struct Result
	{
	};
};

struct FGetBlockedUsers
{
	static constexpr TCHAR Name[] = TEXT("GetBlockedUsers");

	/** Input struct for Social::GetBlockedUsers */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
	};

	/** Output struct for Social::GetBlockedUsers */
	struct Result
	{
		/** Array of blocked users */
		TArray<FAccountId> BlockedUsers;
	};
};

struct FBlockUser
{
	static constexpr TCHAR Name[] = TEXT("BlockUser");

	/** Input struct for Social::BlockUser */
	struct Params
	{
		/** Account Id of the local user making the request */
		FAccountId LocalAccountId;
		/** Account Id of user to block */
		FAccountId TargetAccountId;
	};

	/** Output struct for Social::BlockUser */
	struct Result
	{
	};
};

class ISocial
{
public:
	virtual ~ISocial() = default;

	/**
	 * Query the friends list
	 * 
	 * @param Params for the QueryFriends call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FQueryFriends> QueryFriends(FQueryFriends::Params&& Params) = 0;

	/**
	 * Get the contents of a previously queried friends list
	 *
	 * @param Params for the GetFriends call
	 * @return Result
	 */
	virtual TOnlineResult<FGetFriends> GetFriends(FGetFriends::Params&& Params) = 0;

	/**
	 * Send a friend invite
	 *
	 * @param Params for the SendFriendInvite call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FSendFriendInvite> SendFriendInvite(FSendFriendInvite::Params&& Params) = 0;

	/**
	 * Accept a friend invite
	 *
	 * @param Params for the AcceptFriendInvite call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FAcceptFriendInvite> AcceptFriendInvite(FAcceptFriendInvite::Params&& Params) = 0;

	/**
	 * Reject a friend invite
	 *
	 * @param Params for the RejectFriendInvite call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FRejectFriendInvite> RejectFriendInvite(FRejectFriendInvite::Params&& Params) = 0;

	/**
	 * Get the event that is triggered when a friends list is updated
	 * This typically happens when QueryFriends is called, a friend or block list modifying function is called,
	 *   or is called in response to an event coming from a backend service
	 *
	 * @return Event that can be bound to
	 */
	virtual TOnlineEvent<void(const FRelationshipUpdated&)> OnRelationshipUpdated() = 0;

	/**
	 * Query the blocked users list
	 *
	 * @param Params for the QueryBlockedUsers call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FQueryBlockedUsers> QueryBlockedUsers(FQueryBlockedUsers::Params&& Params) = 0;

	/**
	 * Get the contents of a previously queried blocked users list
	 *
	 * @param Params for the GetBlockedUsers call
	 * @return Result
	 */
	virtual TOnlineResult<FGetBlockedUsers> GetBlockedUsers(FGetBlockedUsers::Params&& Params) = 0;

	/**
	 * Block a user
	 *
	 * @param Params for the BlockUser call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FBlockUser> BlockUser(FBlockUser::Params&& Params) = 0;
};

inline const TCHAR* LexToString(ERelationship Relationship)
{
	switch (Relationship)
	{
	case ERelationship::Friend:
		return TEXT("Friend");
	case ERelationship::NotFriend:
		return TEXT("NotFriend");
	case ERelationship::InviteSent:
		return TEXT("InviteSent");
	case ERelationship::InviteReceived:
		return TEXT("InviteReceived");
	case ERelationship::Blocked:
		return TEXT("Blocked");
	}
	return TEXT("Invalid");
}

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FFriend)
	ONLINE_STRUCT_FIELD(FFriend, FriendId),
	ONLINE_STRUCT_FIELD(FFriend, DisplayName),
	ONLINE_STRUCT_FIELD(FFriend, Nickname),
	ONLINE_STRUCT_FIELD(FFriend, Relationship)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryFriends::Params)
	ONLINE_STRUCT_FIELD(FQueryFriends::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryFriends::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetFriends::Params)
	ONLINE_STRUCT_FIELD(FGetFriends::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetFriends::Result)
	ONLINE_STRUCT_FIELD(FGetFriends::Result, Friends)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendFriendInvite::Params)
	ONLINE_STRUCT_FIELD(FSendFriendInvite::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FSendFriendInvite::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FSendFriendInvite::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAcceptFriendInvite::Params)
	ONLINE_STRUCT_FIELD(FAcceptFriendInvite::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FAcceptFriendInvite::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAcceptFriendInvite::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRejectFriendInvite::Params)
	ONLINE_STRUCT_FIELD(FRejectFriendInvite::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FRejectFriendInvite::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRejectFriendInvite::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FRelationshipUpdated)
	ONLINE_STRUCT_FIELD(FRelationshipUpdated, LocalAccountId),
	ONLINE_STRUCT_FIELD(FRelationshipUpdated, RemoteAccountId),
	ONLINE_STRUCT_FIELD(FRelationshipUpdated, OldRelationship),
	ONLINE_STRUCT_FIELD(FRelationshipUpdated, NewRelationship)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryBlockedUsers::Params)
	ONLINE_STRUCT_FIELD(FQueryBlockedUsers::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryBlockedUsers::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetBlockedUsers::Params)
	ONLINE_STRUCT_FIELD(FGetBlockedUsers::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetBlockedUsers::Result)
	ONLINE_STRUCT_FIELD(FGetBlockedUsers::Result, BlockedUsers)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBlockUser::Params)
	ONLINE_STRUCT_FIELD(FBlockUser::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FBlockUser::Params, TargetAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBlockUser::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
