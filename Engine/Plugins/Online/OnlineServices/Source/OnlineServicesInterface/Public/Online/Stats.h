// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/SchemaTypes.h"

namespace UE::Online { template <typename DelegateSignature> class TOnlineEvent; }
namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }

namespace UE::Online {

using FStatValue = FSchemaVariant;
using FOnlineEventParams =  TMap<FString, FSchemaVariant>;

struct FUserStats
{
	/* The user id */
	FAccountId AccountId;
	/* The stats of one account */
	TMap<FString, FStatValue> Stats;
};

struct FUpdateStats
{
	static constexpr TCHAR Name[] = TEXT("UpdateStats");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The stats of users to update */
		TArray<FUserStats> UpdateUsersStats;
	};

	struct Result
	{
	};
};

struct FQueryStats
{
	static constexpr TCHAR Name[] = TEXT("QueryStats");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* Target user id to query */
		FAccountId TargetAccountId;
		/* The stats to query */
		TArray<FString> StatNames;
	};

	struct Result
	{
		/* The stats of queried user */
		TMap<FString, FStatValue> Stats;
	};
};

struct FBatchQueryStats
{
	static constexpr TCHAR Name[] = TEXT("BatchQueryStats");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The user ids to query */
		TArray<FAccountId> TargetAccountIds;
		/* The stats to query */
		TArray<FString> StatNames;
	};

	struct Result
	{
		/* The stats of queried users, there should be no much elements so simply use TArray instead of TMap */
		TArray<FUserStats> UsersStats;
	};
};

struct FTriggerEvent
{
	static constexpr TCHAR Name[] = TEXT("TriggerEvent");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
		/* The name of the event to fire */
		FString EventName;
		/* The events parameters if any */
		FOnlineEventParams EventParams;
	};

	struct Result
	{
	};
};

#if !UE_BUILD_SHIPPING
struct FResetStats
{
	static constexpr TCHAR Name[] = TEXT("ResetStats");

	struct Params
	{
		/* Local user id */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};
#endif // !UE_BUILD_SHIPPING

struct FGetCachedStats
{
	static constexpr TCHAR Name[] = TEXT("GetCachedStats");

	struct Params
	{
	};

	struct Result
	{
		TArray<FUserStats> UsersStats;
	};
};

using FStatsUpdated = FUpdateStats::Params;

// Stats interface is used to upload the stats to EOS or first party platforms, to complete corresponding InGame/Platform features such as Stats Query, Achievements, Leaderboard, etc.
class IStats
{
public:
	virtual ~IStats() = default;

	/**
	 * Upload/send stats to the backend/platform.
	 *
	 * @param Params for the UpdateStats call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FUpdateStats> UpdateStats(FUpdateStats::Params&& Params) = 0;
	/**
	 * Get stats of the specified user.
	 *
	 * @param Params for the QueryStats call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FQueryStats> QueryStats(FQueryStats::Params&& Params) = 0;
	/**
	 * Get stats of the specified users.
	 *
	 * @param Params for the BatchQueryStats call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FBatchQueryStats> BatchQueryStats(FBatchQueryStats::Params&& Params) = 0;
	/**
	 * Trigger a stat modifying event
	 *
	 * @param Params for the TriggerEvent call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FTriggerEvent> TriggerEvent(FTriggerEvent::Params&& Params) = 0;

#if !UE_BUILD_SHIPPING
	/**
	 * Reset stats of the local user.
	 *
	 * @param Params for the ResetStats call
	 * @return
	 */
	virtual TOnlineAsyncOpHandle<FResetStats> ResetStats(FResetStats::Params&& Params) = 0;
#endif // !UE_BUILD_SHIPPING

	/**
	 * Event triggered when stats of user(s) changed
	 */
	virtual TOnlineEvent<void(const FStatsUpdated&)> OnStatsUpdated() = 0;

	/**
	 * Retrieve cached users' stats, which was retrieved when call QueryStats or BatchQueryStats, or after 
	 * calling UpdateStats. When choose to use StatsNull, make sure to listen to OnStatsUpdated
	 * event and use this function to get all latest local stats and put them into SaveGame
	 */
	virtual TOnlineResult<FGetCachedStats> GetCachedStats(FGetCachedStats::Params&& Params) const = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUserStats)
	ONLINE_STRUCT_FIELD(FUserStats, AccountId),
	ONLINE_STRUCT_FIELD(FUserStats, Stats)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateStats::Params)
	ONLINE_STRUCT_FIELD(FUpdateStats::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUpdateStats::Params, UpdateUsersStats)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUpdateStats::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryStats::Params)
	ONLINE_STRUCT_FIELD(FQueryStats::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FQueryStats::Params, TargetAccountId),
	ONLINE_STRUCT_FIELD(FQueryStats::Params, StatNames)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryStats::Result)
	ONLINE_STRUCT_FIELD(FQueryStats::Result, Stats)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTriggerEvent::Params)
	ONLINE_STRUCT_FIELD(FTriggerEvent::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FTriggerEvent::Params, EventName),
	ONLINE_STRUCT_FIELD(FTriggerEvent::Params, EventParams)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTriggerEvent::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBatchQueryStats::Params)
ONLINE_STRUCT_FIELD(FBatchQueryStats::Params, LocalAccountId),
ONLINE_STRUCT_FIELD(FBatchQueryStats::Params, TargetAccountIds),
ONLINE_STRUCT_FIELD(FBatchQueryStats::Params, StatNames)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FBatchQueryStats::Result)
	ONLINE_STRUCT_FIELD(FBatchQueryStats::Result, UsersStats)
END_ONLINE_STRUCT_META()

#if !UE_BUILD_SHIPPING
BEGIN_ONLINE_STRUCT_META(FResetStats::Params)
	ONLINE_STRUCT_FIELD(FResetStats::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FResetStats::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetCachedStats::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetCachedStats::Result)
	ONLINE_STRUCT_FIELD(FGetCachedStats::Result, UsersStats)
END_ONLINE_STRUCT_META()
#endif // !UE_BUILD_SHIPPING

/* Meta*/ }

/* UE::Online */ }
