// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

struct FTitleFileEnumerateFiles
{
	static constexpr TCHAR Name[] = TEXT("EnumerateFiles");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FTitleFileGetEnumeratedFiles
{
	static constexpr TCHAR Name[] = TEXT("GetEnumeratedFiles");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Available files */
		TArray<FString> Filenames;
	};
};

using FTitleFileContents = TArray<uint8>;
using FTitleFileContentsRef = TSharedRef<const FTitleFileContents>;
using FTitleFileContentsPtr = TSharedPtr<const FTitleFileContents>;
using FTitleFileContentsWeakPtr = TWeakPtr<const FTitleFileContents>;

struct FTitleFileReadFile
{
	static constexpr TCHAR Name[] = TEXT("ReadFile");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** File to be opened */
		FString Filename;
	};

	struct Result
	{
		/** Contents of the file */
		FTitleFileContentsRef FileContents;
	};
};

class ITitleFile
{
public:
	virtual ~ITitleFile() = default;

	/**
	 * Enumerate all available files
	 */
	virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) = 0;

	/**
	 * Get the cached list of files enumerated by previous call to EnumerateFiles
	 */
	virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) = 0;

	/**
	 * Read a file and return the file contents
	 */
	virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) = 0;
};


namespace Meta {

BEGIN_ONLINE_STRUCT_META(FTitleFileEnumerateFiles::Params)
	ONLINE_STRUCT_FIELD(FTitleFileEnumerateFiles::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileEnumerateFiles::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileGetEnumeratedFiles::Params)
	ONLINE_STRUCT_FIELD(FTitleFileGetEnumeratedFiles::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileGetEnumeratedFiles::Result)
	ONLINE_STRUCT_FIELD(FTitleFileGetEnumeratedFiles::Result, Filenames)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileReadFile::Params)
	ONLINE_STRUCT_FIELD(FTitleFileReadFile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FTitleFileReadFile::Params, Filename)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileReadFile::Result)
	ONLINE_STRUCT_FIELD(FTitleFileReadFile::Result, FileContents)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
