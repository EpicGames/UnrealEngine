// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

struct FUserFileEnumerateFiles
{
	static constexpr TCHAR Name[] = TEXT("EnumerateFiles");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
	};
};

struct FUserFileGetEnumeratedFiles
{
	static constexpr TCHAR Name[] = TEXT("GetEnumeratedFiles");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
	};

	struct Result
	{
		/** Available files for the local user */
		TArray<FString> Filenames;
	};
};

using FUserFileContents = TArray<uint8>;
using FUserFileContentsRef = TSharedRef<const FUserFileContents>;
using FUserFileContentsPtr = TSharedPtr<const FUserFileContents>;
using FUserFileContentsWeakPtr = TWeakPtr<const FUserFileContents>;

struct FUserFileReadFile
{
	static constexpr TCHAR Name[] = TEXT("ReadFile");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** File to be read */
		FString Filename;
	};

	struct Result
	{
		/** Contents of the file */
		FUserFileContentsRef FileContents;
	};
};

struct FUserFileWriteFile
{
	static constexpr TCHAR Name[] = TEXT("WriteFile");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** File to be written */
		FString Filename;
		/** Data to write */
		FUserFileContents FileContents;
	};

	struct Result
	{
	};
};

struct FUserFileCopyFile
{
	static constexpr TCHAR Name[] = TEXT("CopyFile");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** File to copy contents from */
		FString SourceFilename;
		/** File to copy contents to */
		FString TargetFilename;
	};

	struct Result
	{
	};
};

struct FUserFileDeleteFile
{
	static constexpr TCHAR Name[] = TEXT("DeleteFile");

	struct Params
	{
		/** Local user performing the operation */
		FAccountId LocalAccountId;
		/** File to delete */
		FString Filename;
	};

	struct Result
	{
	};
};

class IUserFile
{
public:
	virtual ~IUserFile() = default;

	/**
	 * Enumerate all available files
	 */
	virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) = 0;

	/**
	 * Get the cached list of files enumerated by previous call to EnumerateFiles
	 */
	virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) = 0;

	/**
	 * Read a file and return the file contents
	 */
	virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) = 0;

	/**
	 * Write file contents to a file.
	 */
	virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) = 0;

	/**
	 * Copy file contents to another file
	 */
	virtual TOnlineAsyncOpHandle<FUserFileCopyFile> CopyFile(FUserFileCopyFile::Params&& Params) = 0;

	/**
	 * Delete a file
	 */
	virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUserFileEnumerateFiles::Params)
	ONLINE_STRUCT_FIELD(FUserFileEnumerateFiles::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileEnumerateFiles::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileGetEnumeratedFiles::Params)
	ONLINE_STRUCT_FIELD(FUserFileGetEnumeratedFiles::Params, LocalAccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileGetEnumeratedFiles::Result)
	ONLINE_STRUCT_FIELD(FUserFileGetEnumeratedFiles::Result, Filenames)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileReadFile::Params)
	ONLINE_STRUCT_FIELD(FUserFileReadFile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUserFileReadFile::Params, Filename)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileReadFile::Result)
	ONLINE_STRUCT_FIELD(FUserFileReadFile::Result, FileContents)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileWriteFile::Params)
	ONLINE_STRUCT_FIELD(FUserFileWriteFile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUserFileWriteFile::Params, Filename),
	ONLINE_STRUCT_FIELD(FUserFileWriteFile::Params, FileContents)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileWriteFile::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileCopyFile::Params)
	ONLINE_STRUCT_FIELD(FUserFileCopyFile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUserFileCopyFile::Params, SourceFilename),
	ONLINE_STRUCT_FIELD(FUserFileCopyFile::Params, TargetFilename)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileCopyFile::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileDeleteFile::Params)
	ONLINE_STRUCT_FIELD(FUserFileDeleteFile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FUserFileDeleteFile::Params, Filename)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FUserFileDeleteFile::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
