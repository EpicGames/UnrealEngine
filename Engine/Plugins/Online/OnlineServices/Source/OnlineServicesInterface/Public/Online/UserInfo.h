// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Online/OnlineMeta.h"
#include "Online/OnlineServices.h"

namespace UE::Online { template <typename OpType> class TOnlineAsyncOpHandle; }
namespace UE::Online { template <typename OpType> class TOnlineResult; }
namespace UE::Online::Meta { template <typename StructType> struct TStructDetails; }

namespace UE::Online {

struct FUserInfo
{
	/* Account Id */
	FAccountId AccountId;
	/* Display Name of the User */
	FString DisplayName;
};

struct FQueryUserInfo
{
	static constexpr TCHAR Name[] = TEXT("QueryUserInfo");

	/** Input struct for UserInfo::QueryUserInfo */
	struct Params
	{
		/* Local user to query users for */
		FAccountId LocalAccountId;
		/* List of User ids to Query*/
		TArray<FAccountId> AccountIds;
	};

	/**
	 * Output struct for UserInfo::QueryUserInfo
	 * Obtain the queried user info via GetUserInfo
	 */
	struct Result
	{
	};
};

struct FGetUserInfo
{
	static constexpr TCHAR Name[] = TEXT("GetUserInfo");

	/** Input struct for UserInfo::GetUserInfo */
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* User id to get info for*/
		FAccountId AccountId;
		/* Preferred platform for DisplayName*/
		TOptional<EOnlinePlatformType> DisplayNamePlatform;
	};

	/**
	 * Output struct for UserInfo::GetUserInfo
	 */
	struct Result
	{
		/* User info */
		TSharedRef<FUserInfo> UserInfo;
	};
};

struct FQueryUserAvatar
{
	static constexpr TCHAR Name[] = TEXT("QueryUserAvatar");

	/** Input struct for UserInfo::QueryUserAvatar */
	struct Params
	{
		/* Local user to query users for */
		FAccountId LocalAccountId;
		/* List of User ids to Query*/
		TArray<FAccountId> AccountIds;
	};

	/**
	 * Output struct for UserInfo::QueryUserAvatar
	 * Obtain the queried user info via GetUserAvatar
	 */
	struct Result
	{
	};
};

struct FGetUserAvatar
{
	static constexpr TCHAR Name[] = TEXT("GetUserAvatar");

	/** Input struct for UserInfo::GetUserAvatar */
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* User id to get avatar for*/
		FAccountId AccountId;
	};

	/**
	 * Output struct for UserInfo::GetUserAvatar
	 */
	struct Result
	{
		/* Avatar Url */
		FString AvatarUrl;
	};
};

struct FShowUserProfile
{
	static constexpr TCHAR Name[] = TEXT("ShowUserProfile");

	/** Input struct for UserInfo::ShowUserProfile */
	struct Params
	{
		/* Local user */
		FAccountId LocalAccountId;
		/* User id to display the profile for*/
		FAccountId AccountId;
	};

	/**
	 * Output struct for UserInfo::ShowUserProfile
	 */
	struct Result
	{
	};
};

class IUserInfo
{
public:
	virtual ~IUserInfo() = default;

	/*
	 * Queries the user info for a list of account ids.
	 * Upon success, user info can be accessed with GetUserInfo.
	 * @see FUserInfo
	 * @see GetUserInfo
	 * 
	 * @params Params for the QueryUserInfo call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FQueryUserInfo> QueryUserInfo(FQueryUserInfo::Params&& Params) = 0;

	/*
	 * Get the user info for a previously queried AccountId
	 * @see FUserInfo
	 * @see QueryUserInfo
	 *
	 * @params Params for the GetUserInfo call
	 * @return Result
	 */
	virtual TOnlineResult<FGetUserInfo> GetUserInfo(FGetUserInfo::Params&& Params) = 0;

	/*
	 * Queries the user avatars for a list of account ids
	 * Upon success, avatars can be accessed with GetUserAvatar.
	 * @see GetUserAvatar
	 *
	 * @params Params for the QueryUserAvatar call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FQueryUserAvatar> QueryUserAvatar(FQueryUserAvatar::Params&& Params) = 0;

	/*
	 * Get the user avatar for a previously queried AccountId
	 * @see QueryUserAvatar
	 *
	 * @params Params for the GetUserAvatar call
	 * @return Result
	 */
	virtual TOnlineResult<FGetUserAvatar> GetUserAvatar(FGetUserAvatar::Params&& Params) = 0;

	/*
	 * Show the profile UI for a user
	 *
	 * @params Params for the ShowUserProfile call
	 * @return AsyncOpHandle
	 */
	virtual TOnlineAsyncOpHandle<FShowUserProfile> ShowUserProfile(FShowUserProfile::Params&& Params) = 0;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FUserInfo)
	ONLINE_STRUCT_FIELD(FUserInfo, AccountId),
	ONLINE_STRUCT_FIELD(FUserInfo, DisplayName)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryUserInfo::Params)
	ONLINE_STRUCT_FIELD(FQueryUserInfo::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FQueryUserInfo::Params, AccountIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryUserInfo::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetUserInfo::Params)
	ONLINE_STRUCT_FIELD(FGetUserInfo::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetUserInfo::Params, AccountId),
	ONLINE_STRUCT_FIELD(FGetUserInfo::Params, DisplayNamePlatform)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetUserInfo::Result)
	ONLINE_STRUCT_FIELD(FGetUserInfo::Result, UserInfo)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryUserAvatar::Params)
	ONLINE_STRUCT_FIELD(FQueryUserAvatar::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FQueryUserAvatar::Params, AccountIds)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FQueryUserAvatar::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetUserAvatar::Params)
	ONLINE_STRUCT_FIELD(FGetUserAvatar::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FGetUserAvatar::Params, AccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FGetUserAvatar::Result)
	ONLINE_STRUCT_FIELD(FGetUserAvatar::Result, AvatarUrl)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FShowUserProfile::Params)
	ONLINE_STRUCT_FIELD(FShowUserProfile::Params, LocalAccountId),
	ONLINE_STRUCT_FIELD(FShowUserProfile::Params, AccountId)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FShowUserProfile::Result)
END_ONLINE_STRUCT_META()

/* Meta*/ }

/* UE::Online */ }
