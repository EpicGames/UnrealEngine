// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/OnlineError.h"
#include "Online/OnlineErrorCode.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "winerror.h"
#include "winbase.h"

#define LOCTEXT_NAMESPACE "OnlineErrors"

namespace UE::Online::Errors {

	// I really wanted this to be "HRESULT" to match what the error actually is, however this results in some redefinition since hresult is also a type.. 
	UE_ONLINE_ERROR_CATEGORY(Windows, Engine, 0x2, "Windows")
	UE_ONLINE_ERROR(Windows, Abort,			0x80004004,	TEXT("E_ABORT"),			LOCTEXT("E_ABORT", "Operation aborted"))
	UE_ONLINE_ERROR(Windows, AccessDenied,		0x80070005, TEXT("E_ACCESSDENIED"),		LOCTEXT("E_ACCESSDENIED", "General access denied error"))
	UE_ONLINE_ERROR(Windows, Fail,				0x80004005, TEXT("E_FAIL"),				LOCTEXT("E_FAIL", "Unspecified failure"))
	UE_ONLINE_ERROR(Windows, Handle,			0x80070006, TEXT("E_HANDLE"),			LOCTEXT("E_HANDLE", "Handle that is not valid"))
	UE_ONLINE_ERROR(Windows, InvalidArg,		0x80070057, TEXT("E_INVALIDARG"),		LOCTEXT("E_INVALIDARG", "One or more arguments are not valid"))
	UE_ONLINE_ERROR(Windows, NoInterface,	 	0x80004002, TEXT("E_NOINTERFACE"),		LOCTEXT("E_NOINTERFACE", "No such interface supported"))
	UE_ONLINE_ERROR(Windows, NotImpl, 			0x80004001, TEXT("E_NOTIMPL"),			LOCTEXT("E_NOTIMPL", "Not implemented"))
	UE_ONLINE_ERROR(Windows, OutOfMemory, 		0x8007000E, TEXT("E_OUTOFMEMORY"),		LOCTEXT("E_OUTOFMEMORY", "Failed to allocate necessary memory"))
	UE_ONLINE_ERROR(Windows, Pointer, 			0x80004003, TEXT("E_POINTER"),			LOCTEXT("E_POINTER", "Pointer that is not valid"))
	UE_ONLINE_ERROR(Windows, Unexpected,	 	0x8000FFFF, TEXT("E_UNEXPECTED"),		LOCTEXT("E_UNEXPECTED", "Unexpected failure"))


	inline FString Internal_HResultToString(HRESULT Result)
	{
		TCHAR Buffer[1024];
		Buffer[0] = TEXT('\0');
		DWORD BufferLength = 0;

		// resolve error code
		BufferLength = FormatMessage(FORMAT_MESSAGE_FROM_SYSTEM | FORMAT_MESSAGE_IGNORE_INSERTS, NULL, Result, MAKELANGID(LANG_NEUTRAL, SUBLANG_DEFAULT), Buffer, 1024, NULL);

		if (BufferLength == 0)
		{
			return FString::Printf(TEXT("0x%08x"), Result);
		}

		// remove line break
		TCHAR* NewLine = FCString::Strchr(Buffer, TEXT('\r'));

		if (NewLine != nullptr)
		{
			*NewLine = TEXT('\0');
		}

		return Buffer;
	}

	inline ErrorCodeType ErrorValueFromHRESULT(HRESULT Result)
	{
		return (ErrorCodeType)((uint32)Result);
	}

	inline ErrorCodeType ErrorCodeFromHRESULT(HRESULT Result)
	{
		return ErrorCode::Create(ErrorCode::System::Engine, ErrorCode::Category::Windows, ErrorValueFromHRESULT(Result));
	}

	/// Errors created from this will be equal to the predefined types in ErrorCodes::Windows if they exist.
	inline FOnlineError FromHRESULT(HRESULT Result)
	{
		if (Result == S_OK)
		{
			return Success();
		}

		// This isn't localized, but Windows doesn't even have localized error codes anyway- do we need to add some method for localization? (e.g. try to look up the error first using some table generated by the WINDOWS_ERROR macro and use that error's FText)
		// todo: can we get the error key somehow? (e.g. not "Operation aborted" which is ErrorString but rather E_ABORT). This is what the predefined errors use for the first parameter
		return FOnlineError(ErrorCodeFromHRESULT(Result), MakeShared<FOnlineErrorDetails, ESPMode::ThreadSafe>(FString::Printf(TEXT("Windows.%llx"), ErrorCodeFromHRESULT(Result)), Internal_HResultToString(Result), FText::FromString(Internal_HResultToString(Result))));
	}


} /* namespace UE::Online::Errors */

#include "Windows/HideWindowsPlatformTypes.h"

#undef LOCTEXT_NAMESPACE